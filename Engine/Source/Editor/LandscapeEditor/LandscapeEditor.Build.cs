// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class LandscapeEditor : ModuleRules
{
	public LandscapeEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Json",
				"ApplicationCore",
				"Documentation",
				"Slate",
				"SlateCore",
				"Engine",
				"Landscape",
                "LandscapeEditorUtilities",
                "RenderCore",
                "RHI",
                "InputCore",
				"ImageCore",
				"EditorFramework",
				"UnrealEd",
				"PropertyEditor",
				"ImageWrapper",
                "EditorWidgets",
                "Foliage",
				"ToolMenus",
				"ToolWidgets",
				"SourceControl",
				"DirectoryWatcher",
				"DeveloperSettings",
				"PlacementMode",
				"CQTest",
				"NavigationSystem",
				"VirtualTexturingEditor",
				"EditorInteractiveToolsFramework",
			}
			);


		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"DesktopPlatform",
                "AssetTools",
				"LevelEditor"
			}
			);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"MainFrame",
				"DesktopPlatform",
			}
			);

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			// VS2015 updated some of the CRT definitions but not all of the Windows SDK has been updated to match.
			// Microsoft provides this shim library to enable building with VS2015 until they fix everything up.
			//@todo: remove when no longer neeeded (no other code changes should be necessary).
			if (Target.WindowsPlatform.bNeedsLegacyStdioDefinitionsLib)
			{
				PublicSystemLibraries.Add("legacy_stdio_definitions.lib");
			}
		}

		// KissFFT is used by the smooth tool.
		if (Target.Platform == UnrealTargetPlatform.Win64 || Target.Platform == UnrealTargetPlatform.Mac || Target.Platform == UnrealTargetPlatform.Linux)
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "Kiss_FFT");
		}
		else
		{
			PublicDefinitions.Add("WITH_KISSFFT=0");
		}
	}
}
