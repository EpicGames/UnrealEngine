// Copyright Epic Games, Inc. All Rights Reserved.

#include "Classes/ActorFactoryLandscape.h"
#include "UObject/ConstructorHelpers.h"
#include "Components/SceneComponent.h"
#include "Engine/World.h"
#include "Engine/Texture2D.h"
#include "Components/BillboardComponent.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LandscapeProxy.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdMode.h"
#include "Classes/LandscapePlaceholder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorFactoryLandscape)

#define LOCTEXT_NAMESPACE "Landscape"

UActorFactoryLandscape::UActorFactoryLandscape(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("Landscape", "Landscape");
	NewActorClass = ALandscapeProxy::StaticClass();
}

AActor* UActorFactoryLandscape::SpawnActor(UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams)
{
	GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_Landscape);

	FEdModeLandscape* EdMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);

	EdMode->UISettings->NewLandscape_Location = Transform.GetLocation();
	EdMode->UISettings->NewLandscape_Rotation = Transform.GetRotation().Rotator();

	EdMode->SetCurrentTool("NewLandscape");

	ULevel* LocalLevel = ValidateSpawnActorLevel(InLevel, InSpawnParams);
	if (LocalLevel != nullptr)
	{
		FActorSpawnParameters SpawnInfo(InSpawnParams);
		SpawnInfo.OverrideLevel = LocalLevel;
		return LocalLevel->OwningWorld->SpawnActor(ALandscapePlaceholder::StaticClass(), &Transform, SpawnInfo);
	}

	return nullptr;
}

ALandscapePlaceholder::ALandscapePlaceholder(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinderOptional<UTexture2D> TerrainTexture;
		FConstructorStatics()
			: TerrainTexture(TEXT("/Engine/EditorResources/S_Terrain"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent0"));
	RootComponent = SceneComponent;
	RootComponent->Mobility = EComponentMobility::Static;

#if WITH_EDITORONLY_DATA
	UBillboardComponent* SpriteComponent = CreateEditorOnlyDefaultSubobject<UBillboardComponent>(TEXT("Sprite"));
	if (SpriteComponent)
	{
		SpriteComponent->Sprite = ConstructorStatics.TerrainTexture.Get();
		SpriteComponent->SetRelativeScale3D(FVector(0.5f, 0.5f, 0.5f));
		SpriteComponent->SetupAttachment(RootComponent);
		SpriteComponent->SetRelativeLocation(FVector(0, 0, 100));
		SpriteComponent->SetUsingAbsoluteScale(true);
	}
#endif
}

bool ALandscapePlaceholder::TeleportTo(const FVector& DestLocation, const FRotator& DestRotation, bool bIsATest /*= false*/, bool bNoCheck /*= false*/)
{
	bool bResult = Super::TeleportTo(DestLocation, DestRotation, bIsATest, bNoCheck);

	GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_Landscape);

	FEdModeLandscape* EdMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);

	EdMode->UISettings->NewLandscape_Location = GetActorLocation();
	EdMode->UISettings->NewLandscape_Rotation = GetActorRotation();

	EdMode->SetCurrentTool("NewLandscape");

	return bResult;
}

void ALandscapePlaceholder::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (!HasAnyFlags(RF_Transient))
	{
		Destroy();
	}
}

#undef LOCTEXT_NAMESPACE
