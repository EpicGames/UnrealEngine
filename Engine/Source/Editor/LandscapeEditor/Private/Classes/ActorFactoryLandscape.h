// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactoryLandscape.generated.h"

class AActor;
class ULevel;
struct FActorSpawnParameters;

UCLASS(MinimalAPI, config=Editor)
class UActorFactoryLandscape : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	virtual AActor* SpawnActor( UObject* Asset, ULevel* InLevel, const FTransform& Transform, const FActorSpawnParameters& InSpawnParams) override;
	//~ End UActorFactory Interface
};
