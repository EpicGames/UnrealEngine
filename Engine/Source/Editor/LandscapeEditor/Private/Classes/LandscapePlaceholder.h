// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "GameFramework/Actor.h"
#include "LandscapePlaceholder.generated.h"

UCLASS(NotPlaceable)
class ALandscapePlaceholder : public AActor
{
	GENERATED_UCLASS_BODY()

	//virtual void PostEditMove(bool bFinished) override;
	virtual bool TeleportTo(const FVector& DestLocation, const FRotator& DestRotation, bool bIsATest = false, bool bNoCheck = false) override;

	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
};
