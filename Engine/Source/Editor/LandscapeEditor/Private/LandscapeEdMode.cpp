// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEdMode.h"

#include "Algo/Accumulate.h"
#include "Algo/Transform.h"
#include "Algo/Find.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "SceneView.h"
#include "Engine/Texture2D.h"
#include "EditorViewportClient.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Engine/Light.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "Landscape.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeSubsystem.h"
#include "LandscapeSettings.h"
#include "LandscapeTiledImage.h"
#include "LandscapeEditLayer.h"

#include "EditorSupportDelegates.h"
#include "ScopedTransaction.h"
#include "LandscapeEdit.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditorUtils.h"
#include "LandscapeRender.h"
#include "LandscapeDataAccess.h"
#include "Framework/Commands/UICommandList.h"
#include "LevelEditor.h"
#include "Toolkits/ToolkitManager.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "InstancedFoliageActor.h"
#include "EditorWorldExtension.h"
#include "LandscapeEdModeTools.h"
#include "LandscapeInfoMap.h"
#include "LandscapeImportHelper.h"
#include "LandscapeConfigHelper.h"
#include "UObject/ObjectSaveContext.h"
#include "Math/TransformCalculus3D.h"

//Slate dependencies
#include "Misc/FeedbackContext.h"
#include "IAssetViewport.h"
#include "SLevelViewport.h"
#include "SLandscapeEditor.h"
#include "Framework/Application/SlateApplication.h"

// Classes
#include "LandscapeMaterialInstanceConstant.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineSelection.h"
#include "ComponentReregisterContext.h"
#include "EngineUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "LandscapeEditorCommands.h"
#include "Framework/Commands/InputBindingManager.h"
#include "MouseDeltaTracker.h"
#include "Interfaces/IMainFrameModule.h"
#include "LandscapeBlueprintBrushBase.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Settings/EditorExperimentalSettings.h"
#include "ComponentRecreateRenderStateContext.h"
#include "VisualLogger/VisualLogger.h"

#define LOCTEXT_NAMESPACE "Landscape"

DEFINE_LOG_CATEGORY(LogLandscapeEdMode);

void FLandscapeTool::SetEditRenderType()
{
	GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectRegion | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask);
}

namespace UE::Landscape::Editor::Private
{
	/** Indicates the user is currently moving the landscape gizmo object by dragging the mouse. */
	bool GIsGizmoDragging = false;
	/** Indicates the user is currently changing the landscape brush radius/falloff by dragging the mouse. */
	bool GIsAdjustingBrush = false;

	static FAutoConsoleVariable CVarVisualLogShowBrushPhysics(
		TEXT("landscape.VisualLog.ShowBrushPhysics"),
		false,
		TEXT("When visual log is active, allows to leave a visual log of the physics queries made for the purpose of the landscape brush"));
}
// namespace UE::Landscape::Editor::Private

//
// FEdModeLandscape
//

/** Constructor */
FEdModeLandscape::FEdModeLandscape()
	: FEdMode()
	, UISettings(nullptr)
	, CurrentToolMode(nullptr)
	, CurrentTool(nullptr)
	, CurrentBrush(nullptr)
	, GizmoBrush(nullptr)
	, CurrentToolIndex(INDEX_NONE)
	, CurrentBrushSetIndex(0)
	, CurrentLandscapeViewMode(ELandscapeViewMode::Normal)
	, PendingRenameTargetLayerIndex(INDEX_NONE)
	, NewLandscapePreviewMode(ENewLandscapePreviewMode::None)
	, ImportExportMode(EImportExportMode::Import)
	, CopyPasteTool(nullptr)
	, SplinesTool(nullptr)
	, TargetLayerStartingIndex(0)
	, CachedLandscapeMaterial(nullptr)
	, ToolActiveViewport(nullptr)
	, bIsPaintingInVR(false)
	, InteractorPainting(nullptr)
	, bNeedsUpdateLayerUsageInformation(false)
	, bUpdatingLandscapeInfo(false)
	, bNeedsUpdateTargetLayerList(false)
	, bHasMapChanged(false)
	, InputProcessor(MakeShared<FInputProcessor>())
{
	using namespace UE::Landscape::Editor;
	
	// Initialize modes
	UpdateToolModes();
	
	// Initialize tools.
	InitializeTool_Paint();
	InitializeTool_Smooth();
	InitializeTool_Flatten();
	InitializeTool_Erosion();
	InitializeTool_HydraErosion();
	InitializeTool_Noise();
	InitializeTool_NewLandscape();
	InitializeTool_ResizeLandscape();
	InitializeTool_ImportExport();
	InitializeTool_Select();
	InitializeTool_AddComponent();
	InitializeTool_DeleteComponent();
	InitializeTool_MoveToLevel();
	InitializeTool_Mask();
	InitializeTool_CopyPaste();
	InitializeTool_Visibility();
	InitializeTool_Splines();
	InitializeTool_Ramp();
	InitializeTool_Mirror();
	InitializeTool_BlueprintBrush();

	// Initialize brushes
	InitializeBrushes();

	CurrentBrush = LandscapeBrushSets[0].Brushes[0];
	
	CurrentToolTarget.LandscapeInfo = nullptr;
	CurrentToolTarget.TargetType = ELandscapeToolTargetType::Heightmap;
	CurrentToolTarget.LayerInfo = nullptr;

	UISettings = NewObject<ULandscapeEditorObject>(GetTransientPackage(), TEXT("UISettings"), RF_Transactional);
	UISettings->SetParent(this);

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	TSharedPtr<FUICommandList> CommandList = LandscapeEditorModule.GetLandscapeLevelViewportCommandList();

	const FLandscapeEditorCommands& LandscapeActions = FLandscapeEditorCommands::Get();
	CommandList->MapAction(LandscapeActions.IncreaseBrushSize, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushSize, true), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.DecreaseBrushSize, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushSize, false), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.IncreaseBrushFalloff, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushFalloff, true), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.DecreaseBrushFalloff, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushFalloff, false), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.IncreaseBrushStrength, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushStrength, true), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.DecreaseBrushStrength, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeBrushStrength, false), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.IncreaseAlphaBrushRotation, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeAlphaBrushRotation, true), FCanExecuteAction(), FIsActionChecked());
	CommandList->MapAction(LandscapeActions.DecreaseAlphaBrushRotation, FExecuteAction::CreateRaw(this, &FEdModeLandscape::ChangeAlphaBrushRotation, false), FCanExecuteAction(), FIsActionChecked());
}

/** Destructor */
FEdModeLandscape::~FEdModeLandscape()
{
	// Destroy tools.
	LandscapeTools.Empty();

	// Destroy brushes
	LandscapeBrushSets.Empty();

	// Clean up Debug Materials
	FlushRenderingCommands();

	InteractorPainting = nullptr;
}

/** FGCObject interface */
void FEdModeLandscape::AddReferencedObjects(FReferenceCollector& Collector)
{
	// Call parent implementation
	FEdMode::AddReferencedObjects(Collector);

	Collector.AddReferencedObject(UISettings);
}

void FEdModeLandscape::UpdateToolModes()
{
	// Keep mapping of CurrentTool and CurrentTargetLayer
	TMap<FName, FName> PreviousTools;
	TMap<FName, FName> PreviousTargetLayerNames;
	for (const FLandscapeToolMode& Previous : LandscapeToolModes)
	{
		PreviousTools.Add(Previous.ToolModeName, Previous.CurrentToolName);
		PreviousTargetLayerNames.Add(Previous.ToolModeName, Previous.CurrentTargetLayerName);
	}

	LandscapeToolModes.Reset();
	
	FLandscapeToolMode* ToolMode_Manage = new(LandscapeToolModes)FLandscapeToolMode(TEXT("ToolMode_Manage"), ELandscapeToolTargetTypeFlags::None);
	ToolMode_Manage->ValidTools.Add(TEXT("NewLandscape"));
	ToolMode_Manage->ValidTools.Add(TEXT("Select"));
	ToolMode_Manage->ValidTools.Add(TEXT("AddComponent"));
	ToolMode_Manage->ValidTools.Add(TEXT("DeleteComponent"));
	ToolMode_Manage->ValidTools.Add(TEXT("MoveToLevel"));
	ToolMode_Manage->ValidTools.Add(TEXT("ResizeLandscape"));
	ToolMode_Manage->ValidTools.Add(TEXT("Splines"));
	ToolMode_Manage->ValidTools.Add(TEXT("ImportExport"));
	ToolMode_Manage->ValidTools.Add(TEXT("BlueprintBrush"));
	
	// Restore
	FName* PreviousToolName = PreviousTools.Find(ToolMode_Manage->ToolModeName);
	ToolMode_Manage->CurrentToolName = PreviousToolName ? *PreviousToolName : TEXT("Select");

	FName* PreviousTargetLayerName = PreviousTargetLayerNames.Find(ToolMode_Manage->ToolModeName);
	ToolMode_Manage->CurrentTargetLayerName = PreviousTargetLayerName ? *PreviousTargetLayerName : NAME_None;
		

	FLandscapeToolMode* ToolMode_Sculpt = new(LandscapeToolModes)FLandscapeToolMode(TEXT("ToolMode_Sculpt"), ELandscapeToolTargetTypeFlags::Heightmap | ELandscapeToolTargetTypeFlags::Visibility);
	ToolMode_Sculpt->ValidTools.Add(TEXT("Sculpt"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Erase"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Smooth"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Flatten"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Ramp"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Noise"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Erosion"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("HydraErosion"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Visibility"));

	ToolMode_Sculpt->ValidTools.Add(TEXT("Mask"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("CopyPaste"));
	ToolMode_Sculpt->ValidTools.Add(TEXT("Mirror"));
	
	// Restore
	PreviousToolName = PreviousTools.Find(ToolMode_Sculpt->ToolModeName);
	ToolMode_Sculpt->CurrentToolName = PreviousToolName ? *PreviousToolName : TEXT("Sculpt");

	PreviousTargetLayerName = PreviousTargetLayerNames.Find(ToolMode_Sculpt->ToolModeName);
	ToolMode_Sculpt->CurrentTargetLayerName = PreviousTargetLayerName ? *PreviousTargetLayerName : NAME_None;

	FLandscapeToolMode* ToolMode_Paint = new(LandscapeToolModes)FLandscapeToolMode(TEXT("ToolMode_Paint"), ELandscapeToolTargetTypeFlags::Weightmap);
	ToolMode_Paint->ValidTools.Add(TEXT("Paint"));
	ToolMode_Paint->ValidTools.Add(TEXT("Smooth"));
	ToolMode_Paint->ValidTools.Add(TEXT("Flatten"));
	ToolMode_Paint->ValidTools.Add(TEXT("Noise"));
	ToolMode_Paint->ValidTools.Add(TEXT("Visibility"));
	ToolMode_Paint->ValidTools.Add(TEXT("Mask"));

	PreviousToolName = PreviousTools.Find(ToolMode_Paint->ToolModeName);
	ToolMode_Paint->CurrentToolName = PreviousToolName ? *PreviousToolName : TEXT("Paint");

	PreviousTargetLayerName = PreviousTargetLayerNames.Find(ToolMode_Paint->ToolModeName);
	ToolMode_Paint->CurrentTargetLayerName = PreviousTargetLayerName ? *PreviousTargetLayerName : NAME_None;

	// Since available tools might have changed try and reset the current tool
	if (CurrentToolMode && CurrentToolIndex != INDEX_NONE)
	{
		SetCurrentTool(CurrentToolIndex, CurrentToolMode->CurrentTargetLayerName);
	}
}

bool FEdModeLandscape::UsesToolkits() const
{
	return true;
}

TSharedRef<FUICommandList> FEdModeLandscape::GetUICommandList() const
{
	check(Toolkit.IsValid());
	return Toolkit->GetToolkitCommands();
}

void FEdModeLandscape::PostUpdateLayerContent()
{
	if (bNeedsUpdateLayerUsageInformation)
	{
		UpdateLayerUsageInformation();
	}
}

ELandscapeToolTargetType FEdModeLandscape::GetLandscapeToolTargetType() const
{
	if (CurrentToolMode)
	{
		if (CurrentToolMode->ToolModeName == "ToolMode_Sculpt")
		{
			return CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility ? ELandscapeToolTargetType::Visibility : ELandscapeToolTargetType::Heightmap;
		}
		else if (CurrentToolMode->ToolModeName == "ToolMode_Paint")
		{
			return ELandscapeToolTargetType::Weightmap;
		}
	}
	return ELandscapeToolTargetType::Invalid;
}

const ULandscapeEditLayerBase* FEdModeLandscape::GetLandscapeSelectedLayer() const
{
	return GetCurrentEditLayerConst();
}

ULandscapeLayerInfoObject* FEdModeLandscape::GetSelectedLandscapeLayerInfo() const
{
	return CurrentToolTarget.LayerInfo.Get();
}

bool FEdModeLandscape::IsLandscapeViewModeExclusiveToEditorMode(ELandscapeViewMode::Type ViewMode)
{
	return ViewMode == ELandscapeViewMode::DebugLayer || ViewMode == ELandscapeViewMode::LayerContribution;
}

void FEdModeLandscape::SetLandscapeInfo(ULandscapeInfo* InLandscapeInfo)
{
	check(!InLandscapeInfo || InLandscapeInfo->SupportsLandscapeEditing());
	const bool bIsInitialLandscapeInfo = CurrentToolTarget.LandscapeInfo == nullptr;
	if (CurrentToolTarget.LandscapeInfo != InLandscapeInfo)
	{
		{
			TGuardValue<bool> GuardFlag(bUpdatingLandscapeInfo, true);
			CurrentToolTarget.LandscapeInfo = InLandscapeInfo;

			// Only update the selected edit layer index for the initial landscape, 
			// this avoids setting the last saved index to 0 when switching between existing landscapes
			if (bIsInitialLandscapeInfo)
			{
				SetSelectedEditLayer(0);
			}

			UpdateTargetList();
			UpdateToolModes();
		}

		RefreshDetailPanel();
	}
}

int32 FEdModeLandscape::GetAccumulatedAllLandscapesResolution() const
{
	int32 TotalResolution = 0;

	TotalResolution = Algo::Accumulate(LandscapeList, TotalResolution, [](int32 Accum, const FLandscapeListInfo& LandscapeListInfo)
	{
		return Accum + ((LandscapeListInfo.Width > 0) ? LandscapeListInfo.Width : 1)
					 * ((LandscapeListInfo.Height > 0) ? LandscapeListInfo.Height : 1);
	});

	return TotalResolution;
}

bool FEdModeLandscape::IsLandscapeResolutionCompliant() const
{
	const TObjectPtr<const ULandscapeSettings> Settings = GetDefault<ULandscapeSettings>();
	check(Settings);

	if (Settings->IsLandscapeResolutionRestricted())
	{
		int32 TotalResolution = (CurrentTool != nullptr) ? CurrentTool->GetToolActionResolutionDelta() : 0;
		TotalResolution += GetAccumulatedAllLandscapesResolution();
		
		return TotalResolution <= Settings->GetTotalResolutionLimit();
	}

	return true;
}

bool FEdModeLandscape::DoesCurrentToolAffectEditLayers() const
{
	return (CurrentTool != nullptr) ? CurrentTool->AffectsEditLayers() : false;
}

FText FEdModeLandscape::GetLandscapeResolutionErrorText() const
{
	const TObjectPtr<const ULandscapeSettings> Settings = GetDefault<ULandscapeSettings>();
	check(Settings);

	return FText::Format(LOCTEXT("LandscapeResolutionError", "Total resolution for all Landscape actors cannot exceed the equivalent of {0} x {0}."), Settings->GetSideResolutionLimit());
}

int32 FEdModeLandscape::GetNewLandscapeResolutionX() const
{
	return UISettings->NewLandscape_ComponentCount.X * UISettings->NewLandscape_SectionsPerComponent * UISettings->NewLandscape_QuadsPerSection + 1;
}

int32 FEdModeLandscape::GetNewLandscapeResolutionY() const
{
	return UISettings->NewLandscape_ComponentCount.Y * UISettings->NewLandscape_SectionsPerComponent * UISettings->NewLandscape_QuadsPerSection + 1;
}

void FEdModeLandscape::SetInspectedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects)
{
	InspectedObjects = InObjects;
	RefreshInspectedObjectsDetailPanel();
}

bool FEdModeLandscape::IsGlobalModifierPressed() const
{
	return InputProcessor->bIsShiftDown;
}

bool FEdModeLandscape::IsGlobalAlternateModifierPressed() const
{
	return InputProcessor->bIsCtrlDown;
}

/** FEdMode: Called when the mode is entered */
void FEdModeLandscape::Enter()
{
	using namespace UE::Landscape::Editor::Private;

	ErrorReasonOnMouseUp = FText::GetEmpty();

	// Call parent implementation
	FEdMode::Enter();

	UWorld* World = GetWorld();

	if (World != nullptr)
	{
		for (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(World).Map.CreateIterator(); It; ++It)
		{
			if (ULandscapeInfo* LandscapeInfo = It.Value())
			{
				if (ALandscape* Landscape = (IsValid(LandscapeInfo) && LandscapeInfo->SupportsLandscapeEditing()) ? LandscapeInfo->LandscapeActor.Get() : nullptr)
				{
					Landscape->RegisterLandscapeEdMode(this);
				}
			}
		}

		OnLevelsChangedDelegateHandle = GetWorld()->OnLevelsChanged().AddRaw(this, &FEdModeLandscape::HandleLevelsChanged);
	}

	OnLevelActorDeletedDelegateHandle = GEngine->OnLevelActorDeleted().AddSP(this, &FEdModeLandscape::OnLevelActorRemoved);
	OnLevelActorAddedDelegateHandle = GEngine->OnLevelActorAdded().AddSP(this, &FEdModeLandscape::OnLevelActorAdded);
	PreSaveWorldHandle = FEditorDelegates::PreSaveWorldWithContext.AddSP(this, &FEdModeLandscape::OnPreSaveWorld);
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		OnIsEditingDisallowedChangedHandle = FirstLevelEditor->GetEditorModeManager().OnIsEditingDisallowedChanged().AddSP(this, &FEdModeLandscape::HandleLevelsChanged);
	}
		
	UpdateToolModes();

	ALandscapeProxy* SelectedLandscape = GEditor->GetSelectedActors()->GetTop<ALandscapeProxy>();
	if (SelectedLandscape && SelectedLandscape->GetLandscapeInfo()->SupportsLandscapeEditing())
	{
		SetLandscapeInfo(SelectedLandscape->GetLandscapeInfo());
		GEditor->SelectNone(false, true);
		GEditor->SelectActor(SelectedLandscape, true, false);
	}
	else
	{
		GEditor->SelectNone(true, true);
	}

	for (TActorIterator<ALandscapeGizmoActiveActor> It(GetWorld()); It; ++It)
	{
		ALandscapeGizmoActiveActor* GizmoActor = *It;
		if (GizmoActor->HasAnyFlags(RF_Transient))
		{
			UISettings->CurrentGizmoActor = *It;
			break;
		}
	}


	if (!UISettings->CurrentGizmoActor.IsValid())
	{
		FActorSpawnParameters SpawnParams;
		SpawnParams.ObjectFlags |= RF_Transient;
		UISettings->CurrentGizmoActor = GetWorld()->SpawnActor<ALandscapeGizmoActiveActor>(SpawnParams);
		UISettings->CurrentGizmoActor->ImportFromClipboard();
	}

	// Update list of landscapes and layers
	// For now depends on the SpawnActor() above in order to get the current editor world as edmodes don't get told
	UpdateLandscapeList();
	UpdateBrushList();

	OnWorldChangeDelegateHandle                 = FEditorSupportDelegates::WorldChange.AddRaw(this, &FEdModeLandscape::HandleLevelsChanged);

	if (CurrentToolTarget.LandscapeInfo.IsValid() && World != nullptr)
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
		{
			LandscapeSubsystem->OnLandscapeProxyMaterialChanged().AddRaw(this, &FEdModeLandscape::OnLandscapeMaterialChangedDelegate);
		}

		if (ALandscape* Landscape = GetLandscape())
		{
			Landscape->OnBlueprintBrushChangedDelegate().AddRaw(this, &FEdModeLandscape::RefreshDetailPanel);
			Landscape->RequestSplineLayerUpdate();
			Landscape->RequestLayersContentUpdateForceAll();
		}
	}

	TWeakObjectPtr<ALandscapeGizmoActiveActor> CurrentGizmoActor = UISettings->CurrentGizmoActor;
	if (CurrentGizmoActor.IsValid())
	{
		CurrentGizmoActor->SetTargetLandscape(CurrentToolTarget.LandscapeInfo.Get());
		CurrentGizmoActor->SnapType = UISettings->SnapMode;
	}

	int32 SquaredDataTex = ALandscapeGizmoActiveActor::DataTexSize * ALandscapeGizmoActiveActor::DataTexSize;

	if (CurrentGizmoActor.IsValid() && !CurrentGizmoActor->GizmoTexture)
	{
		// Init Gizmo Texture...
		CurrentGizmoActor->GizmoTexture = NewObject<UTexture2D>(GetTransientPackage(), NAME_None, RF_Transient);
		if (CurrentGizmoActor->GizmoTexture)
		{
			CurrentGizmoActor->GizmoTexture->Source.Init(
				ALandscapeGizmoActiveActor::DataTexSize,
				ALandscapeGizmoActiveActor::DataTexSize,
				1,
				1,
				TSF_G8
				);
			CurrentGizmoActor->GizmoTexture->SRGB = false;
			CurrentGizmoActor->GizmoTexture->CompressionNone = true;
			CurrentGizmoActor->GizmoTexture->MipGenSettings = TMGS_NoMipmaps;
			CurrentGizmoActor->GizmoTexture->AddressX = TA_Clamp;
			CurrentGizmoActor->GizmoTexture->AddressY = TA_Clamp;
			CurrentGizmoActor->GizmoTexture->LODGroup = TEXTUREGROUP_Terrain_Weightmap;
			uint8* TexData = CurrentGizmoActor->GizmoTexture->Source.LockMip(0);
			FMemory::Memset(TexData, 0, SquaredDataTex*sizeof(uint8));
			// Restore Sampled Data if exist...
			if (CurrentGizmoActor->CachedScaleXY > 0.0f)
			{
				int32 SizeX = FMath::CeilToInt(CurrentGizmoActor->CachedWidth / CurrentGizmoActor->CachedScaleXY);
				int32 SizeY = FMath::CeilToInt(CurrentGizmoActor->CachedHeight / CurrentGizmoActor->CachedScaleXY);
				for (int32 Y = 0; Y < CurrentGizmoActor->SampleSizeY; ++Y)
				{
					for (int32 X = 0; X < CurrentGizmoActor->SampleSizeX; ++X)
					{
						float TexX = static_cast<float>(X * SizeX / CurrentGizmoActor->SampleSizeX);
						float TexY = static_cast<float>(Y * SizeY / CurrentGizmoActor->SampleSizeY);
						int32 LX = FMath::FloorToInt(TexX);
						int32 LY = FMath::FloorToInt(TexY);

						float FracX = TexX - LX;
						float FracY = TexY - LY;

						FGizmoSelectData* Data00 = CurrentGizmoActor->SelectedData.Find(FIntPoint(LX, LY));
						FGizmoSelectData* Data10 = CurrentGizmoActor->SelectedData.Find(FIntPoint(LX + 1, LY));
						FGizmoSelectData* Data01 = CurrentGizmoActor->SelectedData.Find(FIntPoint(LX, LY + 1));
						FGizmoSelectData* Data11 = CurrentGizmoActor->SelectedData.Find(FIntPoint(LX + 1, LY + 1));

						TexData[X + Y*ALandscapeGizmoActiveActor::DataTexSize] = static_cast<uint8>(FMath::Lerp(
							FMath::Lerp(Data00 ? Data00->Ratio : 0, Data10 ? Data10->Ratio : 0, FracX),
							FMath::Lerp(Data01 ? Data01->Ratio : 0, Data11 ? Data11->Ratio : 0, FracX),
							FracY
							) * 255);
					}
				}
			}
			CurrentGizmoActor->GizmoTexture->Source.UnlockMip(0);
			CurrentGizmoActor->GizmoTexture->PostEditChange();
			FlushRenderingCommands();
		}
	}

	if (CurrentGizmoActor.IsValid() && CurrentGizmoActor->SampledHeight.Num() != SquaredDataTex)
	{
		CurrentGizmoActor->SampledHeight.Empty(SquaredDataTex);
		CurrentGizmoActor->SampledHeight.AddZeroed(SquaredDataTex);
		CurrentGizmoActor->DataType = LGT_None;
	}

	if (CurrentGizmoActor.IsValid()) // Update Scene Proxy
	{
		CurrentGizmoActor->ReregisterAllComponents();
	}

	GLandscapeEditRenderMode = ELandscapeEditRenderMode::None;
	GLandscapeEditModeActive = true;

	// Load UI settings from config file
	UISettings->Load();

	// It is cleared on exit so update here even if LandscapeInfo hasn't changed
	UpdateTargetList();

	FName ToolkitPalette = NAME_None;

	// Initialize current tool prior to creating the landscape toolkit in case it has a dependency on it
	if (LandscapeList.Num() == 0)
	{
		SetCurrentToolMode("ToolMode_Manage", false);
		SetCurrentTool("NewLandscape");
		ToolkitPalette = "ToolMode_Manage";
	}
	else
	{
		if (CurrentToolMode == nullptr || (CurrentToolMode->CurrentToolName == FName("NewLandscape")) || CurrentToolMode->CurrentToolName == NAME_None)
		{
			SetCurrentToolMode("ToolMode_Sculpt", false);
			SetCurrentTool("Sculpt");
			ToolkitPalette = "ToolMode_Sculpt";
		}
		else
		{
			SetCurrentTool(CurrentToolMode->CurrentToolName);
			ToolkitPalette = CurrentToolMode->ToolModeName;
		}
	}

	// Create the landscape editor window
	if (!Toolkit.IsValid())
	{
		Toolkit = MakeShareable(new FLandscapeToolKit);
		Toolkit->Init(Owner->GetToolkitHost());
		Toolkit->SetCurrentPalette(ToolkitPalette);
	}

	// Force real-time viewports.  We'll back up the current viewport state so we can restore it when the
	// user exits this mode.
	const bool bWantRealTime = true;
	ForceRealTimeViewports(bWantRealTime);

	CurrentBrush->EnterBrush();
	if (GizmoBrush)
	{
		GizmoBrush->EnterBrush();
	}

	// Reset mouse tracking info : 
	GIsGizmoDragging = false;
	GIsAdjustingBrush  = false;

	// Restore view mode if it was auto changed last time the landscape editor was closed
	if (IsLandscapeViewModeExclusiveToEditorMode(CurrentLandscapeViewMode))
	{
		GLandscapeViewMode = CurrentLandscapeViewMode;
	}

	// Set the inspected object to empty when entering edit mode
	SetInspectedObjects({});

	// Make sure HasMapChanged is reset when entering edit mode
	bHasMapChanged = false;

	InputProcessor->StartTrackingInput();
}

/** FEdMode: Called when the mode is exited */
void FEdModeLandscape::Exit()
{
	using namespace UE::Landscape::Editor::Private;

	InputProcessor->StopTrackingInput();

	// Reset mouse tracking info : 
	GIsGizmoDragging = false;
	GIsAdjustingBrush  = false;

	// Set the inspected object to empty when exiting edit mode
	SetInspectedObjects({});

	UWorld* World = GetWorld();

	if (World != nullptr)
	{
		for (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(World).Map.CreateIterator(); It; ++It)
		{
			if (ULandscapeInfo* LandscapeInfo = It.Value())
			{
				if (ALandscape* Landscape = (IsValid(LandscapeInfo) && LandscapeInfo->SupportsLandscapeEditing()) ? LandscapeInfo->LandscapeActor.Get() : nullptr)
				{
					Landscape->UnregisterLandscapeEdMode();
				}
			}
		}

		World->OnLevelsChanged().Remove(OnLevelsChangedDelegateHandle);
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	if (TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor())
	{
		FirstLevelEditor->GetEditorModeManager().OnIsEditingDisallowedChanged().Remove(OnIsEditingDisallowedChangedHandle);
	}
	FEditorDelegates::PreSaveWorldWithContext.Remove(PreSaveWorldHandle);
	GEngine->OnLevelActorDeleted().Remove(OnLevelActorDeletedDelegateHandle);
	GEngine->OnLevelActorAdded().Remove(OnLevelActorAddedDelegateHandle);
	
	FEditorSupportDelegates::WorldChange.Remove(OnWorldChangeDelegateHandle);

	if (CurrentToolTarget.LandscapeInfo.IsValid() && World != nullptr)
	{
		if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
		{
			LandscapeSubsystem->OnLandscapeProxyMaterialChanged().RemoveAll(this);
		}

		if (ALandscape* Landscape = GetLandscape())
		{
			Landscape->OnBlueprintBrushChangedDelegate().RemoveAll(this);
		}
	}

	// Restore real-time viewport state if we changed it
	const bool bWantRealTimeOverride = false;
	ForceRealTimeViewports(bWantRealTimeOverride);

	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}

	CurrentBrush->LeaveBrush();
	if (GizmoBrush)
	{
		GizmoBrush->LeaveBrush();
	}

	if (CurrentTool)
	{
		CurrentTool->PreviousBrushIndex = CurrentBrushSetIndex;
		CurrentTool->ExitTool();
	}
	CurrentTool = nullptr;
	// Leave CurrentToolIndex set so we can restore the active tool on re-opening the landscape editor

	LandscapeList.Empty();
	LandscapeTargetList.Empty();
	TargetLayerStartingIndex = 0;

	// Save UI settings to config file
	UISettings->Save();

	// Save the last used view mode 
	CurrentLandscapeViewMode = static_cast<ELandscapeViewMode::Type>(GLandscapeViewMode);

	// Reset global view if current mode is only supported with editor open
	if (IsLandscapeViewModeExclusiveToEditorMode(CurrentLandscapeViewMode))
	{
		GLandscapeViewMode = ELandscapeViewMode::Normal;
	}
	GLandscapeEditRenderMode = ELandscapeEditRenderMode::None;
	GLandscapeEditModeActive = false;

	UISettings->CurrentGizmoActor = nullptr;

	GEditor->SelectNone(false, true);

	// Clear all GizmoActors if there is no Landscape in World
	bool bIsLandscapeExist = false;
	if (TActorIterator<ALandscapeProxy> It(GetWorld()); It)
	{
		bIsLandscapeExist = true;
	}

	if (!bIsLandscapeExist)
	{
		for (TActorIterator<ALandscapeGizmoActor> It(GetWorld()); It; ++It)
		{
			GetWorld()->DestroyActor(*It, false, false);
		}
	}

	// Redraw one last time to remove any landscape editor stuff from view
	GEditor->RedrawLevelEditingViewports();

	// Call parent implementation
	FEdMode::Exit();
}

void FEdModeLandscape::OnPreSaveWorld(UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext)
{
	// If the mode is pending deletion, don't run the presave routine.
	if (!Owner->IsModeActive(GetID()))
	{
		return;
	}

	// Avoid doing this during procedural saves to keep determinism and we don't want to do this on GameWorlds.
	if (!InWorld->IsGameWorld() && !ObjectSaveContext.IsProceduralSave())
	{
		ULandscapeInfoMap& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(InWorld);
		for (const auto& Pair : LandscapeInfoMap.Map)
		{
			if (const ULandscapeInfo* LandscapeInfo = Pair.Value)
			{
				if (ALandscape* LandscapeActor = LandscapeInfo->LandscapeActor.Get())
				{
					LandscapeActor->OnPreSave();
				}
			}
		}
	}
}

/** FEdMode: Called once per frame */
void FEdModeLandscape::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	FEdMode::Tick(ViewportClient, DeltaTime);

	if (!IsEditingEnabled())
	{
		return;
	}

	FViewport* const Viewport = ViewportClient->Viewport;

	if (ToolActiveViewport && ToolActiveViewport == Viewport && ensure(CurrentTool) && !bIsPaintingInVR)
	{
		// Require Ctrl or not as per user preference
		const ELandscapeFoliageEditorControlType LandscapeEditorControlType = GetDefault<ULevelEditorViewportSettings>()->LandscapeEditorControlType;

		if (!Viewport->KeyState(EKeys::LeftMouseButton) ||
			(LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl && !IsCtrlDown(Viewport)))
		{
			// Don't end the current tool if we are just modifying it
			if (!IsAdjustingBrush(ViewportClient) && CurrentTool->IsToolActive())
			{
				CurrentTool->EndTool(ViewportClient);
				Viewport->CaptureMouse(false);
				ToolActiveViewport = nullptr;
			}
		}
	}

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		const bool bStaleTargetLandscapeInfo = CurrentToolTarget.LandscapeInfo.IsStale();
		const bool bStaleTargetLandscape = CurrentToolTarget.LandscapeInfo.IsValid() && (CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() == nullptr);

		if (bStaleTargetLandscapeInfo || bStaleTargetLandscape)
		{
			UpdateLandscapeList();
		}

		if (CurrentToolTarget.LandscapeInfo.IsValid())
		{
			ALandscapeProxy* LandscapeProxy = CurrentToolTarget.LandscapeInfo->GetLandscapeProxy();
			
			if (LandscapeProxy == nullptr ||
				LandscapeProxy->GetLandscapeMaterial() != CachedLandscapeMaterial)
			{
				UpdateTargetList();
			}
			else
			{
				if (CurrentTool)
				{
					CurrentTool->Tick(ViewportClient, DeltaTime);
				}
			
				if (CurrentBrush)
				{
					CurrentBrush->Tick(ViewportClient, DeltaTime);
				}
			
				if (CurrentBrush != GizmoBrush && UISettings->CurrentGizmoActor.IsValid() && GizmoBrush && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo))
				{
					GizmoBrush->Tick(ViewportClient, DeltaTime);
				}
			}
		}
	}

	static int32 LastLandscapeSplineFalloffModulationValue = CVarLandscapeSplineFalloffModulation.GetValueOnAnyThread();
	if (LastLandscapeSplineFalloffModulationValue != CVarLandscapeSplineFalloffModulation.GetValueOnAnyThread())
	{
		if (ALandscape* Landscape = GetLandscape())
		{
			Landscape->RequestSplineLayerUpdate();
		}
		LastLandscapeSplineFalloffModulationValue = CVarLandscapeSplineFalloffModulation.GetValueOnAnyThread();
	}

	if (bNeedsUpdateTargetLayerList)
	{
		// Updating target list because a landscape material has changed, regenerate lists that may hold on to a ThumbnailMIC
		UpdateTargetList(/*bRegenerateThumbnails =*/ true);
		bNeedsUpdateTargetLayerList = false;
	}
}

ELandscapeTargetLayerActionType FEdModeLandscape::GetEstimatedTargetLayerActionType(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	const bool bIsGlobalModifierPressed = IsGlobalModifierPressed();
	const bool bIsGlobalAlternateModifierPressed = IsGlobalAlternateModifierPressed();

	// Bail out if one of the LandscapeEditorControlType's "Hold/Don't Hold Ctrl" condition is not met, so that the tools don't mistakenly let the user know that they will do anything :
	const ELandscapeFoliageEditorControlType LandscapeEditorControlType = GetDefault<ULevelEditorViewportSettings>()->LandscapeEditorControlType;
	if ((LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl && !bIsGlobalAlternateModifierPressed)
		|| (LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireNoCtrl && bIsGlobalAlternateModifierPressed))
	{
		return ELandscapeTargetLayerActionType::None;
	}

	return CurrentTool 
		? CurrentTool->GetTargetLayerActionType(Target->LayerInfoObj.Get(), bIsGlobalModifierPressed, bIsGlobalAlternateModifierPressed)
		: ELandscapeTargetLayerActionType::None;
}

/** FEdMode: Called when the mouse enters the viewport area */
bool FEdModeLandscape::MouseEnter(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 MouseX, int32 MouseY)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool bHandled = false;
	if (CurrentTool)
	{
		bHandled = CurrentTool->MouseEnter(InViewportClient, InViewport, MouseX, MouseY);
	}

	return bHandled;
}

/** FEdMode: Called when the mouse exits the viewport area */
bool FEdModeLandscape::MouseLeave(FEditorViewportClient * InViewportClient, FViewport * InViewport)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool bHandled = false;
	if (CurrentTool)
	{
		bHandled = CurrentTool->MouseLeave(InViewportClient, InViewport);
	}

	return bHandled;
}

/** FEdMode: Called when the mouse is moved over the viewport */
bool FEdModeLandscape::MouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 MouseX, int32 MouseY)
{
	// due to mouse capture this should only ever be called on the active viewport
	// if it ever gets called on another viewport the mouse has been released without us picking it up
	if (ToolActiveViewport && ensure(CurrentTool) && !bIsPaintingInVR)
	{
		int32 MouseXDelta = MouseX - InViewportClient->GetCachedMouseX();
		int32 MouseYDelta = MouseY - InViewportClient->GetCachedMouseY();

		const FLandscapeEditorCommands& LandscapeActions = FLandscapeEditorCommands::Get();
		const bool bIsDragChordBrushSizeAndFalloffPressed = InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushSizeAndFalloff);
		const bool bIsDragChordBrushSizeOnlyPressed = InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushSize);
		const bool bIsDragChordBrushFalloffOnlyPressed = InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushFalloff);
		const bool bIsDragChordStrengthPressed = InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushStrength);
		const bool bIsDragChordPressed = bIsDragChordBrushSizeAndFalloffPressed || bIsDragChordBrushSizeOnlyPressed || bIsDragChordBrushFalloffOnlyPressed || bIsDragChordStrengthPressed;

		if (FMath::Abs(MouseXDelta) > 0 || FMath::Abs(MouseYDelta) > 0)
		{
			const bool bHorizontallyDominant = FMath::Abs(MouseXDelta) >= FMath::Abs(MouseYDelta);
			const bool bIncrease = bHorizontallyDominant ?
				MouseXDelta > 0 : 
				MouseYDelta < 0; // The way y position is stored here is inverted relative to expected mouse movement to change brush size

			// Are we altering something about the brush?
			if (bIsDragChordBrushSizeAndFalloffPressed)
			{
				// If a chord controlling both Size and Falloff is specified, control Size when moving mostly left-right and Falloff when moving mostly up/down
				if (bHorizontallyDominant)
				{
					ChangeBrushSize(bIncrease);
				}
				else
				{
					ChangeBrushFalloff(bIncrease);
				}
				return true;
			}

			if (bIsDragChordBrushSizeOnlyPressed)
			{
				ChangeBrushSize(bIncrease);
				return true;
			}

			if (bIsDragChordBrushFalloffOnlyPressed)
			{
				ChangeBrushFalloff(bIncrease);
				return true;
			}

			if (bIsDragChordStrengthPressed)
			{
				ChangeBrushStrength(bIncrease);
				return true;
			}
		}

		// Require Ctrl or not as per user preference
		const ELandscapeFoliageEditorControlType LandscapeEditorControlType = GetDefault<ULevelEditorViewportSettings>()->LandscapeEditorControlType;

		if ((ToolActiveViewport != InViewport)
			|| !InViewport->KeyState(EKeys::LeftMouseButton)
			|| (LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl && !IsCtrlDown(InViewport)))
		{
			if (CurrentTool->IsToolActive())
			{
				CurrentTool->EndTool(InViewportClient);
			}
			// Don't let go of the mouse capture while using one the drag shortcuts in order to maintain the cursor updated every frame : 
			if (!bIsDragChordPressed)
			{
				InViewport->CaptureMouse(false);
			}
			ToolActiveViewport = nullptr;
		}
	}

	if (!IsEditingEnabled())
	{
		return false;
	}

	bool Result = false;
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool)
		{
			Result = CurrentTool->MouseMove(InViewportClient, InViewport, MouseX, MouseY);
			InViewportClient->Invalidate(false, false);
		}
	}
	return Result;
}

bool FEdModeLandscape::GetCursor(EMouseCursor::Type& OutCursor) const
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool Result = false;
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool)
		{
			Result = CurrentTool->GetCursor(OutCursor);
		}
	}

	return Result;
}

bool FEdModeLandscape::GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool Result = false;
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool)
		{
			Result = CurrentTool->GetOverrideCursorVisibility(bWantsOverride, bHardwareCursorVisible, bSoftwareCursorVisible);
		}
	}

	return Result;
}

bool FEdModeLandscape::PreConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool Result = false;
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool)
		{
			Result = CurrentTool->PreConvertMouseMovement(InViewportClient);
		}
	}

	return Result;
}

bool FEdModeLandscape::PostConvertMouseMovement(FEditorViewportClient* InViewportClient)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	bool Result = false;
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool)
		{
			Result = CurrentTool->PostConvertMouseMovement(InViewportClient);
		}
	}

	return Result;
}

bool FEdModeLandscape::DisallowMouseDeltaTracking() const
{
	// We never want to use the mouse delta tracker while painting
	return (ToolActiveViewport != nullptr);
}

/**
 * Called when the mouse is moved while a window input capture is in effect
 *
 * @param	InViewportClient	Level editor viewport client that captured the mouse input
 * @param	InViewport			Viewport that captured the mouse input
 * @param	InMouseX			New mouse cursor X coordinate
 * @param	InMouseY			New mouse cursor Y coordinate
 *
 * @return	true if input was handled
 */
bool FEdModeLandscape::CapturedMouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 MouseX, int32 MouseY)
{
	return MouseMove(ViewportClient, Viewport, MouseX, MouseY);
}

/** FEdMode: Called when a mouse button is pressed */
bool FEdModeLandscape::StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	using namespace UE::Landscape::Editor::Private;

	if (UISettings->CurrentGizmoActor.IsValid() && UISettings->CurrentGizmoActor->IsSelected() && GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo)
	{
		GIsGizmoDragging = true;
		return true;
	}
	else if (IsAdjustingBrush(InViewportClient))
	{ 
		GIsAdjustingBrush = true;
		// We're adjusting the brush via mouse tracking, return true in order to prevent the viewport client from doing any mouse dragging operation while we're doing it
		return true;
	}
	return false;
}



/** FEdMode: Called when the a mouse button is released */
bool FEdModeLandscape::EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport)
{
	using namespace UE::Landscape::Editor::Private;

	if (GIsGizmoDragging)
	{
		GIsGizmoDragging = false;
		return true;
	}
	if (GIsAdjustingBrush)
	{
		GIsAdjustingBrush = false;
		return true;
	}
	return false;
}

bool FEdModeLandscape::BeginTransform(const FGizmoState& InState)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (CurrentTool && CurrentTool->BeginTransform(InState))
	{
		return true;
	}

	return false;
}

bool FEdModeLandscape::EndTransform(const FGizmoState& InState)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (CurrentTool && CurrentTool->EndTransform(InState))
	{
		return true;
	}

	return false;
}

/** Trace under the mouse cursor and return the landscape hit and the hit location (in landscape quad space) */
bool FEdModeLandscape::LandscapeMouseTrace(FEditorViewportClient* ViewportClient, float& OutHitX, float& OutHitY)
{
	int32 MouseX = ViewportClient->Viewport->GetMouseX();
	int32 MouseY = ViewportClient->Viewport->GetMouseY();

	return LandscapeMouseTrace(ViewportClient, MouseX, MouseY, OutHitX, OutHitY);
}

bool FEdModeLandscape::LandscapeMouseTrace(FEditorViewportClient* ViewportClient, FVector& OutHitLocation)
{
	int32 MouseX = ViewportClient->Viewport->GetMouseX();
	int32 MouseY = ViewportClient->Viewport->GetMouseY();

	return LandscapeMouseTrace(ViewportClient, MouseX, MouseY, OutHitLocation);
}

/** Trace under the specified coordinates and return the landscape hit and the hit location (in landscape quad space) */
bool FEdModeLandscape::LandscapeMouseTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, float& OutHitX, float& OutHitY)
{
	FVector HitLocation;
	bool bResult = LandscapeMouseTrace(ViewportClient, MouseX, MouseY, HitLocation);
	OutHitX = static_cast<float>(HitLocation.X);
	OutHitY = static_cast<float>(HitLocation.Y);
	return bResult;
}

bool FEdModeLandscape::LandscapeMouseTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, FVector& OutHitLocation)
{
	// Cache a copy of the world pointer	
	UWorld* World = ViewportClient->GetWorld();

	// Compute a world space ray from the screen space mouse coordinates
	FSceneViewFamilyContext ViewFamily(FSceneViewFamilyContext::ConstructionValues(ViewportClient->Viewport, ViewportClient->GetScene(), ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(ViewportClient->IsRealtime()));

	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
	FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY);
	FVector MouseViewportRayDirection = MouseViewportRay.GetDirection();

	FVector Start = MouseViewportRay.GetOrigin();
	FVector End = Start + WORLD_MAX * MouseViewportRayDirection;
	if (ViewportClient->IsOrtho())
	{
		Start -= WORLD_MAX * MouseViewportRayDirection;
	}

	return LandscapeTrace(Start, End, MouseViewportRayDirection,OutHitLocation);
}

struct FEdModeLandscape::FProcessLandscapeTraceHitsResult
{
	FVector HitLocation;
	ULandscapeHeightfieldCollisionComponent* HeightfieldComponent;
	ALandscapeProxy* LandscapeProxy;
};

bool FEdModeLandscape::ProcessLandscapeTraceHits(const TArray<FHitResult>& InResults, FProcessLandscapeTraceHitsResult& OutLandscapeTraceHitsResult)
{
	for (int32 i = 0; i < InResults.Num(); i++)
	{
		const FHitResult& Hit = InResults[i];
		ULandscapeHeightfieldCollisionComponent* CollisionComponent = Cast<ULandscapeHeightfieldCollisionComponent>(Hit.Component.Get());
		if (CollisionComponent)
		{
			ALandscapeProxy* HitLandscape = CollisionComponent->GetLandscapeProxy();
			if (HitLandscape &&
				CurrentToolTarget.LandscapeInfo.IsValid() &&
				CurrentToolTarget.LandscapeInfo->LandscapeGuid == HitLandscape->GetLandscapeGuid())
			{
				OutLandscapeTraceHitsResult.HeightfieldComponent = CollisionComponent;
				OutLandscapeTraceHitsResult.HitLocation = Hit.Location;
				OutLandscapeTraceHitsResult.LandscapeProxy = HitLandscape;
				return true;
			}
		}
	}
	return false;
}

bool FEdModeLandscape::LandscapeTrace(const FVector& InRayOrigin, const FVector& InRayEnd, const FVector& InDirection, FVector& OutHitLocation)
{
	using namespace UE::Landscape::Editor::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(FEdModeLandscape_LandscapeTrace);
	if (!CurrentTool || !CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return false;
	}

	FVector Start = InRayOrigin;
	FVector End = InRayEnd;

	// Cache a copy of the world pointer
	UWorld* World = GetWorld();

	// Check Tool Trace first
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ToolTrace);
		if (CurrentTool->HitTrace(Start, End, OutHitLocation))
		{
			return true;
		}
	}
	
	TArray<FHitResult> Results;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LineTrace);
		// Each landscape component has 2 collision shapes, 1 of them is specific to landscape editor
		// Trace only ECC_Visibility channel, so we do hit only Editor specific shape
		if (World->LineTraceMultiByObjectType(Results, Start, End, FCollisionObjectQueryParams(ECollisionChannel::ECC_Visibility), FCollisionQueryParams(SCENE_QUERY_STAT(LandscapeTrace), true)))
		{
			if (FProcessLandscapeTraceHitsResult ProcessResult; ProcessLandscapeTraceHits(Results, ProcessResult))
			{
				OutHitLocation = ProcessResult.LandscapeProxy->LandscapeActorToWorld().InverseTransformPosition(ProcessResult.HitLocation);
			
				UE_CVLOG_SEGMENT_THICK(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, InRayOrigin, InRayEnd, FColor(100,255,100), 4, TEXT("landscape:ray-hit"));
				UE_CVLOG_LOCATION(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose,  ProcessResult.LandscapeProxy->LandscapeActorToWorld().TransformPosition(OutHitLocation), 20.0,  FColor(100,100,255), TEXT("landscape:point-hit"));
				return true;
			}
		}
	}

	UE_CVLOG_SEGMENT_THICK(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, InRayOrigin, InRayEnd, FColor(255,100,100), 2, TEXT("landscape:ray-miss"));
		
	if (CurrentTool->UseSphereTrace())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SphereLikeTrace);

		// If there is no landscape directly under the mouse search for a landscape collision
		// under the shape of the brush. 
		FVector UpVector = FVector::UpVector;
		if (FMath::IsNearlyEqual(FMath::Abs(FVector::DotProduct(UpVector, InDirection.GetSafeNormal(SMALL_NUMBER, UpVector))), 1.0f))
		{
			/* If we're too close to being colinear then pick another vector to do the cross products with. */
			UpVector = FVector::RightVector;
		}
		FVector RightVector = InDirection.Cross(UpVector).GetUnsafeNormal();
		UpVector = RightVector.Cross(InDirection);
		FMatrix RayTransform(InDirection, RightVector, UpVector, FVector::ZeroVector);
		
		// Add a slight offset so that we don't end up with the landscape brush appearing slightly on the border of the landscape when
		//  the "projected' brush center is at a distance from the border that is close to the radius : 
		const double AdditionalRadius = 50.0;
		double BrushRadius = UISettings->GetCurrentToolBrushRadius() + AdditionalRadius;

		auto LineTraceAroundCenter = [this, World, &Start, &End, &RayTransform] (float InAngle, float InRadius) -> TOptional<FProcessLandscapeTraceHitsResult>
		{
			TArray<FHitResult> Results;
			FVector Offset(0.0, InRadius * FMath::Cos(InAngle), InRadius * FMath::Sin(InAngle));
			FVector AdjustedRayStart = TransformPoint(RayTransform, Offset) + Start;
			FVector AdjustedRayEnd = TransformPoint(RayTransform, Offset) + End;
			double HitDistance = -1.0;

			if (World->LineTraceMultiByObjectType(Results, AdjustedRayStart, AdjustedRayEnd, FCollisionObjectQueryParams(ECollisionChannel::ECC_Visibility), FCollisionQueryParams(SCENE_QUERY_STAT(LandscapeTrace), true)))
			{
				if (FProcessLandscapeTraceHitsResult ProcessResult; ProcessLandscapeTraceHits(Results, ProcessResult))
				{
					UE_CVLOG_SEGMENT_THICK(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, AdjustedRayStart, AdjustedRayEnd, FColor(100, 255, 100), 4, TEXT("landscape:ray-hit"));
					UE_CVLOG_LOCATION(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, ProcessResult.HitLocation, 20.0, FColor(100, 100, 255), TEXT("landscape:point-hit"));

					HitDistance = (ProcessResult.HitLocation - Start).Length();
					return ProcessResult;
				}
			}
			else
			{
				UE_CVLOG_SEGMENT_THICK(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, AdjustedRayStart, AdjustedRayEnd, FColor(255, 100, 100), 2, TEXT("landscape:ray-miss"));
			}
			return TOptional<FProcessLandscapeTraceHitsResult>();
		};

		// Don't use a sphere cast as it can get very expensive for large brush sizes. Use several concentric ray casts instead : 
		const int32 MaxConcentricSamples = 10;
		const double RadiusIncrement = 1024.0;
		// Bound the number of concentric circles so that we don't have too many ray casts at a given time even if the brush radius is very large : 
		int32 NumConcentricSamples = FMath::Clamp(static_cast<int32>(BrushRadius / RadiusIncrement), 1, MaxConcentricSamples);
		double ActualRadiusIncrement = BrushRadius / NumConcentricSamples;

		const double DistanceBetweenAngularSamples = 1024.0;
		// Bound the number of angular samples so that we don't have too many ray casts at a given time even if the brush radius is very large : 
		const int32 MinAngularSamples = 6;
		const int32 MaxAngularSamples = 12;
		
		float MeanHeight = 0.0f;
		int32 Count = 0;
		TOptional<FProcessLandscapeTraceHitsResult> BestHitResult;
		int32 ConcentricIndex = 0;
		for (double Radius = ActualRadiusIncrement; Radius < (BrushRadius + UE_SMALL_NUMBER); Radius += ActualRadiusIncrement, ++ConcentricIndex)
		{
			double Circumference = TWO_PI * Radius;
			int32 NumAngularSamples = FMath::Clamp(static_cast<int32>(Circumference / DistanceBetweenAngularSamples), MinAngularSamples, MaxAngularSamples);
			double AngleIncrement = TWO_PI / NumAngularSamples;
			// Alternate by half the angle offset in order to have a better coverage over the entire radius : 
			double InitialAngle = (ConcentricIndex % 2) * AngleIncrement / 2.0;
			for (int32 AngularSampleIndex = 0; AngularSampleIndex < NumAngularSamples; ++AngularSampleIndex)
			{
				if (TOptional<FProcessLandscapeTraceHitsResult> HitResult = LineTraceAroundCenter(
					static_cast<float>(AngularSampleIndex * AngleIncrement + InitialAngle), static_cast<float>(Radius)); HitResult.IsSet())
				{ 
					// Compute the mean height in order to extrapolate the landscape hit to a "fake" landscape plane that extends beyond the borders : 
					if (TOptional<float> Height = HitResult->LandscapeProxy->GetHeightAtLocation(HitResult->HitLocation, EHeightfieldSource::Editor); Height.IsSet())
					{
						MeanHeight += Height.GetValue();
						Count++;
						BestHitResult = HitResult;
					}

				}
			}
		}

		if (Count > 0)
		{
			check(BestHitResult.IsSet());
			FVector PointOnPlane(BestHitResult->HitLocation);
			MeanHeight /= (float)Count;
			PointOnPlane.Z = MeanHeight;

			UE_CVLOG_LOCATION(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, PointOnPlane, 10.0, FColor(100, 100, 255), TEXT("landscape:point-on-plane"));

			if (FMath::Abs(FVector::DotProduct(InDirection, FVector::ZAxisVector)) < SMALL_NUMBER)
			{
				// the ray and plane are nearly parallel, and won't intersect (or if they do it will be wildly far away)
				return false;
			}

			const FPlane Plane(PointOnPlane, FVector::ZAxisVector);
			FVector EstimatedHitLocation = FMath::RayPlaneIntersection(Start, InDirection, Plane);
			check(!EstimatedHitLocation.ContainsNaN());

			UE_CVLOG_LOCATION(CVarVisualLogShowBrushPhysics->GetBool(), World, LogLandscapeEdMode, VeryVerbose, EstimatedHitLocation, 10.0, FColor(100, 100, 255), TEXT("landscape:estimated-hit-location"));

			OutHitLocation = BestHitResult->LandscapeProxy->LandscapeActorToWorld().InverseTransformPosition(EstimatedHitLocation);
			return true;
		}
	}

	return false;
}

bool FEdModeLandscape::LandscapePlaneTrace(FEditorViewportClient* ViewportClient, const FPlane& Plane, FVector& OutHitLocation)
{
	int32 MouseX = ViewportClient->Viewport->GetMouseX();
	int32 MouseY = ViewportClient->Viewport->GetMouseY();

	return LandscapePlaneTrace(ViewportClient, MouseX, MouseY, Plane, OutHitLocation);
}

bool FEdModeLandscape::LandscapePlaneTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, const FPlane& Plane, FVector& OutHitLocation)
{
	// Compute a world space ray from the screen space mouse coordinates
	FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
		ViewportClient->Viewport,
		ViewportClient->GetScene(),
		ViewportClient->EngineShowFlags)
		.SetRealtimeUpdate(ViewportClient->IsRealtime()));
	FSceneView* View = ViewportClient->CalcSceneView(&ViewFamily);
	FViewportCursorLocation MouseViewportRay(View, ViewportClient, MouseX, MouseY);

	FVector Start = MouseViewportRay.GetOrigin();
	FVector End = Start + WORLD_MAX * MouseViewportRay.GetDirection();

	OutHitLocation = FMath::LinePlaneIntersection(Start, End, Plane);

	return true;
}

namespace
{
	const int32 SelectionSizeThresh = 2 * 256 * 256;
	FORCEINLINE bool IsSlowSelect(ULandscapeInfo* LandscapeInfo)
	{
		if (LandscapeInfo)
		{
			int32 MinX = MAX_int32, MinY = MAX_int32, MaxX = MIN_int32, MaxY = MIN_int32;
			LandscapeInfo->GetSelectedExtent(MinX, MinY, MaxX, MaxY);
			return (MinX != MAX_int32 && ((MaxX - MinX) * (MaxY - MinY)));
		}
		return false;
	}
};

EEditAction::Type FEdModeLandscape::GetActionEditDuplicate()
{
	EEditAction::Type Result = EEditAction::Skip;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->GetActionEditDuplicate();
		}
	}

	return Result;
}

EEditAction::Type FEdModeLandscape::GetActionEditDelete()
{
	EEditAction::Type Result = EEditAction::Skip;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->GetActionEditDelete();
		}

		if (Result == EEditAction::Skip)
		{
			// Prevent deleting Gizmo during LandscapeEdMode
			if (UISettings->CurrentGizmoActor.IsValid())
			{
				if (UISettings->CurrentGizmoActor->IsSelected())
				{
					if (GEditor->GetSelectedActors()->Num() > 1)
					{
						GEditor->GetSelectedActors()->Deselect(UISettings->CurrentGizmoActor.Get());
						Result = EEditAction::Skip;
					}
					else
					{
						Result = EEditAction::Halt;
					}
				}
			}
		}
	}

	return Result;
}

EEditAction::Type FEdModeLandscape::GetActionEditCut()
{
	EEditAction::Type Result = EEditAction::Skip;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->GetActionEditCut();
		}
	}

	if (Result == EEditAction::Skip)
	{
		// Special case: we don't want the 'normal' cut operation to be possible at all while in this mode, 
		// so we need to stop evaluating the others in-case they come back as true.
		return EEditAction::Halt;
	}

	return Result;
}

EEditAction::Type FEdModeLandscape::GetActionEditCopy()
{
	EEditAction::Type Result = EEditAction::Skip;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->GetActionEditCopy();
		}

		if (Result == EEditAction::Skip)
		{
			if (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo || GLandscapeEditRenderMode & (ELandscapeEditRenderMode::Select))
			{
				if (UISettings->CurrentGizmoActor.IsValid() && GizmoBrush && UISettings->CurrentGizmoActor->TargetLandscapeInfo.IsValid())
				{
					Result = EEditAction::Process;
				}
			}
		}
	}

	return Result;
}

EEditAction::Type FEdModeLandscape::GetActionEditPaste()
{
	EEditAction::Type Result = EEditAction::Skip;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->GetActionEditPaste();
		}

		if (Result == EEditAction::Skip)
		{
			if (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo || GLandscapeEditRenderMode & (ELandscapeEditRenderMode::Select))
			{
				if (UISettings->CurrentGizmoActor.IsValid() && GizmoBrush && UISettings->CurrentGizmoActor->TargetLandscapeInfo.IsValid())
				{
					Result = EEditAction::Process;
				}
			}
		}
	}

	return Result;
}

bool FEdModeLandscape::ProcessEditDuplicate()
{
	if (!IsEditingEnabled())
	{
		return true;
	}

	bool Result = false;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->ProcessEditDuplicate();
		}
	}

	return Result;
}

bool FEdModeLandscape::ProcessEditDelete()
{
	if (!IsEditingEnabled())
	{
		return true;
	}

	bool Result = false;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->ProcessEditDelete();
		}
	}

	return Result;
}

bool FEdModeLandscape::ProcessEditCut()
{
	if (!IsEditingEnabled())
	{
		return true;
	}

	bool Result = false;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->ProcessEditCut();
		}
	}

	return Result;
}

bool FEdModeLandscape::ProcessEditCopy()
{
	if (!IsEditingEnabled())
	{
		return true;
	}

	bool Result = false;

	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->ProcessEditCopy();
		}

		if (!Result)
		{
			ALandscapeBlueprintBrushBase* CurrentlySelectedBPBrush = nullptr;
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				CurrentlySelectedBPBrush = Cast<ALandscapeBlueprintBrushBase>(*It);
				if (CurrentlySelectedBPBrush)
				{
					break;
				}
			}

			if (!CurrentlySelectedBPBrush)
			{
				bool IsSlowTask = IsSlowSelect(UISettings->CurrentGizmoActor->TargetLandscapeInfo.Get());
				if (IsSlowTask)
				{
					GWarn->BeginSlowTask(LOCTEXT("BeginFitGizmoAndCopy", "Fit Gizmo to Selected Region and Copy Data..."), true);
				}

				FScopedTransaction Transaction(LOCTEXT("LandscapeGizmo_Copy", "Copy landscape data to Gizmo"));
				UISettings->CurrentGizmoActor->Modify();
				UISettings->CurrentGizmoActor->FitToSelection();
				CopyDataToGizmo();
				SetCurrentTool(FName("CopyPaste"));

				if (IsSlowTask)
				{
					GWarn->EndSlowTask();
				}

				Result = true;
			}			
		}
	}

	return Result;
}

bool FEdModeLandscape::ProcessEditPaste()
{
	if (!IsEditingEnabled())
	{
		return true;
	}

	bool Result = false;

	const ULandscapeEditLayerBase* SplinesEditLayer = nullptr;

	if (CurrentTool == (FLandscapeTool*)SplinesTool)
	{
		ALandscape* Landscape = GetLandscape();
		SplinesEditLayer = Landscape ? Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass()) : nullptr;
	}
	FText Reason;
	if (!CanEditLayer(&Reason, SplinesEditLayer))
	{
		FMessageDialog::Open(EAppMsgType::Ok, Reason);
		return Result;
	}
	
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		if (CurrentTool != nullptr)
		{
			Result = CurrentTool->ProcessEditPaste();
		}

		if (!Result)
		{
			ALandscapeBlueprintBrushBase* CurrentlySelectedBPBrush = nullptr;
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				CurrentlySelectedBPBrush = Cast<ALandscapeBlueprintBrushBase>(*It);
				if (CurrentlySelectedBPBrush)
				{
					break;
				}
			}

			if (!CurrentlySelectedBPBrush)
			{
				bool IsSlowTask = IsSlowSelect(UISettings->CurrentGizmoActor->TargetLandscapeInfo.Get());
				if (IsSlowTask)
				{
					GWarn->BeginSlowTask(LOCTEXT("BeginPasteGizmoDataTask", "Paste Gizmo Data..."), true);
				}
				PasteDataFromGizmo();
				SetCurrentTool(FName("CopyPaste"));
				if (IsSlowTask)
				{
					GWarn->EndSlowTask();
				}

				Result = true;
			}
		}
	}

	return Result;
}

bool FEdModeLandscape::HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
	{
		return false;
	}

	// Override Click Input for Splines Tool
	if (CurrentTool && CurrentTool->HandleClick(HitProxy, Click))
	{
		return true;
	}

	return false;
}

bool FEdModeLandscape::IsAdjustingBrush(FEditorViewportClient* InViewportClient) const
{
	const FLandscapeEditorCommands& LandscapeActions = FLandscapeEditorCommands::Get();
	if (InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushSizeAndFalloff))
	{
		return true;
	}
	if (InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushSize))
	{
		return true;
	}
	if (InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushFalloff))
	{
		return true;
	}
	if (InViewportClient->IsCommandChordPressed(LandscapeActions.DragBrushStrength))
	{
		return true;
	}
	return false;
}

void FEdModeLandscape::ChangeBrushSize(bool bIncrease)
{
	UISettings->Modify();
	if (CurrentBrush->GetBrushType() == ELandscapeBrushType::Component)
	{
		int32 Radius = UISettings->BrushComponentSize;
		if (bIncrease)
		{
			++Radius;
		}
		else
		{
			--Radius;
		}
		Radius = (int32)FMath::Clamp(Radius, 1, 64);
		UISettings->BrushComponentSize = Radius;
	}
	else
	{
		float Radius = UISettings->GetCurrentToolBrushRadius();
		const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>();
		const float SliderMin = 10.0f;
		const float SliderMax = LandscapeSettings->GetBrushSizeUIMax();
		float Diff = 0.05f; 
		if (!bIncrease)
		{
			Diff = -Diff;
		}

		float NewValue = Radius * (1.0f + Diff);
		if (bIncrease)
		{
			NewValue = FMath::Max(NewValue, Radius + 1.0f);
		}
		else
		{
			NewValue = FMath::Min(NewValue, Radius - 1.0f);
		}

		NewValue = FMath::Clamp(NewValue, SliderMin, SliderMax);
		UISettings->SetCurrentToolBrushRadius(NewValue);
	}
}


void FEdModeLandscape::ChangeBrushFalloff(bool bIncrease)
{
	UISettings->Modify();
	float Falloff = UISettings->GetCurrentToolBrushFalloff();
	const float SliderMin = 0.0f;
	const float SliderMax = 1.0f;
	float Diff = 0.05f; 
	if (!bIncrease)
	{
		Diff = -Diff;
	}

	float NewValue = Falloff * (1.0f + Diff);

	if (bIncrease)
	{
		NewValue = FMath::Max(NewValue, Falloff + 0.05f);
	}
	else
	{
		NewValue = FMath::Min(NewValue, Falloff - 0.05f);
	}

	NewValue = FMath::Clamp(NewValue, SliderMin, SliderMax);
	UISettings->SetCurrentToolBrushFalloff(NewValue);
}


void FEdModeLandscape::ChangeBrushStrength(bool bIncrease)
{
	UISettings->Modify();
	float Strength = UISettings->GetCurrentToolStrength();
	const float SliderMin = 0.01f;
	const float SliderMax = 10.0f;
	float Diff = 0.05f; //6.0f / SliderMax;
	if (!bIncrease)
	{
		Diff = -Diff;
	}

	float NewValue = Strength * (1.0f + Diff);

	if (bIncrease)
	{
		NewValue = FMath::Max(NewValue, Strength + 0.05f);
	}
	else
	{
		NewValue = FMath::Min(NewValue, Strength - 0.05f);
	}

	NewValue = FMath::Clamp(NewValue, SliderMin, SliderMax);
	UISettings->SetCurrentToolStrength(NewValue);
}

void FEdModeLandscape::ChangeAlphaBrushRotation(bool bIncrease)
{
	const float SliderMin = -180.f;
	const float SliderMax = 180.f;
	const float DefaultDiffValue = 10.f;
	const float Diff = bIncrease ? DefaultDiffValue : -DefaultDiffValue;
	UISettings->Modify();
	UISettings->AlphaBrushRotation = FMath::Clamp(UISettings->AlphaBrushRotation + Diff, SliderMin, SliderMax);
}


/** FEdMode: Called when a key is pressed */
bool FEdModeLandscape::InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (!ErrorReasonOnMouseUp.IsEmpty() && Key == EKeys::LeftMouseButton && Event == IE_Released)
	{
		FMessageDialog::Open(EAppMsgType::Ok, ErrorReasonOnMouseUp);
		ErrorReasonOnMouseUp = FText::GetEmpty();
		return false;
	}


	if(IsAdjustingBrush(ViewportClient))
	{
		ToolActiveViewport = Viewport;
		return false; // false to let FEditorViewportClient.InputKey start mouse tracking and enable InputDelta() so we can use it
	}

	if (Event != IE_Released)
	{
		ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");

		if (LandscapeEditorModule.GetLandscapeLevelViewportCommandList()->ProcessCommandBindings(Key, FSlateApplication::Get().GetModifierKeys(), false/*Event == IE_Repeat*/))
		{
			return true;
		}
	}

	if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
	{
		if (CurrentTool && CurrentTool->InputKey(ViewportClient, Viewport, Key, Event))
		{
			return false; // false to let FEditorViewportClient.InputKey start/end mouse tracking and enable InputDelta() so we can use it
		}
	}
	else
	{
		// Override Key Input for Selection Brush
		if (CurrentBrush)
		{
			TOptional<bool> BrushKeyOverride = CurrentBrush->InputKey(ViewportClient, Viewport, Key, Event);
			if (BrushKeyOverride.IsSet())
			{
				return BrushKeyOverride.GetValue();
			}
		}

		if (CurrentTool && CurrentTool->InputKey(ViewportClient, Viewport, Key, Event) == true)
		{
			return true;
		}

		// Require Ctrl or not as per user preference
		ELandscapeFoliageEditorControlType LandscapeEditorControlType = GetDefault<ULevelEditorViewportSettings>()->LandscapeEditorControlType;

		// HACK - Splines tool has not yet been updated to support not using ctrl
		if (CurrentBrush->GetBrushType() == ELandscapeBrushType::Splines)
		{
			LandscapeEditorControlType = ELandscapeFoliageEditorControlType::RequireCtrl;
		}

		// Special case to handle where user paint with Left Click then pressing a moving camera input, we do not want to process them so as long as the tool is active ignore other input
		if (CurrentTool != nullptr && CurrentTool->IsToolActive())
		{
			return true;
		}

		if (Key == EKeys::LeftMouseButton && Event == IE_Pressed)
		{
			// When debugging it's possible to miss the "mouse released" event, if we get a "mouse pressed" event when we think it's already pressed then treat it as release first
			if (ToolActiveViewport)
			{
				CurrentTool->EndTool(ViewportClient); //-V595
				Viewport->CaptureMouse(false);
				ToolActiveViewport = nullptr;
			}

			// Only activate tool if we're not already moving the camera and we're not trying to drag a transform widget
			// Not using "if (!ViewportClient->IsMovingCamera())" because it's wrong in ortho viewports :D
			bool bMovingCamera = Viewport->KeyState(EKeys::MiddleMouseButton) || Viewport->KeyState(EKeys::RightMouseButton) || IsAltDown(Viewport);

			if ((Viewport->IsPenActive() && Viewport->GetTabletPressure() > 0.0f) ||
				(!bMovingCamera && ViewportClient->GetCurrentWidgetAxis() == EAxisList::None &&
					((LandscapeEditorControlType == ELandscapeFoliageEditorControlType::IgnoreCtrl) ||
					 (LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl   && IsCtrlDown(Viewport)) ||
					 (LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireNoCtrl && !IsCtrlDown(Viewport)))))
			{
				if (CurrentTool && (CurrentTool->GetSupportedTargetType() == ELandscapeToolTargetType::Invalid || CurrentToolTarget.TargetType != ELandscapeToolTargetType::Invalid))
				{
					FVector HitLocation;
					if (LandscapeMouseTrace(ViewportClient, HitLocation))
					{
						if (CurrentTool->AffectsEditLayers() && !CanEditSelectedTargetLayer(&ErrorReasonOnMouseUp))
						{
							return true;
						}

						Viewport->CaptureMouse(true);

						if (CurrentTool->CanToolBeActivated())
						{
							bool bToolActive = CurrentTool->BeginTool(ViewportClient, CurrentToolTarget, HitLocation);
							if (bToolActive)
							{
								ToolActiveViewport = Viewport;
							}
							else
							{
								ToolActiveViewport = nullptr;
								Viewport->CaptureMouse(false);
							}
							ViewportClient->Invalidate(false, false);
							return bToolActive;
						}
					}
				}
				return true;
			}

			if (UISettings->CurrentGizmoActor.IsValid() && UISettings->CurrentGizmoActor->IsSelected() && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo) && IsAltDown(Viewport))
			{ 
				// When manipulating the gizmo with Alt down, don't do anything but return true to indicate it's on purpose and prevent the editor from doing the usual thing
				//  (alt-drag is usually "duplicate actor" and we want to prevent duplicating the gizmo actor) 
				return true; 
			}
		}

		if (Key == EKeys::LeftMouseButton ||
			(LandscapeEditorControlType == ELandscapeFoliageEditorControlType::RequireCtrl && (Key == EKeys::LeftControl || Key == EKeys::RightControl)))
		{
			if (Event == IE_Released && CurrentTool && CurrentTool->IsToolActive() && ToolActiveViewport)
			{
				//Set the cursor position to that of the slate cursor so it wont snap back
				Viewport->SetPreCaptureMousePosFromSlateCursor();
				CurrentTool->EndTool(ViewportClient);
				Viewport->CaptureMouse(false);
				ToolActiveViewport = nullptr;
				return true;
			}
		}

		// Prev tool
		if (Event == IE_Pressed && Key == EKeys::Comma)
		{
			if (CurrentTool && CurrentTool->IsToolActive() && ToolActiveViewport)
			{
				CurrentTool->EndTool(ViewportClient);
				Viewport->CaptureMouse(false);
				ToolActiveViewport = nullptr;
			}

			if (CurrentTool)
			{
				int32 OldToolIndex = CurrentToolMode->ValidTools.Find(CurrentTool->GetToolName());
				int32 NewToolIndex = FMath::Max(OldToolIndex - 1, 0);
				SetCurrentTool(CurrentToolMode->ValidTools[NewToolIndex]);
			}
			else if (CurrentToolMode && CurrentToolMode->ValidTools.Num() > 0)
			{
				SetCurrentTool(CurrentToolMode->ValidTools[0]);
			}

			return true;
		}

		// Next tool
		if (Event == IE_Pressed && Key == EKeys::Period)
		{
			if (CurrentTool && ToolActiveViewport)
			{
				CurrentTool->EndTool(ViewportClient);
				Viewport->CaptureMouse(false);
				ToolActiveViewport = nullptr;
			}

			if (CurrentTool)
			{
				int32 OldToolIndex = CurrentToolMode->ValidTools.Find(CurrentTool->GetToolName());
				int32 NewToolIndex = FMath::Min(OldToolIndex + 1, CurrentToolMode->ValidTools.Num() - 1);
				SetCurrentTool(CurrentToolMode->ValidTools[NewToolIndex]);
			}
			else if (CurrentToolMode && CurrentToolMode->ValidTools.Num() > 0)
			{
				SetCurrentTool(CurrentToolMode->ValidTools[0]);
			}

			return true;
		}
	}

	return false;
}

/** FEdMode: Called when mouse drag input is applied */
bool FEdModeLandscape::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (CurrentTool && CurrentTool->InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale))
	{
		return true;
	}

	return false;
}

void FEdModeLandscape::SetCurrentToolMode(FName ToolModeName, bool bRestoreCurrentTool /*= true*/)
{
	if (CurrentToolMode == nullptr || ToolModeName != CurrentToolMode->ToolModeName)
	{
		if (FLandscapeToolMode* NewToolMode = Algo::FindBy(LandscapeToolModes, ToolModeName, &FLandscapeToolMode::ToolModeName))
		{
			// Save current tool mode before changing
			LastActiveToolModeName = CurrentToolMode ? CurrentToolMode->ToolModeName : FName("ToolMode_Sculpt");

			CurrentToolMode = NewToolMode;

			if (bRestoreCurrentTool)
			{
				if (CurrentToolMode->CurrentToolName.IsNone())
				{
					CurrentToolMode->CurrentToolName = CurrentToolMode->ValidTools[0];
					CurrentToolMode->CurrentTargetLayerName = NAME_None;
				}
				SetCurrentTool(CurrentToolMode->CurrentToolName, CurrentToolMode->CurrentTargetLayerName);
			}
			if (Toolkit.IsValid())
			{
				Toolkit->SetCurrentPalette(ToolModeName);
			}
		}
	}
}

void FEdModeLandscape::SetCurrentTool(FName ToolName, FName TargetLayerName)
{
	// Several tools have identically named versions for sculpting and painting
	// Prefer the one with the same target type as the current mode

	int32 BackupToolIndex = INDEX_NONE;
	int32 ToolIndex = INDEX_NONE;
	for (int32 i = 0; i < LandscapeTools.Num(); ++i)
	{
		FLandscapeTool* Tool = LandscapeTools[i].Get();
		if (ToolName == Tool->GetToolName())
		{
			if (EnumHasAnyFlags(UE::Landscape::GetLandscapeToolTargetTypeAsFlags(Tool->GetSupportedTargetType()), CurrentToolMode->SupportedTargetTypes))
			{
				ToolIndex = i;
				break;
			}
			else if (BackupToolIndex == INDEX_NONE)
			{
				BackupToolIndex = i;
			}
		}
	}

	if (ToolIndex == INDEX_NONE)
	{
		checkf(BackupToolIndex != INDEX_NONE, TEXT("Tool '%s' not found, please check name is correct!"), *ToolName.ToString());
		ToolIndex = BackupToolIndex;
	}
	check(ToolIndex != INDEX_NONE);

	SetCurrentTool(ToolIndex, TargetLayerName);
}

void FEdModeLandscape::SetCurrentTargetLayer(FName TargetLayerName, TWeakObjectPtr<ULandscapeLayerInfoObject> LayerInfo)
{
	if (CurrentToolMode)
	{
		// Cache current Layer Name so we can set it back when switching between Modes
		CurrentToolMode->CurrentTargetLayerName = TargetLayerName;
	}
	CurrentToolTarget.LayerName = TargetLayerName;
	CurrentToolTarget.LayerInfo = LayerInfo;
}

void FEdModeLandscape::SetCurrentTool(int32 ToolIndex, FName TargetLayerName)
{
	// Save current tool name before changing
	LastActiveToolName = CurrentTool ? CurrentTool->GetToolName() : FName("Sculpt");

	if (CurrentTool)
	{
		CurrentTool->PreviousBrushIndex = CurrentBrushSetIndex;
		CurrentTool->ExitTool();
		CurrentTool = nullptr;
	}
	CurrentToolIndex = LandscapeTools.IsValidIndex(ToolIndex) ? ToolIndex : 0;
	FLandscapeTool* NewTool = LandscapeTools[CurrentToolIndex].Get();

	if (!CurrentToolMode->ValidTools.Contains(NewTool->GetToolName()))
	{
		// if tool isn't valid for this mode then automatically switch modes. this mostly happens with shortcut keys
		FLandscapeToolMode* BackupToolMode = LandscapeToolModes.FindByPredicate([NewToolName = NewTool->GetToolName()](const FLandscapeToolMode& ToolMode)
			{
				return ToolMode.ValidTools.Contains(NewToolName);
			});

		// if no valid mode is found, keep the mode and fallback to the first valid tool
		if (BackupToolMode != nullptr)
		{
			SetCurrentToolMode(BackupToolMode->ToolModeName, false);
		}
		else
		{
			SetCurrentTool(CurrentToolMode->ValidTools[0]);
			return;
		}
	}

	// Assign 
	CurrentTool = NewTool;

	// Set target type appropriate for tool
	if (CurrentTool->GetSupportedTargetType() == ELandscapeToolTargetType::Invalid)
	{
		CurrentToolTarget.TargetType = ELandscapeToolTargetType::Invalid;
		SetCurrentTargetLayer(NAME_None, nullptr);
	}
	else
	{
		if (CurrentTool->GetSupportedTargetType() != CurrentToolTarget.TargetType)
		{
			const ELandscapeToolTargetType TargetType = CurrentTool->GetSupportedTargetType();
			auto Filter = [TargetType, TargetLayerName](const TSharedRef<FLandscapeTargetListInfo>& Target) { return (Target->TargetType == TargetType) && (TargetLayerName == NAME_None || TargetLayerName == Target->LayerName); };
			const TSharedRef<FLandscapeTargetListInfo>* Target = LandscapeTargetList.FindByPredicate(Filter);
			if (Target != nullptr)
			{
				check(CurrentToolTarget.LandscapeInfo == (*Target)->LandscapeInfo);
				CurrentToolTarget.TargetType = (*Target)->TargetType;
				SetCurrentTargetLayer((*Target)->LayerName, (*Target)->LayerInfoObj);
			}
			else // can happen with for example paint tools if there are no paint layers defined
			{
				CurrentToolTarget.TargetType = ELandscapeToolTargetType::Invalid;
				SetCurrentTargetLayer(NAME_None, nullptr);
			}
		}
	}

	CurrentTool->EnterTool();

	CurrentTool->SetEditRenderType();
	//bool MaskEnabled = CurrentTool->SupportsMask() && CurrentToolTarget.LandscapeInfo.IsValid() && CurrentToolTarget.LandscapeInfo->SelectedRegion.Num();

	CurrentToolMode->CurrentToolName = CurrentTool->GetToolName();

	// Set Brush
	if (!LandscapeBrushSets.IsValidIndex(CurrentTool->PreviousBrushIndex))
	{
		SetCurrentBrushSet(CurrentTool->ValidBrushes[0]);
	}
	else
	{
		SetCurrentBrushSet(CurrentTool->PreviousBrushIndex);
	}

	// Update GizmoActor Landscape Target (is this necessary?)
	if (UISettings->CurrentGizmoActor.IsValid() && CurrentToolTarget.LandscapeInfo.IsValid())
	{
		UISettings->CurrentGizmoActor->SetTargetLandscape(CurrentToolTarget.LandscapeInfo.Get());
	}

	if (Toolkit.IsValid())
	{
		StaticCastSharedPtr<FLandscapeToolKit>(Toolkit)->NotifyToolChanged();
	}

	GEditor->RedrawLevelEditingViewports();
}

void FEdModeLandscape::RefreshDetailPanel()
{
	if (Toolkit.IsValid() && !bUpdatingLandscapeInfo)
	{
		StaticCastSharedPtr<FLandscapeToolKit>(Toolkit)->RefreshDetailPanel();
	}
}

void FEdModeLandscape::RefreshInspectedObjectsDetailPanel()
{
	// Inspected objects update separate from the main details panel 
	// When inspecting an object, we don't want the main panel to refresh because this resets the scroll 
	// navigating the user away from their current view
	if (Toolkit.IsValid() && !bUpdatingLandscapeInfo)
	{
		StaticCastSharedPtr<FLandscapeToolKit>(Toolkit)->RefreshInspectedObjectsDetailPanel();
	}
}

void FEdModeLandscape::RegenerateLayerThumbnails()
{
	// Regenerate the lists that might hold on to a ThumbnailMIC : 
	UpdateTargetList(/*bRegenerateThumbnails = */true);
	UISettings->RefreshImportLayersList(/*bRefreshFromTarget = */false, /*bRegenerateThumbnails = */true); // Once for NewLandscape_Material
	UISettings->RefreshImportLayersList(/*bRefreshFromTarget = */true, /*bRegenerateThumbnails = */true); // Once for the target list
}

void FEdModeLandscape::SetCurrentBrushSet(FName BrushSetName)
{
	for (int32 BrushIndex = 0; BrushIndex < LandscapeBrushSets.Num(); BrushIndex++)
	{
		if (BrushSetName == LandscapeBrushSets[BrushIndex].BrushSetName)
		{
			SetCurrentBrushSet(BrushIndex);
			return;
		}
	}
}

void FEdModeLandscape::SetCurrentBrushSet(int32 BrushSetIndex)
{
	if (CurrentBrushSetIndex != BrushSetIndex)
	{
		LandscapeBrushSets[CurrentBrushSetIndex].PreviousBrushIndex = LandscapeBrushSets[CurrentBrushSetIndex].Brushes.IndexOfByKey(CurrentBrush);

		CurrentBrushSetIndex = BrushSetIndex;
		if (CurrentTool)
		{
			CurrentTool->PreviousBrushIndex = BrushSetIndex;
		}

		SetCurrentBrush(LandscapeBrushSets[CurrentBrushSetIndex].PreviousBrushIndex);
	}
}

void FEdModeLandscape::SetCurrentBrush(FName BrushName)
{
	for (int32 BrushIndex = 0; BrushIndex < LandscapeBrushSets[CurrentBrushSetIndex].Brushes.Num(); BrushIndex++)
	{
		if (BrushName == LandscapeBrushSets[CurrentBrushSetIndex].Brushes[BrushIndex]->GetBrushName())
		{
			SetCurrentBrush(BrushIndex);
			return;
		}
	}
}

void FEdModeLandscape::SetCurrentBrush(int32 BrushIndex)
{
	if (CurrentBrush != LandscapeBrushSets[CurrentBrushSetIndex].Brushes[BrushIndex])
	{
		CurrentBrush->LeaveBrush();
		CurrentBrush = LandscapeBrushSets[CurrentBrushSetIndex].Brushes[BrushIndex];
		CurrentBrush->EnterBrush();

		if (Toolkit.IsValid())
		{
			StaticCastSharedPtr<FLandscapeToolKit>(Toolkit)->NotifyBrushChanged();
		}
	}
}

const TArray<ALandscapeBlueprintBrushBase*>& FEdModeLandscape::GetBrushList() const
{
	return BrushList;
}

const TArray<TSharedRef<FLandscapeTargetListInfo>>& FEdModeLandscape::GetTargetList() const
{
	return LandscapeTargetList;
}

const TArray<FLandscapeListInfo>& FEdModeLandscape::GetLandscapeList()
{
	return LandscapeList;
}

const FString FEdModeLandscape::GetTargetLayerAssetPackagePath(const bool bIsEmptyPathValid) const
{
	if (UISettings->TargetLayerAssetFilePath.bUseAssetDirectoryPath)
	{
		// When default path is enabled but no path has been chosen we should use the shared assets path
		// UI methods can optionally return an empty path to show a HintText
		if (!UISettings->TargetLayerAssetFilePath.DirectoryPath.Path.IsEmpty() || bIsEmptyPathValid)
		{
			return UISettings->TargetLayerAssetFilePath.DirectoryPath.Path;
		}
	}

	const ULevel* Level = GetWorld()->GetCurrentLevel();
	return UE::Landscape::GetSharedAssetsPath(Level);
}

int32 FEdModeLandscape::UpdateLandscapeList()
{
	LandscapeList.Empty();

	// Use EditorModeManager to skip landscape actors that can't be edited
	// One example is the ULevelInstanceEditorMode when in level instance restricted mode and 
	// the actor is not part of the editing level instance.
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> FirstLevelEditor = LevelEditorModule.GetFirstLevelEditor();
	auto IsEditingDisallowedByEdModes = [&FirstLevelEditor](AActor* Actor)
	{
		return FirstLevelEditor && Actor && FirstLevelEditor->GetEditorModeManager().IsEditingDisallowed(Actor);
	};

	if (!UISettings->CurrentGizmoActor.IsValid())
	{
		ALandscapeGizmoActiveActor* GizmoActor = nullptr;
		if (TActorIterator<ALandscapeGizmoActiveActor> It(GetWorld()); It)
		{
			GizmoActor = *It;
		}
	}

	int32 CurrentIndex = INDEX_NONE;
	UWorld* World = GetWorld();
	
	if (World)
	{
		int32 Index = 0;
		auto& LandscapeInfoMap = ULandscapeInfoMap::GetLandscapeInfoMap(World);

		auto AddLandscape = [this, &CurrentIndex, &Index](ALandscape* Landscape, ULandscapeInfo* LandscapeInfo)
		{
			if (Landscape)
			{
				Landscape->RegisterLandscapeEdMode(this);
			}

			ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
			if (LandscapeProxy)
			{
				if (CurrentToolTarget.LandscapeInfo == LandscapeInfo)
				{
					CurrentIndex = Index;

					// Update GizmoActor Landscape Target (is this necessary?)
					if (UISettings->CurrentGizmoActor.IsValid())
					{
						UISettings->CurrentGizmoActor->SetTargetLandscape(LandscapeInfo);
					}
				}

				int32 MinX, MinY, MaxX, MaxY;
				int32 Width = 0, Height = 0;
				if (LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
				{
					Width = MaxX - MinX + 1;
					Height = MaxY - MinY + 1;
				}

				LandscapeList.Add(FLandscapeListInfo(*LandscapeProxy->GetName(), LandscapeInfo,
					LandscapeInfo->ComponentSizeQuads, LandscapeInfo->ComponentNumSubsections, Width, Height));
				Index++;
			}
		};

		TArray<TPair<ALandscape*, ULandscapeInfo*>> PersistentLevelLandscapes;
		for (auto It = LandscapeInfoMap.Map.CreateIterator(); It; ++It)
		{
			ULandscapeInfo* LandscapeInfo = It.Value();
			if (IsValid(LandscapeInfo) && LandscapeInfo->SupportsLandscapeEditing())
			{
				ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
				if (IsEditingDisallowedByEdModes(Landscape))
				{
					if (Landscape && Landscape->GetLevel()->IsPersistentLevel())
					{
						PersistentLevelLandscapes.Emplace(Landscape, LandscapeInfo);
					}
				}
				else
				{
					AddLandscape(Landscape, LandscapeInfo);
				}
			}
		}

		// Landscape Mode will fallback on Persistent Level's Landscape(s) if none are found after
		// applying the editor modes IsEditingDisallowed.
		if (LandscapeList.IsEmpty() && !PersistentLevelLandscapes.IsEmpty())
		{
			for (const TPair<ALandscape*, ULandscapeInfo*>& Landscape : PersistentLevelLandscapes)
			{
				AddLandscape(Landscape.Key, Landscape.Value);
			}
		}
	}

	if (CurrentIndex == INDEX_NONE)
	{
		if (LandscapeList.Num() > 0)
		{
			FName CurrentToolName = CurrentTool != nullptr ? CurrentTool->GetToolName() : FName();
			SetLandscapeInfo(LandscapeList[0].Info);
			CurrentIndex = 0;
			SetSelectedEditLayer(0);

			UpdateLayerUsageInformation();
						
			if (!CurrentToolName.IsNone())
			{
				SetCurrentTool(CurrentToolName);
			}
		}
		else
		{
			// no landscape, switch to "new landscape" tool
			SetLandscapeInfo(nullptr);
			SetCurrentToolMode("ToolMode_Manage", false);
			SetCurrentTool("NewLandscape");
		}
	}

	if (!CanEditCurrentTarget())
	{
		SetCurrentToolMode("ToolMode_Manage", false);
		SetCurrentTool("NewLandscape");
	}

	return CurrentIndex;
}

bool FEdModeLandscape::IsGridBased() const
{
	return GetWorld()->GetSubsystem<ULandscapeSubsystem>()->IsGridBased();
}

void FEdModeLandscape::SetTargetLandscape(const TWeakObjectPtr<ULandscapeInfo>& InLandscapeInfo)
{
	if ((CurrentToolTarget.LandscapeInfo == InLandscapeInfo) || !InLandscapeInfo.IsValid())
	{
		return;
	}

	// Unregister from old one
	if (CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if (ALandscape* Landscape = GetLandscape())
		{
			Landscape->OnBlueprintBrushChangedDelegate().RemoveAll(this);
		}
	}

	SetLandscapeInfo(InLandscapeInfo.Get());
	// force a Leave and Enter the current tool, in case it has something about the current landscape cached
	SetCurrentTool(CurrentToolIndex);
	if (UISettings->CurrentGizmoActor.IsValid())
	{
		UISettings->CurrentGizmoActor->SetTargetLandscape(CurrentToolTarget.LandscapeInfo.Get());
	}

	// register to new one
	if (CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if (ALandscape* Landscape = GetLandscape())
		{
			Landscape->OnBlueprintBrushChangedDelegate().AddRaw(this, &FEdModeLandscape::RefreshDetailPanel);
		}
	}

	UpdateLayerUsageInformation();
}

bool FEdModeLandscape::CanEditCurrentTarget(FText* Reason) const
{
	static FText DummyReason;
	FText& LocalReason = Reason ? *Reason : DummyReason;

	if (!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		LocalReason = NSLOCTEXT("UnrealEd", "LandscapeInvalidTarget", "No landscape selected.");
		return false;
	}

	ALandscapeProxy* Proxy = CurrentToolTarget.LandscapeInfo->GetLandscapeProxy();

	// Landscape Layer Editing not available without a loaded Landscape Actor
	if (GetLandscape() == nullptr)
	{
		if (!Proxy)
		{
           		LocalReason = NSLOCTEXT("UnrealEd", "LandscapeNotFound", "No Landscape found.");
			return false;
		}

		LocalReason = NSLOCTEXT("UnrealEd", "LandscapeActorNotLoaded", "Landscape actor is not loaded. An edit layer landscape is required for editing.");
		return false;
	}
	
	// To edit a Landscape with layers all components need to be registered
	if(Proxy && !CurrentToolTarget.LandscapeInfo->AreAllComponentsRegistered())
	{
		LocalReason = NSLOCTEXT("UnrealEd", "LandscapeMissingComponents", "Landscape has some unregistered components (hidden levels).");
		return false;
	}

	return true;
}

void FEdModeLandscape::UpdateTargetList(bool bRegenerateThumbnails)
{
	LandscapeTargetList.Empty();
	TargetLayerStartingIndex = 0;

	if (CurrentToolTarget.LandscapeInfo.IsValid())
	{
		ALandscape* Landscape = CurrentToolTarget.LandscapeInfo->LandscapeActor.Get();
		if (Landscape != nullptr)
		{
			CachedLandscapeMaterial = Landscape->GetLandscapeMaterial();
			const int32 SelectedEditLayerIndex = GetSelectedEditLayerIndex();
			bool bFoundSelected = false;

			// Add heightmap
			LandscapeTargetList.Add(MakeShareable(new FLandscapeTargetListInfo(LOCTEXT("Heightmap", "Heightmap"), ELandscapeToolTargetType::Heightmap, CurrentToolTarget.LandscapeInfo.Get(), SelectedEditLayerIndex, /* bIsLayerReferencedByMaterial=*/ false)));

			if (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
			{
				bFoundSelected = true;
			}

			// Add visibility
			FLandscapeInfoLayerSettings VisibilitySettings(ALandscapeProxy::VisibilityLayer, Landscape);
			LandscapeTargetList.Add(MakeShareable(new FLandscapeTargetListInfo(LOCTEXT("Visibility", "Visibility"), ELandscapeToolTargetType::Visibility, VisibilitySettings, SelectedEditLayerIndex, /* bIsLayerReferencedByMaterial=*/ false)));

			if (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility)
			{
				bFoundSelected = true;
			}

			// Add layers
			TargetLayerStartingIndex = LandscapeTargetList.Num();
			TSet<FName> LayerNames;
			CurrentToolTarget.LandscapeInfo->ForEachLandscapeProxy([&](ALandscapeProxy* Proxy)
				{
					LayerNames.Append(Proxy->RetrieveTargetLayerNamesFromMaterials(/*bIncludeVisibilityLayer =*/false));
					return true;
				});

			// Don't recreate the render state of everything, only update the materials context
			FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
			for (auto It = CurrentToolTarget.LandscapeInfo->Layers.CreateIterator(); It; It++)
			{
				FLandscapeInfoLayerSettings& LayerSettings = *It;

				FName LayerName = LayerSettings.GetLayerName();

				if (LayerSettings.LayerInfoObj == ALandscapeProxy::VisibilityLayer)
				{
					// Already handled above
					continue;
				}

				if (!bFoundSelected &&
					CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap &&
					CurrentToolTarget.LayerInfo == LayerSettings.LayerInfoObj &&
					CurrentToolTarget.LayerName == LayerSettings.LayerName)
				{
					bFoundSelected = true;
				}

				// Ensure thumbnails are built 
				if ((LayerSettings.ThumbnailMIC == nullptr) || bRegenerateThumbnails)
				{
					LayerSettings.ThumbnailMIC = UE::Landscape::CreateLandscapeLayerThumbnailMIC(MaterialUpdateContext, LayerSettings.Owner.IsValid() ? LayerSettings.Owner.Get()->GetLandscapeMaterial() : nullptr, LayerName);
				}

				// Add the layer
				LandscapeTargetList.Add(MakeShareable(new FLandscapeTargetListInfo(FText::FromName(LayerName), ELandscapeToolTargetType::Weightmap, LayerSettings, SelectedEditLayerIndex, /* bIsLayerReferencedByMaterial=*/ LayerNames.Contains(LayerName))));
			}

			if (!bFoundSelected)
			{
				CurrentToolTarget.TargetType = ELandscapeToolTargetType::Invalid;
				SetCurrentTargetLayer(NAME_None, nullptr);
			}

			UISettings->TargetDisplayOrder = Landscape->TargetDisplayOrder;

			UpdateTargetLayerDisplayOrder(UISettings->TargetDisplayOrder, UISettings->bTargetDisplayOrdersAscending);
		}
	}

	TargetsListUpdated.Broadcast();

	UpdateLayerUsageInformation();
}

void FEdModeLandscape::UpdateTargetLayerDisplayOrder(ELandscapeLayerDisplayMode InTargetDisplayOrder, bool bInIsAscendingOrder)
{
	if (!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return;
	}

	ALandscape* Landscape = CurrentToolTarget.LandscapeInfo->LandscapeActor.Get();
	if (Landscape == nullptr)
	{
		return;
	}

	bool DetailPanelRefreshRequired = false;

	// Save value to landscape
	Landscape->TargetDisplayOrder = InTargetDisplayOrder;
	TArray<FName>& SavedTargetNameList = Landscape->TargetDisplayOrderList;

	switch (InTargetDisplayOrder)
	{
		case ELandscapeLayerDisplayMode::Default:
		{
			SavedTargetNameList.Empty();

			for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : LandscapeTargetList)
			{
				SavedTargetNameList.Add(TargetInfo->LayerName);
			}

			DetailPanelRefreshRequired = true;
		}
		break;

		case ELandscapeLayerDisplayMode::Alphabetical:
		{
			SavedTargetNameList.Empty();

			// Add only layers to be able to sort them by name
			for (int32 i = GetTargetLayerStartingIndex(); i < LandscapeTargetList.Num(); ++i)
			{
				SavedTargetNameList.Add(LandscapeTargetList[i]->LayerName);
			}

			SavedTargetNameList.Sort(FNameLexicalLess());

			// Then insert the non layer target that shouldn't be sorted
			for (int32 i = 0; i < GetTargetLayerStartingIndex(); ++i)
			{
				SavedTargetNameList.Insert(LandscapeTargetList[i]->LayerName, i);
			}

			DetailPanelRefreshRequired = true;
		}
		break;

		case ELandscapeLayerDisplayMode::UserSpecific:
		{
			for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : LandscapeTargetList)
			{
				bool Found = false;

				for (const FName& LayerName : SavedTargetNameList)
				{
					if (TargetInfo->LayerName == LayerName)
					{
						Found = true;
						break;
					}
				}

				if (!Found)
				{
					DetailPanelRefreshRequired = true;
					SavedTargetNameList.Add(TargetInfo->LayerName);
				}
			}

			// Handle the removing of elements from material
			for (int32 i = SavedTargetNameList.Num() - 1; i >= 0; --i)
			{
				bool Found = false;

				for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : LandscapeTargetList)
				{
					if (SavedTargetNameList[i] == TargetInfo->LayerName)
					{
						Found = true;
						break;
					}
				}

				if (!Found)
				{
					DetailPanelRefreshRequired = true;
					SavedTargetNameList.RemoveSingle(SavedTargetNameList[i]);
				}
			}
		}
		break;

		case ELandscapeLayerDisplayMode::ByBlendMethod:
		{
			SavedTargetNameList.Empty();

			TArray<TSharedRef<FLandscapeTargetListInfo>> SortedTargetList;
			// Add only layers to be able to sort them by blend type
			for (int32 i = GetTargetLayerStartingIndex(); i < LandscapeTargetList.Num(); ++i)
			{
				SortedTargetList.Add(LandscapeTargetList[i]);
			}

			SortedTargetList.Sort([](const TSharedRef<FLandscapeTargetListInfo>& InLHS, const TSharedRef<FLandscapeTargetListInfo>& InRHS)
				{
					// Put invalid layers at the end : 
					if (InLHS->LayerInfoObj == nullptr)
					{
						return (InRHS->LayerInfoObj == nullptr) ? InLHS->LayerName.LexicalLess(InRHS->LayerName) : false;
					}
					else if (InRHS->LayerInfoObj == nullptr)
					{
						return true;
					}

					if (InLHS->LayerInfoObj->GetBlendMethod() == InRHS->LayerInfoObj->GetBlendMethod())
					{
						// For a given blend method, sort by blend group 
						if (InLHS->LayerInfoObj->GetBlendGroup() != InRHS->LayerInfoObj->GetBlendGroup())
						{
							// But put empty group first (most frequent case)
							if (InLHS->LayerInfoObj->GetBlendGroup().IsNone())
							{
								return true;
							}
							else if (InRHS->LayerInfoObj->GetBlendGroup().IsNone())
							{
								return false;
							}
							
							return InLHS->LayerInfoObj->GetBlendGroup().LexicalLess(InRHS->LayerInfoObj->GetBlendGroup());
						}

						return InLHS->LayerName.LexicalLess(InRHS->LayerName);
					}
					else
					{
						static constexpr int32 BlendMethodSortOrder[static_cast<int32>(ELandscapeTargetLayerBlendMethod::Count)] = 
						{
							2, /* ELandscapeTargetLayerBlendMethod::None */
							1, /* ELandscapeTargetLayerBlendMethod::FinalWeightBlending */
							0, /* ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending */
						};

						return (BlendMethodSortOrder[static_cast<int32>(InLHS->LayerInfoObj->GetBlendMethod())] < BlendMethodSortOrder[static_cast<int32>(InRHS->LayerInfoObj->GetBlendMethod())]);
					}
				});
			Algo::Transform(SortedTargetList, SavedTargetNameList, [](const TSharedRef<FLandscapeTargetListInfo>& InTargetListInfo) { return InTargetListInfo->GetLayerName(); });

			// Then insert the non layer target that shouldn't be sorted
			for (int32 i = 0; i < GetTargetLayerStartingIndex(); ++i)
			{
				SavedTargetNameList.Insert(LandscapeTargetList[i]->LayerName, i);
			}

			DetailPanelRefreshRequired = true;
		}
		break;
	}	

	if (!bInIsAscendingOrder)
	{
		Algo::Reverse(SavedTargetNameList);
	}

	if (DetailPanelRefreshRequired)
	{
		RefreshDetailPanel();
	}
}

void FEdModeLandscape::OnLandscapeMaterialChangedDelegate(ALandscapeProxy* InProxyChanged, const FOnLandscapeProxyMaterialChangedParams& InParams)
{
	// call UpdateTargetList next frame to avoid being called by all proxies
	bNeedsUpdateTargetLayerList = true;
}

void FEdModeLandscape::RequestUpdateLayerUsageInformation()
{
	bNeedsUpdateLayerUsageInformation = true;
}

void FEdModeLandscape::UpdateLayerUsageInformation(TWeakObjectPtr<ULandscapeLayerInfoObject>* LayerInfoObjectThatChanged)
{
	bNeedsUpdateLayerUsageInformation = false;

	if (!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return;
	}

	bool DetailPanelRefreshRequired = false;
	
	TArray<TWeakObjectPtr<ULandscapeLayerInfoObject>> LayerInfoObjectToProcess;
	const TArray<TSharedRef<FLandscapeTargetListInfo>>& TargetList = GetTargetList();

	if (LayerInfoObjectThatChanged != nullptr)
	{
		if ((*LayerInfoObjectThatChanged).IsValid())
		{
			LayerInfoObjectToProcess.Add(*LayerInfoObjectThatChanged);
		}
	}
	else
	{
		LayerInfoObjectToProcess.Reserve(TargetList.Num());

		for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : TargetList)
		{
			if (!TargetInfo->LayerInfoObj.IsValid() || TargetInfo->TargetType != ELandscapeToolTargetType::Weightmap)
			{
				continue;
			}

			LayerInfoObjectToProcess.Add(TargetInfo->LayerInfoObj);
		}
	}
	
	TArray<ULandscapeLayerInfoObject*> UsedLayerInfos;
	CurrentToolTarget.LandscapeInfo->GetUsedPaintLayers(FGuid(), UsedLayerInfos);
		
	for (const TWeakObjectPtr<ULandscapeLayerInfoObject>& LayerInfoObj : LayerInfoObjectToProcess)
	{
		if (ULandscapeLayerInfoObject* LayerInfo = LayerInfoObj.Get())
		{
			bool bUsed = UsedLayerInfos.Contains(LayerInfo);
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (LayerInfo->IsReferencedFromLoadedData != bUsed)
			{
				LayerInfo->IsReferencedFromLoadedData = bUsed;
				DetailPanelRefreshRequired = true;
			}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	
	if (DetailPanelRefreshRequired)
	{
		RefreshDetailPanel();
	}
}

bool FEdModeLandscape::ShouldShowLayer(TSharedRef<FLandscapeTargetListInfo> Target) const
{
	if (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
	{
		return (Target->TargetType == ELandscapeToolTargetType::Heightmap);
	}
	else if (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility)
	{
		return (Target->TargetType == ELandscapeToolTargetType::Visibility);
	}

	// Weightmap case : 
	if (Target->TargetType != ELandscapeToolTargetType::Weightmap)
	{
		return false;
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!UISettings->ShowUnusedLayers && (!Target->LayerInfoObj.IsValid() || !Target->LayerInfoObj.Get()->IsReferencedFromLoadedData))
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return false;
	}

	// check each string in the filter strings list against our layer name
	if (!UISettings->TargetLayersFilterString.IsEmpty())
	{
		// Build a list of strings that must be matched
		TArray<FString> FilterStrings;

		FString FilterString = UISettings->TargetLayersFilterString;
		FilterString.TrimStartAndEndInline();
		FilterString.ParseIntoArray(FilterStrings, TEXT(" "), true /*bCullEmpty*/);

		const FString LayerName = Target->GetLayerName().ToString();
		for (const FString& String : FilterStrings)
		{
			if (!LayerName.Contains(String))
			{
				return false;
			}
		}
	}

	return true;
}

int32 FEdModeLandscape::GetTargetLayerStartingIndex() const
{
	return TargetLayerStartingIndex;
}

const TArray<FName>* FEdModeLandscape::GetTargetDisplayOrderList() const
{
	if (!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return nullptr;
	}

	ALandscape* Landscape = CurrentToolTarget.LandscapeInfo->LandscapeActor.Get();
	if (Landscape == nullptr)
	{
		return nullptr;
	}

	return &Landscape->TargetDisplayOrderList;
}

void FEdModeLandscape::MoveTargetLayerDisplayOrder(int32 IndexToMove, int32 IndexToDestination)
{
	if (!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return;
	}

	ALandscape* Landscape = CurrentToolTarget.LandscapeInfo->LandscapeActor.Get();
	if (Landscape == nullptr)
	{
		return;
	}
	
	FName Data = Landscape->TargetDisplayOrderList[IndexToMove];
	Landscape->TargetDisplayOrderList.RemoveAt(IndexToMove);
	Landscape->TargetDisplayOrderList.Insert(Data, IndexToDestination);

	Landscape->TargetDisplayOrder = ELandscapeLayerDisplayMode::UserSpecific;
	UISettings->TargetDisplayOrder = ELandscapeLayerDisplayMode::UserSpecific;

	RefreshDetailPanel();
}

FEdModeLandscape::FTargetsListUpdated FEdModeLandscape::TargetsListUpdated;

void FEdModeLandscape::HandleLevelsChanged()
{
	UpdateLandscapeList();
	UpdateTargetList();
	UpdateBrushList();

	// if the Landscape is deleted then close the landscape editor
	const bool bHadLandscape = (NewLandscapePreviewMode == ENewLandscapePreviewMode::None);
	if (bHadLandscape && CurrentToolTarget.LandscapeInfo == nullptr)
	{
		RequestDeletion();
	}

	// if a landscape is added somehow then switch to sculpt
	if (!bHadLandscape && CanEditCurrentTarget())
	{
		SetCurrentTool("Select");
		SetCurrentTool("Sculpt");
	}
}

/** FEdMode: Render the mesh paint tool */
void FEdModeLandscape::Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI)
{
	/** Call parent implementation */
	FEdMode::Render(View, Viewport, PDI);

	if (!IsEditingEnabled())
	{
		return;
	}

	// Override Rendering for Splines Tool
	if (CurrentTool)
	{
		CurrentTool->Render(View, Viewport, PDI);
	}
}

/** FEdMode: Render HUD elements for this tool */
void FEdModeLandscape::DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas)
{
}

bool FEdModeLandscape::UsesTransformWidget() const
{
	if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
	{
		return true;
	}

	// Override Widget for Splines Tool
	if (CurrentTool && CurrentTool->UsesTransformWidget())
	{
		return true;
	}

	return (UISettings->CurrentGizmoActor.IsValid() && UISettings->CurrentGizmoActor->IsSelected() && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo));
}

bool FEdModeLandscape::ShouldDrawWidget() const
{
	return UsesTransformWidget();
}

EAxisList::Type FEdModeLandscape::GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const
{
	if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None)
	{
		// Override Widget for Splines Tool
		if (CurrentTool)
		{
			return CurrentTool->GetWidgetAxisToDraw(InWidgetMode);
		}
	}

	switch (InWidgetMode)
	{
	case UE::Widget::WM_Translate:
		return EAxisList::XYZ;
	case UE::Widget::WM_Rotate:
		return EAxisList::Z;
	case UE::Widget::WM_Scale:
		return EAxisList::XYZ;
	default:
		return EAxisList::None;
	}
}

FVector FEdModeLandscape::GetWidgetLocation() const
{
	if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
	{
		return UISettings->NewLandscape_Location;
	}

	if (UISettings->CurrentGizmoActor.IsValid() && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo) && UISettings->CurrentGizmoActor->IsSelected())
	{
		if (UISettings->CurrentGizmoActor->TargetLandscapeInfo.IsValid() && UISettings->CurrentGizmoActor->TargetLandscapeInfo->GetLandscapeProxy())
		{
			// Apply Landscape transformation when it is available
			ULandscapeInfo* LandscapeInfo = UISettings->CurrentGizmoActor->TargetLandscapeInfo.Get();
			return UISettings->CurrentGizmoActor->GetActorLocation()
				+ FQuatRotationMatrix(LandscapeInfo->GetLandscapeProxy()->GetActorQuat()).TransformPosition(FVector(0, 0, UISettings->CurrentGizmoActor->GetLength()));
		}
		return UISettings->CurrentGizmoActor->GetActorLocation();
	}

	// Override Widget for Splines Tool
	if (CurrentTool && CurrentTool->OverrideWidgetLocation())
	{
		return CurrentTool->GetWidgetLocation();
	}

	return FEdMode::GetWidgetLocation();
}

bool FEdModeLandscape::GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
	{
		InMatrix = FRotationMatrix(UISettings->NewLandscape_Rotation);
		return true;
	}

	// Override Widget for Splines Tool
	if (CurrentTool && CurrentTool->OverrideWidgetRotation())
	{
		InMatrix = CurrentTool->GetWidgetRotation();
		return true;
	}

	return false;
}

bool FEdModeLandscape::GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData)
{
	return GetCustomDrawingCoordinateSystem(InMatrix, InData);
}

/** FEdMode: Handling SelectActor */
bool FEdModeLandscape::Select(AActor* InActor, bool bInSelected)
{
	if (!IsEditingEnabled())
	{
		return false;
	}

	if (InActor->IsA<ALandscapeProxy>() && bInSelected)
	{
		ALandscapeProxy* Landscape = CastChecked<ALandscapeProxy>(InActor);
		ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
		if (LandscapeInfo && LandscapeInfo->SupportsLandscapeEditing())
		{
			if (CurrentToolTarget.LandscapeInfo != LandscapeInfo)
			{
				SetLandscapeInfo(LandscapeInfo);

				// If we were in "New Landscape" mode and we select a landscape then switch to editing mode
				if (NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
				{
					SetCurrentTool("Sculpt");
				}
			}
		}
	}

	if (IsSelectionAllowed(InActor, bInSelected))
	{
		// false means "we haven't handled the selection", which allows the editor to perform the selection
		// so false means "allow"
		return false;
	}

	// true means "we have handled the selection", which effectively blocks the selection from happening
	// so true means "block"
	return true;
}

/** FEdMode: Check to see if an actor can be selected in this mode - no side effects */
bool FEdModeLandscape::IsSelectionAllowed(AActor* InActor, bool bInSelection) const
{
	if (!bInSelection)
	{
		// always allow de-selection
		return true;
	}

	if (!IsEditingEnabled())
	{
		return false;
	}

	// Override Selection for Splines Tool
	if (CurrentTool && CurrentTool->OverrideSelection())
	{
		return CurrentTool->IsSelectionAllowed(InActor, bInSelection);
	}

	if (InActor->IsA(ALandscapeProxy::StaticClass()))
	{
		return true;
	}
	else if (InActor->IsA(ALandscapeGizmoActor::StaticClass()))
	{
		return true;
	}
	else if (InActor->IsA(ALandscapeBlueprintBrushBase::StaticClass()))
	{
		return true;
	}

	return false;
}

/** FEdMode: Called when the currently selected actor has changed */
void FEdModeLandscape::ActorSelectionChangeNotify()
{
	if (UISettings->CurrentGizmoActor.IsValid() && UISettings->CurrentGizmoActor->IsSelected() && (GEditor->GetSelectedActors()->CountSelections<AActor>() != 1))
	{
		GEditor->SelectNone(false, true);
		GEditor->SelectActor(UISettings->CurrentGizmoActor.Get(), true, false, true);
	}
}

void FEdModeLandscape::ActorMoveNotify()
{
	//GUnrealEd->UpdateFloatingPropertyWindows();
}

void FEdModeLandscape::PostUndo()
{
	UpdateLandscapeList();
	UpdateTargetList();
	UpdateBrushList();
}

/** Forces all level editor viewports to realtime mode */
void FEdModeLandscape::ForceRealTimeViewports(const bool bEnable)
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				const FText SystemDisplayName = LOCTEXT("RealtimeOverrideMessage_Landscape", "Landscape Mode");
				if (bEnable)
				{
					Viewport.AddRealtimeOverride(bEnable, SystemDisplayName);
				}
				else
				{
					Viewport.RemoveRealtimeOverride(SystemDisplayName, false);
				}
			}
		}
	}
}

void FEdModeLandscape::ReimportData(const FLandscapeTargetListInfo& TargetInfo)
{
	const FString& SourceFilePath = TargetInfo.GetReimportFilePath();
	if (SourceFilePath.Len())
	{
		FScopedSetLandscapeEditingLayer Scope(GetLandscape(), GetCurrentLayerGuid(), [&] { RequestLayersContentUpdateForceAll(); });
		ImportData(TargetInfo, SourceFilePath);
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "LandscapeReImport_BadFileName", "Reimport Source Filename is invalid"));
	}
}

template<class T>
void ImportDataInternal(ULandscapeInfo* LandscapeInfo, const FString& Filename, bool bSingleFile, bool bFlipYAxis, const FIntRect& ImportRegionVerts, ELandscapeImportTransformType TransformType, FIntPoint Offset, TFunctionRef<void(int32, int32, int32, int32, const TArray<T>&)> SetDataFunc)
{
	if (!LandscapeInfo)
	{
		return;
	}

	FLandscapeTiledImage TiledImage;
	TiledImage.Load<T>(*Filename);
	FIntPoint ImportResolution = TiledImage.GetResolution();

	bool bTransformInputData = false;

	if ((ImportResolution.X != ImportRegionVerts.Width() || ImportResolution.Y != ImportRegionVerts.Height()) && TransformType != ELandscapeImportTransformType::Subregion)
	{
		bTransformInputData = true;

		FFormatNamedArguments Args;
		Args.Add(TEXT("ImportSizeX"), ImportResolution.X);
		Args.Add(TEXT("ImportSizeY"), ImportResolution.Y);
		Args.Add(TEXT("LandscapeSizeX"), ImportRegionVerts.Width());
		Args.Add(TEXT("LandscapeSizeY"), ImportRegionVerts.Height());

		auto Result = FMessageDialog::Open(EAppMsgType::OkCancel,
			FText::Format(NSLOCTEXT("LandscapeEditor.Import", "Import_SizeMismatch", "The import size ({ImportSizeX}\u00D7{ImportSizeY}) does not match the current Landscape extent ({LandscapeSizeX}\u00D7{LandscapeSizeY}), if you continue it will be padded/clipped to fit"), Args));

		if (Result != EAppReturnType::Ok)
		{
			return;
		}
	}

	bool bReadUnmodified = bTransformInputData ||
		TransformType == ELandscapeImportTransformType::None ||
		TransformType == ELandscapeImportTransformType::Subregion;

	FIntRect DestRegionVerts = ImportRegionVerts;

	// None/Original mode doesn't handle resolution mismatch at all.  Tailor the destination region using the offset and image resolution.
	if (TransformType == ELandscapeImportTransformType::None)
	{
		DestRegionVerts.Min = Offset;
		DestRegionVerts.Max = Offset + ImportResolution;
	}
	else if (TransformType == ELandscapeImportTransformType::Subregion)
	{
		// Subregion mode depends on TransformImportData to re-align data to the offset even if it starts at the correct resolution.
		bTransformInputData = true;
	}

	TArray<T> ImportData;
	if (bReadUnmodified)
	{
		TiledImage.Read<T>(ImportData, bFlipYAxis);
	}
	else
	{
		const FIntRect RegionToLoad(0, 0, DestRegionVerts.Width(), DestRegionVerts.Height());
		TiledImage.ReadRegion<T>(RegionToLoad, ImportData, bFlipYAxis);
	}

	{
		TArray<T> FinalData;
		if (bTransformInputData)
		{
			FLandscapeImportHelper::TransformImportData<T>(ImportData, FinalData, FLandscapeImportResolution(ImportResolution.X, ImportResolution.Y), FLandscapeImportResolution(ImportRegionVerts.Width(), ImportRegionVerts.Height()), TransformType, Offset);
		}
		else
		{
			FinalData = MoveTemp(ImportData);
		}

		// SetDataFunc (FHeightmapAccessor::SetData / TAlphamapAccessor::SetData) expects the data buffer to exactly match the region dimensions.  Important
		// for memory access safety and stride.
		check(DestRegionVerts.Area() == FinalData.Num());

		// Set Data is in Quads (so remove 1)
		SetDataFunc(DestRegionVerts.Min.X, DestRegionVerts.Min.Y, DestRegionVerts.Max.X - 1, DestRegionVerts.Max.Y - 1, FinalData);
	}
}

void FEdModeLandscape::ImportHeightData(ULandscapeInfo* LandscapeInfo, const FGuid& LayerGuid, const FString& Filename, const FIntRect& ImportRegionVerts, ELandscapeImportTransformType TransformType, FIntPoint Offset, const ELandscapeLayerPaintingRestriction& PaintRestriction, bool bFlipYAxis)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FEdModeLandscape::ImportHeightData);

	ImportDataInternal<uint16>(LandscapeInfo, Filename, UseSingleFileImport(), bFlipYAxis, ImportRegionVerts, TransformType, Offset, [LandscapeInfo, LayerGuid, PaintRestriction](int32 MinX, int32 MinY, int32 MaxX, int32 MaxY, const TArray<uint16>& Data)
	{
		ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
		FScopedSetLandscapeEditingLayer Scope(Landscape, LayerGuid, [&] { check(Landscape); Landscape->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_Heightmap_All); });

		FScopedTransaction Transaction(LOCTEXT("Undo_ImportHeightmap", "Importing Landscape Heightmap"));

		FHeightmapAccessor<false> HeightmapAccessor(LandscapeInfo);
		HeightmapAccessor.SetData(MinX, MinY, MaxX, MaxY, Data.GetData());
	});
}

void FEdModeLandscape::ImportWeightData(ULandscapeInfo* LandscapeInfo, const FGuid& LayerGuid, ULandscapeLayerInfoObject* LayerInfo, const FString& Filename, const FIntRect& ImportRegionVerts, ELandscapeImportTransformType TransformType, FIntPoint Offset, const ELandscapeLayerPaintingRestriction& PaintRestriction, bool bFlipYAxis)
{
	if (LayerInfo)
	{
		ImportDataInternal<uint8>(LandscapeInfo, Filename, UseSingleFileImport(), bFlipYAxis, ImportRegionVerts, TransformType, Offset, [LandscapeInfo, LayerGuid, LayerInfo, PaintRestriction](int32 MinX, int32 MinY, int32 MaxX, int32 MaxY, const TArray<uint8>& Data)
		{
			ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
			FScopedSetLandscapeEditingLayer Scope(Landscape, LayerGuid, [&] { check(Landscape); Landscape->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_Weightmap_All); });

			FScopedTransaction Transaction(LOCTEXT("Undo_ImportWeightmap", "Importing Landscape Layer"));

			TAlphamapAccessor</*bInUseInterp = */false> AlphamapAccessor(LandscapeInfo, LayerInfo);
			AlphamapAccessor.SetData(MinX, MinY, MaxX, MaxY, Data.GetData(), PaintRestriction);
		});
	}
}

void FEdModeLandscape::ImportData(const FLandscapeTargetListInfo& TargetInfo, const FString& Filename)
{
	ULandscapeInfo* LandscapeInfo = TargetInfo.LandscapeInfo.Get();
	FIntRect ImportExtent;
	if (LandscapeInfo && LandscapeInfo->GetLandscapeExtent(ImportExtent))
	{
		// Import is in Verts (Extent is in Quads)
		ImportExtent.Max.X += 1;
		ImportExtent.Max.Y += 1;

		if (TargetInfo.TargetType == ELandscapeToolTargetType::Heightmap)
		{
			FScopedSlowTask Progress(1, LOCTEXT("ImportingLandscapeHeightmapTask", "Importing Landscape Heightmap..."));
			Progress.MakeDialog();
			ImportHeightData(LandscapeInfo, GetCurrentLayerGuid(), Filename, ImportExtent, ELandscapeImportTransformType::ExpandCentered);
		}
		else
		{
			FScopedSlowTask Progress(1, LOCTEXT("ImportingLandscapeWeightmapTask", "Importing Landscape Layer Weightmap..."));
			Progress.MakeDialog();
			ImportWeightData(LandscapeInfo, GetCurrentLayerGuid(), TargetInfo.LayerInfoObj.Get(), Filename, ImportExtent, ELandscapeImportTransformType::ExpandCentered);
		}
	}
}

void FEdModeLandscape::DeleteLandscapeComponents(ULandscapeInfo* LandscapeInfo, TSet<ULandscapeComponent*> ComponentsToDelete)
{
	LandscapeInfo->Modify();
	ALandscapeProxy* MainLandscapeProxy = LandscapeInfo->GetLandscapeProxy();
	check(MainLandscapeProxy != nullptr);

	TSet<ALandscapeProxy*> AffectedProxies;
	for (ULandscapeComponent* Component : ComponentsToDelete)
	{
		if (ALandscapeProxy* Proxy = Component->GetLandscapeProxy())
		{
			AffectedProxies.Add(Proxy);
		}
		Component->Modify();
		ULandscapeHeightfieldCollisionComponent* CollisionComp = Component->GetCollisionComponent();
		if (CollisionComp)
		{
			CollisionComp->Modify();
		}
	}

	int32 ComponentSizeVerts = LandscapeInfo->ComponentNumSubsections * (LandscapeInfo->SubsectionSizeQuads + 1);
	int32 NeedHeightmapSize = 1 << FMath::CeilLogTwo(ComponentSizeVerts);

	TSet<ULandscapeComponent*> HeightmapUpdateComponents;
	// Need to split all the component which share Heightmap with selected components
	// Search neighbor only
	for (ULandscapeComponent* Component : ComponentsToDelete)
	{
		int32 SearchX = Component->GetHeightmap()->Source.GetSizeX() / NeedHeightmapSize;
		int32 SearchY = Component->GetHeightmap()->Source.GetSizeY() / NeedHeightmapSize;
		FIntPoint ComponentBase = Component->GetSectionBase() / Component->ComponentSizeQuads;

		for (int32 Y = 0; Y < SearchY; ++Y)
		{
			for (int32 X = 0; X < SearchX; ++X)
			{
				// Search for four directions...
				for (int32 Dir = 0; Dir < 4; ++Dir)
				{
					int32 XDir = (Dir >> 1) ? 1 : -1;
					int32 YDir = (Dir % 2) ? 1 : -1;
					ULandscapeComponent* Neighbor = LandscapeInfo->XYtoComponentMap.FindRef(ComponentBase + FIntPoint(XDir*X, YDir*Y));
					if (Neighbor && Neighbor->GetHeightmap() == Component->GetHeightmap() && !HeightmapUpdateComponents.Contains(Neighbor))
					{
						Neighbor->Modify();
						HeightmapUpdateComponents.Add(Neighbor);
					}
				}
			}
		}
	}

	TArray<FComponentRecreateRenderStateContext> RecreateRenderStateContexts;

	{
		TArray<UActorComponent*> ComponentsToReregister(HeightmapUpdateComponents.Array());
		FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);

		// Changing Heightmap format for selected components
		for (ULandscapeComponent* Component : HeightmapUpdateComponents)
		{
			ALandscape::SplitHeightmap(Component, nullptr, &MaterialUpdateContext, &RecreateRenderStateContexts, false);
		}

		FMultiComponentReregisterContext RegisterContext(ComponentsToReregister);
	}

	RecreateRenderStateContexts.Empty();

	// Remove attached foliage
	for (ULandscapeComponent* Component : ComponentsToDelete)
	{
		ULandscapeHeightfieldCollisionComponent* CollisionComp = Component->GetCollisionComponent();
		if (CollisionComp)
		{
			AInstancedFoliageActor::DeleteInstancesForComponent(MainLandscapeProxy->GetWorld(), CollisionComp);
		}
	}

	TArray<UActorComponent*> NeighborsComponentToReregister;

	// Check which ones are need for height map change
	for (ULandscapeComponent* Component : ComponentsToDelete)
	{
		// Reset neighbors LOD information
		FIntPoint ComponentBase = Component->GetSectionBase() / Component->ComponentSizeQuads;
		FIntPoint NeighborKeys[8] =
		{
			ComponentBase + FIntPoint(-1, -1),
			ComponentBase + FIntPoint(+0, -1),
			ComponentBase + FIntPoint(+1, -1),
			ComponentBase + FIntPoint(-1, +0),
			ComponentBase + FIntPoint(+1, +0),
			ComponentBase + FIntPoint(-1, +1),
			ComponentBase + FIntPoint(+0, +1),
			ComponentBase + FIntPoint(+1, +1)
		};

		for (const FIntPoint& NeighborKey : NeighborKeys)
		{
			ULandscapeComponent* NeighborComp = LandscapeInfo->XYtoComponentMap.FindRef(NeighborKey);
			if (NeighborComp && !ComponentsToDelete.Contains(NeighborComp))
			{
				NeighborComp->Modify();
				NeighborComp->InvalidateLightingCache();

				NeighborsComponentToReregister.AddUnique(NeighborComp);
			}
		}

		// Remove Selected Region in deleted Component
		for (int32 Y = 0; Y < Component->ComponentSizeQuads; ++Y)
		{
			for (int32 X = 0; X < Component->ComponentSizeQuads; ++X)
			{
				LandscapeInfo->SelectedRegion.Remove(FIntPoint(X, Y) + Component->GetSectionBase());
			}
		}

		{
			TSet<UTexture2D*> OldHeightmapTextures;
			OldHeightmapTextures.Add(Component->GetHeightmap(FGuid()));
			// Also process all edit layers heightmaps :
			Component->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
			{
				OldHeightmapTextures.Add(Component->GetHeightmap(LayerGuid));
			});

			for (UTexture2D* HeightmapTexture : OldHeightmapTextures)
			{
				check(HeightmapTexture != nullptr);
				HeightmapTexture->SetFlags(RF_Transactional);
				HeightmapTexture->Modify();
				HeightmapTexture->MarkPackageDirty();
				HeightmapTexture->ClearFlags(RF_Standalone); // Remove when there is no reference for this Heightmap...
			}
		}

		{
			TArray<UTexture2D*> OldWeightmapTextures = Component->GetWeightmapTextures(FGuid());
			// Also process all edit layers weightmaps :
			Component->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
			{
				OldWeightmapTextures.Append(Component->GetWeightmapTextures(LayerGuid));
			});

			for (UTexture2D* WeightmapTexture : OldWeightmapTextures)
			{
				check(WeightmapTexture != nullptr);
				WeightmapTexture->SetFlags(RF_Transactional);
				WeightmapTexture->Modify();
				WeightmapTexture->MarkPackageDirty();
				WeightmapTexture->ClearFlags(RF_Standalone);
			}
		}

		{
			// The shared weightmap usages also need to be cleaned up, otherwise, one might still think these textures apply to deleted components :
			TArray<ULandscapeWeightmapUsage*> ComponentWeightmapTexturesUsage = Component->GetWeightmapTexturesUsage(FGuid());
			// Also process all edit layers weightmap textures usages :
			Component->ForEachLayer([&](const FGuid& LayerGuid, FLandscapeLayerComponentData& LayerData)
			{
				ComponentWeightmapTexturesUsage.Append(Component->GetWeightmapTexturesUsage(LayerGuid));
			});

			for (ULandscapeWeightmapUsage* Usage : ComponentWeightmapTexturesUsage)
			{
				// Make sure these usages don't reference this component anymore :
				Usage->ClearUsage(Component);
			}
		}

		ULandscapeHeightfieldCollisionComponent* CollisionComp = Component->GetCollisionComponent();
		if (CollisionComp)
		{
			CollisionComp->DestroyComponent();
		}
		Component->DestroyComponent();

		// The deletion of components will affect the Nanite landscape components, which therefore need to be ticked so that they can appropriately update if needed 
		for (ALandscapeProxy* AffectedProxy : AffectedProxies)
		{
			AffectedProxy->UpdateRenderingMethod();
		}
	}

	{
		FMultiComponentReregisterContext RegisterContext(NeighborsComponentToReregister);
	}

	// Remove Selection
	LandscapeInfo->ClearSelectedRegion(true);
	//EdMode->SetMaskEnable(Landscape->SelectedRegion.Num());
	GEngine->BroadcastLevelActorListChanged();
}

ALandscape* FEdModeLandscape::ChangeComponentSetting(int32 NumComponentsX, int32 NumComponentsY, int32 NumSubsections, int32 SubsectionSizeQuads, bool bResample)
{
	FScopedSlowTask Progress(3, LOCTEXT("LandscapeChangeComponentSetting", "Changing Landscape Component Settings..."));
	Progress.MakeDialog();
	int32 CurrentTaskProgress = 0;

	check(NumComponentsX > 0);
	check(NumComponentsY > 0);
	check(NumSubsections > 0);
	check(SubsectionSizeQuads > 0);

	const int32 NewComponentSizeQuads = NumSubsections * SubsectionSizeQuads;

	ALandscape* NewLandscape = nullptr;

	ULandscapeInfo* LandscapeInfo = CurrentToolTarget.LandscapeInfo.Get();
	if (ensure(LandscapeInfo != nullptr))
	{
		int32 OldMinX, OldMinY, OldMaxX, OldMaxY;
		if (LandscapeInfo->GetLandscapeExtent(OldMinX, OldMinY, OldMaxX, OldMaxY))
		{
			ALandscape* OldLandscape = LandscapeInfo->LandscapeActor.Get();
			check(OldLandscape != nullptr);

			const int32 OldVertsX = OldMaxX - OldMinX + 1;
			const int32 OldVertsY = OldMaxY - OldMinY + 1;
			const int32 NewVertsX = NumComponentsX * NewComponentSizeQuads + 1;
			const int32 NewVertsY = NumComponentsY * NewComponentSizeQuads + 1;

			TMap<FGuid, TArray<uint16>> HeightDataPerLayers;			
			TMap<FGuid, TArray<FLandscapeImportLayerInfo>> ImportMaterialLayerInfosPerLayers;

			FVector LandscapeOffset = FVector::ZeroVector;
			FIntPoint LandscapeOffsetQuads = FIntPoint::ZeroValue;
			float LandscapeScaleFactor = bResample ? (float)OldLandscape->ComponentSizeQuads / NewComponentSizeQuads : 1.0f;

			int32 NewMinX, NewMinY, NewMaxX, NewMaxY;

			{ // Scope to flush the texture update before doing the import
				FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);

				if (bResample)
				{
					NewMinX = OldMinX / LandscapeInfo->ComponentSizeQuads * NewComponentSizeQuads;
					NewMinY = OldMinY / LandscapeInfo->ComponentSizeQuads * NewComponentSizeQuads;
					NewMaxX = NewMinX + NewVertsX - 1;
					NewMaxY = NewMinY + NewVertsY - 1;
				}
				else
				{
					NewMinX = OldMinX + (OldVertsX - NewVertsX) / 2;
					NewMinY = OldMinY + (OldVertsY - NewVertsY) / 2;
					NewMaxX = NewMinX + NewVertsX - 1;
					NewMaxY = NewMinY + NewVertsY - 1;

					// offset landscape to component boundary
					LandscapeOffset = OldLandscape->GetActorTransform().TransformVector(FVector(NewMinX, NewMinY, 0));
					LandscapeOffsetQuads = FIntPoint(NewMinX, NewMinY);
				}

				auto ExtractHeightmapWeightmapContent = [&](TArray<uint16>& OutHeightData, TArray<FLandscapeImportLayerInfo>& OutImportMaterialLayerInfos)
				{
					if (bResample)
					{
						OutHeightData.AddZeroed(OldVertsX * OldVertsY);

						// GetHeightData alters its args, so make temp copies to avoid screwing things up
						int32 TMinX = OldMinX, TMinY = OldMinY, TMaxX = OldMaxX, TMaxY = OldMaxY;
						LandscapeEdit.GetHeightData(TMinX, TMinY, TMaxX, TMaxY, OutHeightData.GetData(), 0);

						TArray<uint16> ResampledHeightData;

						FIntRect SrcRegion(0, 0, OldVertsX - 1, OldVertsY - 1);
						FIntRect DestRegion(0, 0, NewVertsX - 1, NewVertsY - 1);
						FLandscapeConfigHelper::ResampleData(OutHeightData, ResampledHeightData, SrcRegion, DestRegion);
						OutHeightData = MoveTemp(ResampledHeightData);

						TArray<uint8> ResampledWeightData;
						for (const FLandscapeInfoLayerSettings& LayerSettings : LandscapeInfo->Layers)
						{
							if (LayerSettings.LayerInfoObj != nullptr)
							{
								auto ImportLayerInfo = new(OutImportMaterialLayerInfos) FLandscapeImportLayerInfo(LayerSettings);
								ImportLayerInfo->LayerData.AddZeroed(OldVertsX * OldVertsY);

								TMinX = OldMinX; TMinY = OldMinY; TMaxX = OldMaxX; TMaxY = OldMaxY;
								LandscapeEdit.GetWeightData(LayerSettings.LayerInfoObj, TMinX, TMinY, TMaxX, TMaxY, ImportLayerInfo->LayerData.GetData(), 0);

								FLandscapeConfigHelper::ResampleData(ImportLayerInfo->LayerData, ResampledWeightData, SrcRegion, DestRegion);
								ImportLayerInfo->LayerData = MoveTemp(ResampledWeightData);
							}
						}
					}
					else
					{
						const int32 RequestedMinX = FMath::Max(OldMinX, NewMinX);
						const int32 RequestedMinY = FMath::Max(OldMinY, NewMinY);
						const int32 RequestedMaxX = FMath::Min(OldMaxX, NewMaxX);
						const int32 RequestedMaxY = FMath::Min(OldMaxY, NewMaxY);

						const int32 RequestedVertsX = RequestedMaxX - RequestedMinX + 1;
						const int32 RequestedVertsY = RequestedMaxY - RequestedMinY + 1;

						OutHeightData.AddZeroed(RequestedVertsX * RequestedVertsY * sizeof(uint16));

						// GetHeightData alters its args, so make temp copies to avoid screwing things up
						int32 TMinX = RequestedMinX, TMinY = RequestedMinY, TMaxX = RequestedMaxX, TMaxY = RequestedMaxY;
						LandscapeEdit.GetHeightData(TMinX, TMinY, TMaxX, OldMaxY, OutHeightData.GetData(), 0);
						
						FIntRect SrcRegion(RequestedMinX, RequestedMinY, RequestedMaxX, RequestedMaxY);
						FIntRect DestRegion(NewMinX, NewMinY, NewMaxX, NewMaxY);
						TArray<uint16> ExpandedHeightData;
						FLandscapeConfigHelper::ExpandData(OutHeightData, ExpandedHeightData, SrcRegion, DestRegion, true);
						OutHeightData = MoveTemp(ExpandedHeightData);

						TArray<uint8> ExpandedWeightData;
						for (const FLandscapeInfoLayerSettings& LayerSettings : LandscapeInfo->Layers)
						{
							if (LayerSettings.LayerInfoObj != nullptr)
							{
								auto ImportLayerInfo = new(OutImportMaterialLayerInfos) FLandscapeImportLayerInfo(LayerSettings);
								ImportLayerInfo->LayerData.AddZeroed(NewVertsX * NewVertsY * sizeof(uint8));

								TMinX = RequestedMinX; TMinY = RequestedMinY; TMaxX = RequestedMaxX; TMaxY = RequestedMaxY;
								LandscapeEdit.GetWeightData(LayerSettings.LayerInfoObj, TMinX, TMinY, TMaxX, TMaxY, ImportLayerInfo->LayerData.GetData(), 0);

								FLandscapeConfigHelper::ExpandData(ImportLayerInfo->LayerData, ExpandedWeightData, SrcRegion, DestRegion, true);
								ImportLayerInfo->LayerData = MoveTemp(ExpandedWeightData);
							}
						}
					}
				};

				int32 HeightCount = 0;

				for (const ULandscapeEditLayerBase* OldEditLayer : OldLandscape->GetEditLayersConst())
				{
					FScopedSetLandscapeEditingLayer Scope(OldLandscape, OldEditLayer->GetGuid());

					TArray<uint16> HeightData;
					TArray<FLandscapeImportLayerInfo> ImportMaterialLayerInfos;

					ExtractHeightmapWeightmapContent(HeightData, ImportMaterialLayerInfos);

					HeightCount = FMath::Max(HeightCount, HeightData.Num());
					HeightDataPerLayers.Add(OldEditLayer->GetGuid(), MoveTemp(HeightData));
					ImportMaterialLayerInfosPerLayers.Add(OldEditLayer->GetGuid(), MoveTemp(ImportMaterialLayerInfos));
				}

				TArray<uint16> DefaultHeightData;
				DefaultHeightData.AddUninitialized(HeightCount);

				uint16 DefaultValue = LandscapeDataAccess::GetTexHeight(0.f);

				// Initialize blank heightmap data
				for (int32 i = 0; i < DefaultHeightData.Num(); i++)
				{
					DefaultHeightData[i] = DefaultValue;
				}

				HeightDataPerLayers.Add(FGuid(), DefaultHeightData);

				TArray<FLandscapeImportLayerInfo> EmptyImportLayer;
				ImportMaterialLayerInfosPerLayers.Add(FGuid(), EmptyImportLayer);
				
				if (!bResample)
				{
					NewMinX = 0;
					NewMinY = 0;
					NewMaxX = NewVertsX - 1;
					NewMaxY = NewVertsY - 1;
				}
			}

			Progress.EnterProgressFrame(static_cast<float>(CurrentTaskProgress++));

			const FVector Location = OldLandscape->GetActorLocation() + LandscapeOffset;
			FActorSpawnParameters SpawnParams;
			SpawnParams.OverrideLevel = OldLandscape->GetLevel();
			NewLandscape = OldLandscape->GetWorld()->SpawnActor<ALandscape>(Location, OldLandscape->GetActorRotation(), SpawnParams);
			NewLandscape->bAreNewLandscapeActorsSpatiallyLoaded = OldLandscape->bAreNewLandscapeActorsSpatiallyLoaded;
			NewLandscape->bIncludeGridSizeInNameForLandscapeActors = OldLandscape->bIncludeGridSizeInNameForLandscapeActors;
			NewLandscape->bUseGeneratedLandscapeSplineMeshesActors = OldLandscape->bUseGeneratedLandscapeSplineMeshesActors;

			// Copy all of the shared property settings over
			NewLandscape->CopySharedProperties(OldLandscape);

			// Double check things were copied
			check(NewLandscape->GetPerLODOverrideMaterials().Num() == OldLandscape->GetPerLODOverrideMaterials().Num());
			check(NewLandscape->RuntimeVirtualTextures.Num() == OldLandscape->RuntimeVirtualTextures.Num());
			check(NewLandscape->LandscapeMaterial == OldLandscape->LandscapeMaterial);
			
			const FVector OldScale = OldLandscape->GetActorScale();
			NewLandscape->SetActorRelativeScale3D(FVector(OldScale.X * LandscapeScaleFactor, OldScale.Y * LandscapeScaleFactor, OldScale.Z));

			// LandscapeGuid is stomped by CopySharedProperties, but original guid is not -- fix the mismatch or it will complain during Import
 			NewLandscape->SetLandscapeGuid(FGuid(), /* bValidateGuid= */ false);

			TArrayView<const FLandscapeLayer> LandscapeLayers;
			LandscapeLayers = OldLandscape->GetLayersConst();
			NewLandscape->Import(FGuid::NewGuid(), NewMinX, NewMinY, NewMaxX, NewMaxY, NumSubsections, SubsectionSizeQuads, HeightDataPerLayers, *OldLandscape->ReimportHeightmapFilePath, ImportMaterialLayerInfosPerLayers, ELandscapeImportAlphamapType::Additive, LandscapeLayers);

			ULandscapeInfo* NewLandscapeInfo = NewLandscape->GetLandscapeInfo();
			check(NewLandscapeInfo);

			// Clone landscape splines
			ALandscape* OldLandscapeActor = LandscapeInfo->LandscapeActor.Get();
			if (OldLandscapeActor != nullptr && OldLandscapeActor->GetSplinesComponent() != nullptr)
			{
				ULandscapeSplinesComponent* OldSplines = OldLandscapeActor->GetSplinesComponent();
				ULandscapeSplinesComponent* NewSplines = DuplicateObject<ULandscapeSplinesComponent>(OldSplines, NewLandscape, OldSplines->GetFName());
				// It's possible that the duplication of the ULandscapeSplinesComponent object actually triggered the creation of a new ULandscapeSplinesComponent already, in which case it's just simpler to use it: 
				if (ULandscapeSplinesComponent* AlreadySetSplines = NewLandscape->GetSplinesComponent())
				{
					// This allows SetSplinesComponent() later on not to complain about setting a different ULandscapeSplinesComponent in the landscape :
					NewSplines = AlreadySetSplines;
				}
				NewSplines->AttachToComponent(NewLandscape->GetRootComponent(), FAttachmentTransformRules::KeepWorldTransform);

				const FVector OldSplineScale = OldSplines->GetRelativeTransform().GetScale3D();
				NewSplines->SetRelativeScale3D(FVector(OldSplineScale.X / LandscapeScaleFactor, OldSplineScale.Y / LandscapeScaleFactor, OldSplineScale.Z));
				NewLandscape->SetSplinesComponent(NewSplines);
				NewSplines->RegisterComponent();

				// TODO: Foliage on spline meshes
			}

			Progress.EnterProgressFrame(static_cast<float>(CurrentTaskProgress++));

			if (bResample)
			{
				// Remap foliage to the resampled components
				for (const TPair<FIntPoint, ULandscapeComponent*>& Entry : LandscapeInfo->XYtoComponentMap)
				{
					ULandscapeComponent* NewComponent = NewLandscapeInfo->XYtoComponentMap.FindRef(Entry.Key);
					if (NewComponent)
					{
						ULandscapeHeightfieldCollisionComponent* OldCollisionComponent = Entry.Value->GetCollisionComponent();
						ULandscapeHeightfieldCollisionComponent* NewCollisionComponent = NewComponent->GetCollisionComponent();

						if (OldCollisionComponent && NewCollisionComponent)
						{
							AInstancedFoliageActor::MoveInstancesToNewComponent(OldCollisionComponent->GetWorld(), OldCollisionComponent, NewCollisionComponent);
							if (GetDefault<ULandscapeSettings>()->GetFoliageLandscapeSnappingMode() != EFoliageLandscapeSnappingMode::DontSnap)
							{
								NewCollisionComponent->SnapFoliageInstances(ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::ForceAll);
							}
						}
					}
				}

				Progress.EnterProgressFrame(static_cast<float>(CurrentTaskProgress++));

				// delete any components that were deleted in the original
				TSet<ULandscapeComponent*> ComponentsToDelete;
				for (const TPair<FIntPoint, ULandscapeComponent*>& Entry : NewLandscapeInfo->XYtoComponentMap)
				{
					if (!LandscapeInfo->XYtoComponentMap.Contains(Entry.Key))
					{
						ComponentsToDelete.Add(Entry.Value);
					}
				}
				if (ComponentsToDelete.Num() > 0)
				{
					DeleteLandscapeComponents(NewLandscapeInfo, ComponentsToDelete);
				}
			}
			else
			{
				// Move instances
				for (const TPair<FIntPoint, ULandscapeComponent*>& OldEntry : LandscapeInfo->XYtoComponentMap)
				{
					ULandscapeHeightfieldCollisionComponent* OldCollisionComponent = OldEntry.Value->GetCollisionComponent();

					if (OldCollisionComponent)
					{
						UWorld* World = OldCollisionComponent->GetWorld();

						for (const TPair<FIntPoint, ULandscapeComponent*>& NewEntry : NewLandscapeInfo->XYtoComponentMap)
						{
							ULandscapeHeightfieldCollisionComponent* NewCollisionComponent = NewEntry.Value->GetCollisionComponent();

							if (NewCollisionComponent && FBoxSphereBounds::BoxesIntersect(NewCollisionComponent->Bounds, OldCollisionComponent->Bounds))
							{
								// only transfer instances overlapping the new box in x,y
								FBox Box = NewCollisionComponent->Bounds.GetBox();
								FBox OldBox = OldCollisionComponent->Bounds.GetBox();

								// but allow just about any Z (expand old bounds by max extent)
								double Extent = OldBox.GetExtent().GetMax();
								Box.Min.Z = OldBox.Min.Z - Extent;
								Box.Max.Z = OldBox.Max.Z + Extent;

								AInstancedFoliageActor::MoveInstancesToNewComponent(World, OldCollisionComponent, Box, NewCollisionComponent);
							}
						}
					}
				}

				// Snap them to the bounds
				for (const TPair<FIntPoint, ULandscapeComponent*>& NewEntry : NewLandscapeInfo->XYtoComponentMap)
				{
					ULandscapeHeightfieldCollisionComponent* NewCollisionComponent = NewEntry.Value->GetCollisionComponent();
					if (NewCollisionComponent && (GetDefault<ULandscapeSettings>()->GetFoliageLandscapeSnappingMode() != EFoliageLandscapeSnappingMode::DontSnap))
					{
						NewCollisionComponent->SnapFoliageInstances(ULandscapeHeightfieldCollisionComponent::ESnapFoliageInstancesFlags::ForceAll);
					}
				}

				Progress.EnterProgressFrame(static_cast<float>(CurrentTaskProgress++));

				// delete any components that are in areas that were entirely deleted in the original
				TSet<ULandscapeComponent*> ComponentsToDelete;
				for (const TPair<FIntPoint, ULandscapeComponent*>& Entry : NewLandscapeInfo->XYtoComponentMap)
				{
					const int32 OldX = Entry.Key.X * NewComponentSizeQuads + LandscapeOffsetQuads.X;
					const int32 OldY = Entry.Key.Y * NewComponentSizeQuads + LandscapeOffsetQuads.Y;
					TSet<ULandscapeComponent*> OverlapComponents;
					LandscapeInfo->GetComponentsInRegion(OldX, OldY, OldX + NewComponentSizeQuads, OldY + NewComponentSizeQuads, OverlapComponents, false);
					if (OverlapComponents.Num() == 0)
					{
						ComponentsToDelete.Add(Entry.Value);
					}
				}
				if (ComponentsToDelete.Num() > 0)
				{
					DeleteLandscapeComponents(NewLandscapeInfo, ComponentsToDelete);
				}
			}

			FString OldLandscapeActorLabel = OldLandscape->GetActorLabel();
			// Delete the old Landscape and all its proxies
			for (ALandscapeStreamingProxy* Proxy : TActorRange<ALandscapeStreamingProxy>(OldLandscape->GetWorld()))
			{
				if (Proxy->GetLandscapeActor() == OldLandscapeActor)
				{
					Proxy->Destroy();
				}
			}
			OldLandscape->Destroy();

			NewLandscape->SetActorLabel(OldLandscapeActorLabel);
		}
	}

	GEditor->RedrawLevelEditingViewports();

	return NewLandscape;
}

ELandscapeEditingState FEdModeLandscape::GetEditingState() const
{
	UWorld* World = GetWorld();

	if (GEditor->bIsSimulatingInEditor)
	{
		return ELandscapeEditingState::SIEWorld;
	}
	else if (GEditor->PlayWorld != nullptr)
	{
		return ELandscapeEditingState::PIEWorld;
	}
	else if (World == nullptr)
	{
		return ELandscapeEditingState::Unknown;
	}
	else if (World->GetFeatureLevel() < ERHIFeatureLevel::SM5)
	{
		return ELandscapeEditingState::BadFeatureLevel;
	}
	else if (NewLandscapePreviewMode == ENewLandscapePreviewMode::None &&
		!CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return ELandscapeEditingState::NoLandscape;
	}

	return ELandscapeEditingState::Enabled;
}

int32 FEdModeLandscape::GetLayerCount() const
{
	ALandscape* Landscape = GetLandscape();
	return Landscape ? Landscape->GetEditLayersConst().Num() : 0;
}

void FEdModeLandscape::SetSelectedEditLayer(int32 InLayerIndex)
{
	if (ALandscape* Landscape = GetLandscape())
	{
		Landscape->Modify(false);
		Landscape->SetSelectedEditLayerIndex(InLayerIndex);
	}

	RefreshDetailPanel();
	RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode::Update_Client_Editing);
}

void FEdModeLandscape::ToggleSplinesTool(const ULandscapeEditLayerBase* InEditLayer)
{
	const ALandscape* Landscape = GetLandscape();
	 
	if (Landscape == nullptr || InEditLayer == nullptr)
	{
		return;
	}
 
	// Enters spline mode or toggles to the last active landscape edit mode (LastActive defaults to Sculpt mode/tool)
	if (InEditLayer->IsA<ULandscapeEditLayerSplines>() && CurrentTool && CurrentTool->GetToolName() != FName("Splines"))
	{
		SetCurrentToolMode("ToolMode_Manage", false);
		SetCurrentTool(FName("Splines"));
	}
	else
	{
		SetCurrentToolMode(LastActiveToolModeName, false);
		SetCurrentTool(LastActiveToolName);
	}

	SetSelectedEditLayer(Landscape->GetLayerIndex(InEditLayer->GetGuid()));
}

int32 FEdModeLandscape::GetSelectedEditLayerIndex() const
{
	if (ALandscape* Landscape = GetLandscape())
	{
		return Landscape->GetSelectedEditLayerIndex();
	}

	return INDEX_NONE;
}

ALandscape* FEdModeLandscape::GetLandscape() const
{
	return CurrentToolTarget.LandscapeInfo.IsValid() ? CurrentToolTarget.LandscapeInfo->LandscapeActor.Get() : nullptr;
}

bool FEdModeLandscape::CanRenameLayerTo(int32 InLayerIndex, const FName& InNewName) const
{
	if (ALandscape* Landscape = GetLandscape(); Landscape && !Landscape->IsLayerNameUnique(InNewName))
			{
				return false;
			}
	return true;
}

bool FEdModeLandscape::IsLayerAlphaVisible(int32 InLayerIndex) const
{
	return (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap || CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap);
}

const ULandscapeEditLayerBase* FEdModeLandscape::GetEditLayerConst(int32 InLayerIndex) const
{
	const ALandscape* Landscape = GetLandscape();
	return Landscape ? Landscape->GetEditLayerConst(InLayerIndex) : nullptr;
}

ULandscapeEditLayerBase* FEdModeLandscape::GetEditLayer(int32 InLayerIndex) const
{
	const ALandscape* Landscape = GetLandscape();
	return Landscape ? Landscape->GetEditLayer(InLayerIndex) : nullptr;
}

void FEdModeLandscape::RequestLayersContentUpdate(ELandscapeLayerUpdateMode InUpdateMode)
{
	if (ALandscape* Landscape = GetLandscape())
	{
		Landscape->RequestLayersContentUpdate(InUpdateMode);
	}
}

void FEdModeLandscape::RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode InUpdateMode)
{
	if (ALandscape* Landscape = GetLandscape())
	{
		Landscape->RequestLayersContentUpdateForceAll(InUpdateMode);
	}
}

void FEdModeLandscape::AddBrushToCurrentLayer(ALandscapeBlueprintBrushBase* InBrush)
{
	ALandscape* Landscape = GetLandscape();
	if (Landscape == nullptr)
	{
		return;
	}

	Landscape->AddBrushToLayer(GetSelectedEditLayerIndex(), InBrush);
	RefreshDetailPanel();
}

void FEdModeLandscape::RemoveBrushFromCurrentLayer(int32 InBrushIndex)
{
	ALandscape* Landscape = GetLandscape();

	if (Landscape == nullptr)
	{
		return;
	}

	Landscape->RemoveBrushFromLayer(GetSelectedEditLayerIndex(), InBrushIndex);
	RefreshDetailPanel();
}

ALandscapeBlueprintBrushBase* FEdModeLandscape::GetBrushForCurrentLayer(int32 InBrushIndex) const
{
	if (ALandscape* Landscape = GetLandscape())
	{
		return Landscape->GetBrushForLayer(GetSelectedEditLayerIndex(), InBrushIndex);
	}
	return nullptr;
}

TArray<ALandscapeBlueprintBrushBase*> FEdModeLandscape::GetBrushesForCurrentLayer()
{
	TArray<ALandscapeBlueprintBrushBase*> Brushes;
	if (ALandscape* Landscape = GetLandscape())
	{
		Brushes = Landscape->GetBrushesForLayer(GetSelectedEditLayerIndex());
	}
	return Brushes;
}

void FEdModeLandscape::ShowOnlySelectedBrush(class ALandscapeBlueprintBrushBase* InBrush)
{
	if (ALandscape * Landscape = GetLandscape())
	{
		int32 BrushLayer = Landscape->GetBrushLayer(InBrush);
		TArray<ALandscapeBlueprintBrushBase*> Brushes = Landscape->GetBrushesForLayer(BrushLayer);
		for (ALandscapeBlueprintBrushBase* Brush : Brushes)
		{
			Brush->SetIsVisible(Brush == InBrush);
		}
	}
}

void FEdModeLandscape::DuplicateBrush(class ALandscapeBlueprintBrushBase* InBrush)
{
	GEditor->SelectNone(false, true);
	GEditor->SelectActor(InBrush, true, false, false);

	GEditor->edactDuplicateSelected(InBrush->GetLevel(), false);
}

const ULandscapeEditLayerBase* FEdModeLandscape::GetCurrentEditLayerConst() const
{
	return GetEditLayerConst(GetSelectedEditLayerIndex());
}

void FEdModeLandscape::AutoUpdateDirtyLandscapeSplines()
{
	if (GEditor->IsTransactionActive())
	{
		// Only auto-update if a layer is reserved for landscape splines
		ALandscape* Landscape = GetLandscape();
		if (Landscape)
		{
			// TODO : Only update dirty regions
			Landscape->RequestSplineLayerUpdate();
		}
	}
}

bool FEdModeLandscape::CanEditLayer(FText* Reason /*=nullptr*/, const ULandscapeEditLayerBase* InLayer /*= nullptr*/)
{
	ALandscape* Landscape = GetLandscape();
	const ULandscapeEditLayerBase* TargetLayer = InLayer ? InLayer : GetCurrentEditLayerConst();
	if (!TargetLayer)
	{
		if (Reason)
		{
			*Reason = NSLOCTEXT("UnrealEd", "LandscapeInvalidLayer", "No layer selected. You must first chose a layer to work on.");
		}
		return false;
	}

	check(TargetLayer != nullptr);
	if (!TargetLayer->IsVisible())
	{
		if (Reason)
		{
			*Reason = NSLOCTEXT("UnrealEd", "LandscapeLayerHidden", "Painting or sculpting in a hidden layer is not allowed.");
		}
		return false;
	}
	else if (TargetLayer->IsLocked())
	{
		if (Reason)
		{
			*Reason = NSLOCTEXT("UnrealEd", "LandscapeLayerLocked", "This layer is locked. You must unlock it before you can work on this layer.");
		}
		return false;
	}
	else if (CurrentTool)
	{
		// Special case for the splines tool, which is supported on the splines layer : 
		if (CurrentTool != (FLandscapeTool*)SplinesTool)
		{
			if (Landscape && (TargetLayer == Landscape->FindEditLayerOfTypeConst(ULandscapeEditLayerSplines::StaticClass())))
			{
				if (Reason)
				{
					*Reason = NSLOCTEXT("UnrealEd", "LandscapeLayerReservedForSplines", "This layer is reserved for Landscape Splines.");
				}
				return false;
			}

			if (!TargetLayer->SupportsEditingTools())
			{
				if (Reason)
				{
					*Reason = FText::Format(NSLOCTEXT("UnrealEd", "LandscapeLayerEditLayerDoesntSupportEditing", "This layer's type ({0}) doesn't support direct editing."), TargetLayer->GetClass()->GetDisplayNameText());
				}
				return false;
			}
		}
	}

	return true;
}

bool FEdModeLandscape::CanEditSelectedTargetLayer(FText* Reason /*=nullptr*/, const ULandscapeEditLayerBase* InLayer /*= nullptr*/)
{
	return CanEditTargetLayer(Reason, CurrentToolTarget.LayerInfo.Get(), InLayer);
}

bool FEdModeLandscape::CanEditTargetLayer(FText* Reason /*=nullptr*/, const ULandscapeLayerInfoObject* InLayerInfo  /*=nullptr*/, const ULandscapeEditLayerBase* InLayer /*= nullptr*/)
{
	// Target layer is only editable when the edit layer is
	if (!CanEditLayer(Reason, InLayer))
	{
		return false;
	}

	if (CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap && InLayerInfo == nullptr)
	{
		if (Reason)
		{
			*Reason = NSLOCTEXT("UnrealEd", "LandscapeNeedToCreateLayerInfo", "This layer has no layer info assigned yet. You must create or assign a layer info before you can paint this layer.");
		}
		return false;
	}
	return true;
}

FGuid FEdModeLandscape::GetCurrentLayerGuid() const
{
	const ULandscapeEditLayerBase* EditLayer = GetCurrentEditLayerConst();
	return EditLayer ? EditLayer->GetGuid() : FGuid();
}

void FEdModeLandscape::UpdateBrushList()
{
	BrushList.Empty();
	for (TObjectIterator<ALandscapeBlueprintBrushBase> BrushIt(RF_Transient|RF_ClassDefaultObject|RF_ArchetypeObject, true, EInternalObjectFlags::Garbage); BrushIt; ++BrushIt)
	{
		ALandscapeBlueprintBrushBase* Brush = *BrushIt;
		if (Brush->GetPackage() != GetTransientPackage())
		{
			BrushList.Add(Brush);
		}
	}
}


void FEdModeLandscape::OnLevelActorAdded(AActor* InActor)
{
	if (ALandscape* Landscape = Cast <ALandscape>(InActor))
	{
		Landscape->RegisterLandscapeEdMode(this);
	}

	ALandscapeBlueprintBrushBase* Brush = Cast<ALandscapeBlueprintBrushBase>(InActor);
	if (Brush && Brush->GetPackage() != GetTransientPackage())
	{
		if (!GIsReinstancing)
		{
			AddBrushToCurrentLayer(Brush);
		}
		UpdateBrushList();
		RefreshDetailPanel();
	}
}

void FEdModeLandscape::OnLevelActorRemoved(AActor* InActor)
{
	if (ALandscape* Landscape = Cast <ALandscape>(InActor))
	{
		Landscape->UnregisterLandscapeEdMode();
	}

	ALandscapeBlueprintBrushBase* Brush = Cast<ALandscapeBlueprintBrushBase>(InActor);
	if (Brush && Brush->GetPackage() != GetTransientPackage())
	{
		UpdateBrushList();
		RefreshDetailPanel();
	}
}

bool LandscapeEditorUtils::SetHeightmapData(ALandscapeProxy* Landscape, const TArray<uint16>& Data)
{
	FIntRect ComponentsRect = Landscape->GetBoundingRect() + Landscape->GetSectionBase();

	if (Data.Num() == (1 + ComponentsRect.Width())*(1 + ComponentsRect.Height()))
	{
		FHeightmapAccessor<false> HeightmapAccessor(Landscape->GetLandscapeInfo());
		HeightmapAccessor.SetData(ComponentsRect.Min.X, ComponentsRect.Min.Y, ComponentsRect.Max.X, ComponentsRect.Max.Y, Data.GetData());
		return true;
	}

	return false;
}

bool LandscapeEditorUtils::SetWeightmapData(ALandscapeProxy* Landscape, ULandscapeLayerInfoObject* LayerObject, const TArray<uint8>& Data)
{
	FIntRect ComponentsRect = Landscape->GetBoundingRect() + Landscape->GetSectionBase();

	if (Data.Num() == (1 + ComponentsRect.Width())*(1 + ComponentsRect.Height()))
	{
		TAlphamapAccessor</*bInUseInterp = */false> AlphamapAccessor(Landscape->GetLandscapeInfo(), LayerObject);
		AlphamapAccessor.SetData(ComponentsRect.Min.X, ComponentsRect.Min.Y, ComponentsRect.Max.X, ComponentsRect.Max.Y, Data.GetData(), ELandscapeLayerPaintingRestriction::None);
		return true;
	}

	return false;
}

FName FLandscapeTargetListInfo::GetLayerName() const
{
	return LayerInfoObj.IsValid() ? LayerInfoObj->GetLayerName() : LayerName;
}

void FEdModeLandscape::FInputProcessor::StartTrackingInput()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().RegisterInputPreProcessor(AsShared());
	}
}

void FEdModeLandscape::FInputProcessor::StopTrackingInput()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().UnregisterInputPreProcessor(AsShared());
	}
}

bool FEdModeLandscape::FInputProcessor::HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	bIsShiftDown = SlateApp.GetModifierKeys().IsShiftDown();
	bIsCtrlDown = SlateApp.GetModifierKeys().IsControlDown();
	// return false, as we don't want to disturb the input handling, we just want to track whether the modifier keys are pressed at the application level
	return false;
}

bool FEdModeLandscape::FInputProcessor::HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent)
{
	bIsShiftDown = SlateApp.GetModifierKeys().IsShiftDown();
	bIsCtrlDown = SlateApp.GetModifierKeys().IsControlDown();
	// return false, as we don't want to disturb the input handling, we just want to track whether the modifier keys are pressed at the application level
	return false;
}

#undef LOCTEXT_NAMESPACE