// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UnrealWidgetFwd.h"
#include "LandscapeProxy.h"
#include "EdMode.h"
#include "LandscapeToolInterface.h"
#include "Materials/MaterialInstanceConstant.h"
#include "LandscapeInfo.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeGizmoActiveActor.h"
#include "LandscapeEdit.h"
#include "LandscapeEditTypes.h"
#include "Containers/Set.h"
#include "LandscapeImportHelper.h"
#include "Framework/Application/IInputProcessor.h"

class ALandscape;
class FCanvas;
class FEditorViewportClient;
class FLandscapeToolSplines;
class FPrimitiveDrawInterface;
class FSceneView;
class FUICommandList;
class FViewport;
class ULandscapeComponent;
class ULandscapeEditorObject;
class UViewportInteractor;
class ULandscapeEditLayerBase;
struct FHeightmapToolTarget;
struct FViewportActionKeyInput;
struct FViewportClick;
template<class ToolTarget> class FLandscapeToolCopyPaste;
enum class ESplineNavigationFlags : uint8;

DECLARE_LOG_CATEGORY_EXTERN(LogLandscapeEdMode, Log, All);

// Forward declarations
class ULandscapeEditorObject;
class ULandscapeLayerInfoObject;
class FLandscapeToolSplines;
class UViewportInteractor;
struct FViewportActionKeyInput;
namespace ELandscapeViewMode 
{
	enum Type : int32;
}

struct FHeightmapToolTarget;
template<typename TargetType> class FLandscapeToolCopyPaste;

struct FLandscapeToolMode
{
	const FName ToolModeName;
	ELandscapeToolTargetTypeFlags SupportedTargetTypes;

	TArray<FName> ValidTools;
	FName CurrentToolName;
	FName CurrentTargetLayerName;

	FLandscapeToolMode(FName InToolModeName, ELandscapeToolTargetTypeFlags InSupportedTargetTypes)
		: ToolModeName(InToolModeName)
		, SupportedTargetTypes(InSupportedTargetTypes)
	{
	}
};

struct FLandscapeTargetListInfo
{
	FText TargetLayerDisplayName;									// UI Display Name
	ELandscapeToolTargetType TargetType = ELandscapeToolTargetType::Invalid;
	TWeakObjectPtr<ULandscapeInfo> LandscapeInfo;

	TWeakObjectPtr<ULandscapeLayerInfoObject> LayerInfoObj;			// ignored for heightmap
	FName LayerName;												// ignored for heightmap
	TWeakObjectPtr<class ALandscapeProxy> Owner;					// ignored for heightmap
	TWeakObjectPtr<class UMaterialInstanceConstant> ThumbnailMIC;	// ignored for heightmap
	int32 DebugColorChannel = -1;									// ignored for heightmap
	int32 LayerIndex = INDEX_NONE;
	bool bIsLayerReferencedByMaterial = false;

	FLandscapeTargetListInfo(FText InTargetLayerDisplayName, ELandscapeToolTargetType InTargetType, const FLandscapeInfoLayerSettings& InLayerSettings, int32 InLayerIndex, bool bInIsLayerReferencedByMaterial)
		: TargetLayerDisplayName(InTargetLayerDisplayName)
		, TargetType(InTargetType)
		, LandscapeInfo(InLayerSettings.Owner->GetLandscapeInfo())
		, LayerInfoObj(InLayerSettings.LayerInfoObj)
		, LayerName(InLayerSettings.LayerName)
		, Owner(InLayerSettings.Owner)
		, ThumbnailMIC(InLayerSettings.ThumbnailMIC)
		, DebugColorChannel(InLayerSettings.DebugColorChannel)
		, LayerIndex(InLayerIndex)
		, bIsLayerReferencedByMaterial(bInIsLayerReferencedByMaterial)
	{
	}

	FLandscapeTargetListInfo(FText InTargetLayerDisplayName, ELandscapeToolTargetType InTargetType, ULandscapeInfo* InLandscapeInfo, int32 InLayerIndex, bool bInIsLayerReferencedByMaterial)
		: TargetLayerDisplayName(InTargetLayerDisplayName)
		, TargetType(InTargetType)
		, LandscapeInfo(InLandscapeInfo)
		, LayerInfoObj(nullptr)
		, LayerName(NAME_None)
		, Owner(nullptr)
		, ThumbnailMIC(nullptr)
		, LayerIndex(InLayerIndex)
		, bIsLayerReferencedByMaterial(bInIsLayerReferencedByMaterial)
	{
	}

	int32 GetLandscapeInfoLayerIndex() const
	{
		int32 Index = INDEX_NONE;

		if (TargetType == ELandscapeToolTargetType::Weightmap)
		{
			if (LayerInfoObj.IsValid())
			{
				Index = LandscapeInfo->GetLayerInfoIndex(LayerInfoObj.Get(), Owner.Get());
			}
			else
			{
				Index = LandscapeInfo->GetLayerInfoIndex(LayerName, Owner.Get());
			}
		}

		return Index;
	}

	FLandscapeInfoLayerSettings* GetLandscapeInfoLayerSettings() const
	{
		int32 Index = GetLandscapeInfoLayerIndex();

		if (Index != INDEX_NONE)
		{
			return &LandscapeInfo->Layers[Index];
		}

		return nullptr;
	}

	const FLandscapeTargetLayerSettings* GetTargetLayerSettings() const
	{
		if (TargetType == ELandscapeToolTargetType::Weightmap)
		{
			check(LayerInfoObj.IsValid());
			ALandscapeProxy* Proxy = LandscapeInfo->GetLandscapeProxy();
			check(Proxy != nullptr);
			const FName* TargetLayerName = Proxy->GetTargetLayers().FindKey(FLandscapeTargetLayerSettings(LayerInfoObj.Get()));
			if (TargetLayerName)
			{
				return Proxy->GetTargetLayers().Find(*TargetLayerName);
			}
			else
			{
				return &Proxy->AddTargetLayer(LayerInfoObj->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfoObj.Get()));
			}
		}
		return nullptr;
	}

	FName GetLayerName() const;

	FString GetReimportFilePath() const
	{
		if (TargetType == ELandscapeToolTargetType::Weightmap)
		{
			const FLandscapeTargetLayerSettings* EditorLayerSettings = GetTargetLayerSettings();
			check(EditorLayerSettings);
			return EditorLayerSettings->ReimportLayerFilePath;
		}
		else //if (TargetType == ELandscapeToolTargetType::Heightmap)
		{
			if (LandscapeInfo.IsValid())
			{
				ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();

				if (LandscapeProxy)
				{
					return LandscapeProxy->ReimportHeightmapFilePath;
				}
			}

			return FString(TEXT(""));
		}
	}

	void SetReimportFilePath(const FString& InNewPath)
	{
		if (TargetType == ELandscapeToolTargetType::Weightmap)
		{
			check(LayerInfoObj.IsValid());
			ALandscapeProxy* Proxy = LandscapeInfo->GetLandscapeProxy();
			const FName* TargetLayerName = Proxy->GetTargetLayers().FindKey(FLandscapeTargetLayerSettings(LayerInfoObj.Get()));
			if (TargetLayerName)
			{
				FLandscapeTargetLayerSettings LayerSettings = *Proxy->GetTargetLayers().Find(*TargetLayerName);
				LayerSettings.ReimportLayerFilePath = InNewPath;
				Proxy->UpdateTargetLayer(*TargetLayerName, LayerSettings);
			}
		}
		else //if (TargetType == ELandscapeToolTargetType::Heightmap)
		{
			if (LandscapeInfo.IsValid())
			{
				ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();

				if (LandscapeProxy)
				{
					LandscapeProxy->ReimportHeightmapFilePath = InNewPath;
				}
			}
		}
	}
};

struct FLandscapeListInfo
{
	FString LandscapeName;
	ULandscapeInfo* Info;
	int32 ComponentQuads;
	int32 NumSubsections;
	int32 Width;
	int32 Height;

	FLandscapeListInfo(const TCHAR* InName, ULandscapeInfo* InInfo, int32 InComponentQuads, int32 InNumSubsections, int32 InWidth, int32 InHeight)
		: LandscapeName(InName)
		, Info(InInfo)
		, ComponentQuads(InComponentQuads)
		, NumSubsections(InNumSubsections)
		, Width(InWidth)
		, Height(InHeight)
	{
	}
};

enum class ENewLandscapePreviewMode : uint8
{
	None,
	NewLandscape,
	ImportLandscape,
};

enum class EImportExportMode : uint8
{
	Import,
	Export,
};

enum class ELandscapeEditingState : uint8
{
	Unknown,
	Enabled,
	BadFeatureLevel,
	PIEWorld,
	SIEWorld,
	NoLandscape,
};

/**
 * Landscape editor mode
 */
class FEdModeLandscape 
	: public FEdMode
	, public ILandscapeEdModeInterface
{
public:

	TObjectPtr<ULandscapeEditorObject> UISettings;

	FText ErrorReasonOnMouseUp;

	FLandscapeToolMode* CurrentToolMode;
	FLandscapeTool* CurrentTool;
	FLandscapeBrush* CurrentBrush;
	FLandscapeToolTarget CurrentToolTarget;

	// GizmoBrush for Tick
	FLandscapeBrush* GizmoBrush;
	// UI setting for additional UI Tools
	int32 CurrentToolIndex;
	// UI setting for additional UI Tools
	int32 CurrentBrushSetIndex;
	// Persistent View Mode when toggling landscape editor
	ELandscapeViewMode::Type CurrentLandscapeViewMode;
	// Persistent Target Layer index when entering a layer rename
	int32 PendingRenameTargetLayerIndex;

	ENewLandscapePreviewMode NewLandscapePreviewMode;
	EImportExportMode ImportExportMode;

	// UI callbacks for copy/paste tool
	FLandscapeToolCopyPaste<FHeightmapToolTarget>* CopyPasteTool;
	void CopyDataToGizmo();
	void PasteDataFromGizmo();

	// UI callbacks for splines tool
	FLandscapeToolSplines* SplinesTool;
	void ShowSplineProperties();
	bool HasSelectedSplineSegments() const;
	bool HasAdjacentLinearSplineConnection(ESplineNavigationFlags Flags) const;
	bool HasEndLinearSplineConnection(ESplineNavigationFlags Flags) const;
	void FlipSelectedSplineSegments();
	void GetSelectedSplineOwners(TSet<AActor*>& SelectedSplineOwners) const;
	virtual void SelectAllSplineControlPoints(ESplineNavigationFlags Flags);
	virtual void SelectAllSplineSegments(ESplineNavigationFlags Flags);
	virtual void SelectAllConnectedSplineControlPoints();
	virtual void SelectAllConnectedSplineSegments();
	virtual void SelectAdjacentLinearSplineElement(ESplineNavigationFlags Flags) const;
	virtual void SelectEndLinearSplineElement(ESplineNavigationFlags Flags) const;
	virtual void SelectSplineControlPointsFromCurrentSegmentSelection() const;
	virtual void SelectSplineSegmentsFromCurrentControlPointSelection() const;
	virtual void SplineMoveToCurrentLevel();
	virtual bool CanMoveSplineToCurrentLevel() const;
	virtual void UpdateSplineMeshLevels();
	void SetbUseAutoRotateOnJoin(bool InbAutoRotateOnJoin);
	bool GetbUseAutoRotateOnJoin();
	void SetbAlwaysRotateForward(bool InbAlwaysRotateForward);
	bool GetbAlwaysRotateForward();

	// UI callbacks for ramp tool
	void ApplyRampTool();
	bool CanApplyRampTool();
	void ResetRampTool();

	// UI callbacks for mirror tool
	void ApplyMirrorTool();
	void CenterMirrorTool();

	/** Constructor */
	FEdModeLandscape();

	/** Initialization */
	void InitializeBrushes();
	void InitializeTool_Paint();
	void InitializeTool_Smooth();
	void InitializeTool_Flatten();
	void InitializeTool_Erosion();
	void InitializeTool_HydraErosion();
	void InitializeTool_Noise();
	void InitializeTool_Retopologize();
	void InitializeTool_NewLandscape();
	void InitializeTool_ResizeLandscape();
	void InitializeTool_ImportExport();
	void InitializeTool_Select();
	void InitializeTool_AddComponent();
	void InitializeTool_DeleteComponent();
	void InitializeTool_MoveToLevel();
	void InitializeTool_Mask();
	void InitializeTool_CopyPaste();
	void InitializeTool_Visibility();
	void InitializeTool_Splines();
	void InitializeTool_Ramp();
	void InitializeTool_Mirror();
	void InitializeTool_BlueprintBrush();
	void UpdateToolModes();

	/** Destructor */
	virtual ~FEdModeLandscape();

	/** ILandscapeEdModeInterface */
	virtual void PostUpdateLayerContent() override;
	virtual ELandscapeToolTargetType GetLandscapeToolTargetType() const override;
	virtual const ULandscapeEditLayerBase* GetLandscapeSelectedLayer() const override;
	virtual ULandscapeLayerInfoObject* GetSelectedLandscapeLayerInfo() const override;

	// Deprecated
	virtual void OnCanHaveLayersContentChanged() final {};

	/** FGCObject interface */
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	virtual bool UsesToolkits() const override;

	TSharedRef<FUICommandList> GetUICommandList() const;

	/** FEdMode: Called when the mode is entered */
	virtual void Enter() override;

	/** FEdMode: Called when the mode is exited */
	virtual void Exit() override;

	/** FEdMode: Called when the mouse enters the viewport area */
	virtual bool MouseEnter(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 MouseX, int32 MouseY) override;

	/** FEdMode: Called when the mouse exits the viewport area */
	virtual bool MouseLeave(FEditorViewportClient* InViewportClient, FViewport* Viewport) override;

	/** FEdMode: Called when the mouse is moved over the viewport */
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override;

	/**
	 * FEdMode: Called when the mouse is moved while a window input capture is in effect
	 *
	 * @param	InViewportClient	Level editor viewport client that captured the mouse input
	 * @param	InViewport			Viewport that captured the mouse input
	 * @param	InMouseX			New mouse cursor X coordinate
	 * @param	InMouseY			New mouse cursor Y coordinate
	 *
	 * @return	true if input was handled
	 */
	virtual bool CapturedMouseMove(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 InMouseX, int32 InMouseY) override;

	/** FEdMode: Called when a mouse button is pressed */
	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	/** FEdMode: Called when a mouse button is released */
	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	/** FEdMode: Called when a transform operation begins */
	virtual bool BeginTransform(const FGizmoState& InState) override;

	/** FEdMode: Called when a transform operation ends */
	virtual bool EndTransform(const FGizmoState& InState) override;

	/** FEdMode: Allow us to disable mouse delta tracking during painting */
	virtual bool DisallowMouseDeltaTracking() const override;

	/** FEdMode: Called once per frame per viewport */
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override;

	/** FEdMode: Called when clicking on a hit proxy */
	virtual bool HandleClick(FEditorViewportClient* InViewportClient, HHitProxy* HitProxy, const FViewportClick& Click) override;

	/** True if we are interactively changing the brush size, falloff, or strength */
	bool IsAdjustingBrush(FEditorViewportClient* InViewportClient) const;
	void ChangeBrushSize(bool bIncrease);
	void ChangeBrushFalloff(bool bIncrease);
	void ChangeBrushStrength(bool bIncrease);
	void ChangeAlphaBrushRotation(bool bIncrease);

	/** FEdMode: Called when a key is pressed */
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;

	/** FEdMode: Called when mouse drag input is applied */
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;

	/** FEdMode: Render elements for the landscape tool */
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override;

	/** FEdMode: Render HUD elements for this tool */
	virtual void DrawHUD(FEditorViewportClient* ViewportClient, FViewport* Viewport, const FSceneView* View, FCanvas* Canvas) override;

	/** FEdMode: Handling SelectActor */
	virtual bool Select(AActor* InActor, bool bInSelected) override;

	/** FEdMode: Check to see if an actor can be selected in this mode - no side effects */
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override;

	/** FEdMode: Called when the currently selected actor has changed */
	virtual void ActorSelectionChangeNotify() override;

	virtual void ActorMoveNotify() override;

	virtual void PostUndo() override;

	virtual EEditAction::Type GetActionEditDuplicate() override;
	virtual EEditAction::Type GetActionEditDelete() override;
	virtual EEditAction::Type GetActionEditCut() override;
	virtual EEditAction::Type GetActionEditCopy() override;
	virtual EEditAction::Type GetActionEditPaste() override;
	virtual bool ProcessEditDuplicate() override;
	virtual bool ProcessEditDelete() override;
	virtual bool ProcessEditCut() override;
	virtual bool ProcessEditCopy() override;
	virtual bool ProcessEditPaste() override;

	/** FEdMode: If the EdMode is handling InputDelta (ie returning true from it), this allows a mode to indicated whether or not the Widget should also move. */
	virtual bool AllowWidgetMove() override { return true; }

	/** FEdMode: Draw the transform widget while in this mode? */
	virtual bool ShouldDrawWidget() const override;

	/** FEdMode: Returns true if this mode uses the transform widget */
	virtual bool UsesTransformWidget() const override;
	
	virtual bool RequiresLegacyViewportInteractions() const override { return true; }

	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override;

	virtual FVector GetWidgetLocation() const override;
	virtual bool GetCustomDrawingCoordinateSystem(FMatrix& InMatrix, void* InData) override;
	virtual bool GetCustomInputCoordinateSystem(FMatrix& InMatrix, void* InData) override;

	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override;

	/** Get override cursor visibility settings */	
	virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const override;

	/** Called before mouse movement is converted to drag/rot */
	virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) override;

	/** Called after mouse movement is converted to drag/rot */
	virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) override;

	/** Forces real-time perspective viewports */
	void ForceRealTimeViewports(const bool bEnable);

	/** Trace under the mouse cursor and return the landscape hit and the hit location (in landscape quad space) */
	bool LandscapeMouseTrace(FEditorViewportClient* ViewportClient, float& OutHitX, float& OutHitY);
	bool LandscapeMouseTrace(FEditorViewportClient* ViewportClient, FVector& OutHitLocation);

	/** Trace under the specified coordinates and return the landscape hit and the hit location (in landscape quad space) */
	bool LandscapeMouseTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, float& OutHitX, float& OutHitY);
	bool LandscapeMouseTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, FVector& OutHitLocation);

	/** Trace under the mouse cursor / specified screen coordinates against a world-space plane and return the hit location (in world space) */
	bool LandscapePlaneTrace(FEditorViewportClient* ViewportClient, const FPlane& Plane, FVector& OutHitLocation);
	bool LandscapePlaneTrace(FEditorViewportClient* ViewportClient, int32 MouseX, int32 MouseY, const FPlane& Plane, FVector& OutHitLocation);

	/** Trace under the specified laser start and direction and return the landscape hit and the hit location (in landscape quad space) */
	bool LandscapeTrace(const FVector& InRayOrigin, const FVector& InRayEnd, const FVector& InDirection, FVector& OutHitLocation);

	struct FProcessLandscapeTraceHitsResult;
	
	/** Check if we've collided with the currently edited landscape, OutHitLocation is in the space of the Landscape Actor */   
	bool ProcessLandscapeTraceHits(const TArray<FHitResult>& InResults, FProcessLandscapeTraceHitsResult& OutLandscapeTraceHitsResult);

	void SetCurrentToolMode(FName ToolModeName, bool bRestoreCurrentTool = true);

	/** Change current tool */
	void SetCurrentTool(FName ToolName, FName TargetLayerName = NAME_None);
	void SetCurrentTool(int32 ToolIdx, FName TargetLayerName = NAME_None);
	void SetCurrentTargetLayer(FName TargetLayerName, TWeakObjectPtr<ULandscapeLayerInfoObject> LayerInfo);

	void SetCurrentBrushSet(FName BrushSetName);
	void SetCurrentBrushSet(int32 BrushSetIndex);

	void SetCurrentBrush(FName BrushName);
	void SetCurrentBrush(int32 BrushIndex);

	void UpdateBrushList();
	const TArray<ALandscapeBlueprintBrushBase*>& GetBrushList() const;

	const TArray<TSharedRef<FLandscapeTargetListInfo>>& GetTargetList() const;
	UMaterialInterface* GetTargetLandscapeMaterial() const { return CachedLandscapeMaterial; }
	const TArray<FName>* GetTargetDisplayOrderList() const;
	int32 GetTargetLayerStartingIndex() const;
	const TArray<FLandscapeListInfo>& GetLandscapeList();
	const FString GetTargetLayerAssetPackagePath(const bool bIsEmptyPathValid = false) const;

	int32 UpdateLandscapeList();
	void UpdateTargetList(bool bRegenerateThumbnails = false);
	void SetTargetLandscape(const TWeakObjectPtr<ULandscapeInfo>& InLandscapeInfo);
	bool CanEditCurrentTarget(FText* Reason = nullptr) const;

	/** Update Display order list */
	void UpdateTargetLayerDisplayOrder(ELandscapeLayerDisplayMode InTargetDisplayOrder, bool bInIsAscendingOrder);
	void MoveTargetLayerDisplayOrder(int32 IndexToMove, int32 IndexToDestination);

	void RequestUpdateLayerUsageInformation();
	bool ShouldShowLayer(TSharedRef<FLandscapeTargetListInfo> Target) const;
	void UpdateLayerUsageInformation(TWeakObjectPtr<ULandscapeLayerInfoObject>* LayerInfoObjectThatChanged = nullptr);
	void OnLandscapeMaterialChangedDelegate(ALandscapeProxy* InProxyChanged, const FOnLandscapeProxyMaterialChangedParams& InParams);
	void RefreshDetailPanel();
	void RefreshInspectedObjectsDetailPanel();
	void RegenerateLayerThumbnails();

	bool IsGridBased() const;

	// Edit Layers
	void ToggleSplinesTool(const ULandscapeEditLayerBase* InEditLayer);
	int32 GetLayerCount() const;
	void SetSelectedEditLayer(int32 InLayerIndex);
	int32 GetSelectedEditLayerIndex() const;
	ALandscape* GetLandscape() const;
	bool CanRenameLayerTo(int32 InLayerIndex, const FName& InNewName) const;
	ULandscapeEditLayerBase* GetEditLayer(int32 InLayerIndex) const;
	const ULandscapeEditLayerBase* GetEditLayerConst(int32 InLayerIndex) const;
	bool IsLayerAlphaVisible(int32 InLayerIndex) const;
	const ULandscapeEditLayerBase* GetCurrentEditLayerConst() const;
	FGuid GetCurrentLayerGuid() const;
	void AutoUpdateDirtyLandscapeSplines();
	bool CanEditLayer(FText* Reason = nullptr, const ULandscapeEditLayerBase* InLayer = nullptr);
	bool CanEditSelectedTargetLayer(FText* Reason = nullptr, const ULandscapeEditLayerBase* InLayer = nullptr);
	bool CanEditTargetLayer(FText* Reason = nullptr, const ULandscapeLayerInfoObject* InLayerInfo = nullptr, const ULandscapeEditLayerBase* InLayer = nullptr);

	void AddBrushToCurrentLayer(class ALandscapeBlueprintBrushBase* InBrush);
	void RemoveBrushFromCurrentLayer(int32 InBrushIndex);
	class ALandscapeBlueprintBrushBase* GetBrushForCurrentLayer(int32 InBrushIndex) const;
	TArray<class ALandscapeBlueprintBrushBase*> GetBrushesForCurrentLayer();
	
	void ShowOnlySelectedBrush(class ALandscapeBlueprintBrushBase* InBrush);
	
	void DuplicateBrush(class ALandscapeBlueprintBrushBase* InBrush);

	void RequestLayersContentUpdate(ELandscapeLayerUpdateMode InUpdateMode);
	void RequestLayersContentUpdateForceAll(ELandscapeLayerUpdateMode InUpdateMode = ELandscapeLayerUpdateMode::Update_All);

	void OnLevelActorAdded(AActor* InActor);
	void OnLevelActorRemoved(AActor* InActor);

	DECLARE_EVENT(FEdModeLandscape, FTargetsListUpdated);
	static FTargetsListUpdated TargetsListUpdated;

	void OnPreSaveWorld(class UWorld* InWorld, FObjectPreSaveContext ObjectSaveContext);

	/** Handle notification that visible levels may have changed and we should update the editable landscapes list */
	void HandleLevelsChanged();

	void ReimportData(const FLandscapeTargetListInfo& TargetInfo);
	void ImportData(const FLandscapeTargetListInfo& TargetInfo, const FString& Filename);
	void ImportHeightData(ULandscapeInfo* LandscapeInfo, const FGuid& LayerGuid, const FString& Filename, const FIntRect& ImportRegionVerts, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0,0), const ELandscapeLayerPaintingRestriction& PaintRestriction = ELandscapeLayerPaintingRestriction::None, bool bFlipYAxis = false);
	void ImportWeightData(ULandscapeInfo* LandscapeInfo, const FGuid& LayerGuid, ULandscapeLayerInfoObject* LayerInfo, const FString& Filename, const FIntRect& ImportRegionVerts, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0, 0), const ELandscapeLayerPaintingRestriction& PaintRestriction = ELandscapeLayerPaintingRestriction::None, bool bFlipYAxis = false);
	bool UseSingleFileImport() const { return !IsGridBased(); }

	/** Resample landscape to a different resolution or change the component size */
	ALandscape* ChangeComponentSetting(int32 NumComponentsX, int32 NumComponentsY, int32 InNumSubsections, int32 InSubsectionSizeQuads, bool bResample);

	/** Delete the specified landscape components */
	void DeleteLandscapeComponents(ULandscapeInfo* LandscapeInfo, TSet<ULandscapeComponent*> ComponentsToDelete);

	TArray<FLandscapeToolMode> LandscapeToolModes;
	TArray<TUniquePtr<FLandscapeTool>> LandscapeTools;
	TArray<FLandscapeBrushSet> LandscapeBrushSets;

	ELandscapeEditingState GetEditingState() const;

	bool IsEditingEnabled() const
	{
		return GetEditingState() == ELandscapeEditingState::Enabled;
	}

	static bool IsLandscapeViewModeExclusiveToEditorMode(ELandscapeViewMode::Type ViewMode);

	void SetLandscapeInfo(ULandscapeInfo* InLandscapeInfo);

	/** Returns the sum of all landscape actors resolution. */
	int32 GetAccumulatedAllLandscapesResolution() const;

	/** Returns true if landscape resolution combined to the current tool action is still compliant to the currently applied limitations. */
	bool IsLandscapeResolutionCompliant() const;

	/** Returns true if the current landscape tool handles edit layers. */
	bool DoesCurrentToolAffectEditLayers() const;

	/** Returns the default Error Text when modifying or creating landscape would break the resolution limit. */
	FText GetLandscapeResolutionErrorText() const;

	int32 GetNewLandscapeResolutionX() const;
	int32 GetNewLandscapeResolutionY() const;
	
	TArray<TWeakObjectPtr<UObject>> GetInspectedObjects() const 
	{ 
		return InspectedObjects;
	}
	void SetInspectedObjects(const TArray<TWeakObjectPtr<UObject>>& InObjects);

	bool IsGlobalModifierPressed() const;
	bool IsGlobalAlternateModifierPressed() const;

	/**	See GetTargetLayerActionType, except it's a global function meant to be used in the UX, where there's no viewport, and the action type for a given layer depends on 
	  the pressed modifier buttons (only available per-viewport), so this will give an "estimate" of the result, based on the fact that the modifier button was pressed on 
	  at least one of the active viewports. Not pretty, I know... */
	ELandscapeTargetLayerActionType GetEstimatedTargetLayerActionType(const TSharedRef<FLandscapeTargetListInfo> Target);

private:
	TArray<TSharedRef<FLandscapeTargetListInfo>> LandscapeTargetList;
	TArray<FLandscapeListInfo> LandscapeList;
	TArray<ALandscapeBlueprintBrushBase*> BrushList;

	/** Represent the index offset of the target layer in LandscapeTargetList */
	int32 TargetLayerStartingIndex;

	UMaterialInterface* CachedLandscapeMaterial;

	const FViewport* ToolActiveViewport;

	FDelegateHandle OnWorldChangeDelegateHandle;
	FDelegateHandle OnLevelsChangedDelegateHandle;

	FDelegateHandle OnLevelActorDeletedDelegateHandle;
	FDelegateHandle OnLevelActorAddedDelegateHandle;
	FDelegateHandle PreSaveWorldHandle;
	FDelegateHandle OnIsEditingDisallowedChangedHandle;
	
	/** Check if we are painting using the VREditor */
	bool bIsPaintingInVR;

	/** The interactor that is currently painting, prevents multiple interactors from sculpting when one actually is */
	class UViewportInteractor* InteractorPainting;

	/** Delayed refresh */
	bool bNeedsUpdateLayerUsageInformation;
	bool bUpdatingLandscapeInfo;
	bool bNeedsUpdateTargetLayerList;

	/** When the map is changed, use this flag to make sure Exit() does not overwrite saved UISettings with default data */
	bool bHasMapChanged;

	/** The landscape mode panel has the ability to display the properties of certain UObjects if needed (to bypass the fact that the standard details 
	 panel is not able to display anything else than actors and components) */
	TArray<TWeakObjectPtr<UObject>> InspectedObjects;

	// Stores last active tool info. Used to toggle between current and last editor modes (Eg. Toggle Splines Layer)
	FName LastActiveToolModeName = FName("ToolMode_Sculpt");
	FName LastActiveToolName = FName("Sculpt");

	// Implement a IInputProcessor to be able to track the shift/ctrl modifier key's states globally (i.e. not per-viewport), in order to access it in the editor mode UX code
	struct FInputProcessor : public IInputProcessor, public TSharedFromThis<FInputProcessor>
	{
		void StartTrackingInput();
		void StopTrackingInput();

		//~ Begin IInputProcessor interface
		virtual void Tick(const float DeltaTime, FSlateApplication& SlateApp, TSharedRef<ICursor> Cursor) override {};
		virtual bool HandleKeyDownEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
		virtual bool HandleKeyUpEvent(FSlateApplication& SlateApp, const FKeyEvent& InKeyEvent) override;
		virtual const TCHAR* GetDebugName() const 
		{ 
			return TEXT("FEdModeLandscape::FInputProcessor"); 
		}
		//~ End IInputProcessor interface

		bool bIsShiftDown = false;
		bool bIsCtrlDown = false;
	};
	TSharedRef<FInputProcessor> InputProcessor;
};