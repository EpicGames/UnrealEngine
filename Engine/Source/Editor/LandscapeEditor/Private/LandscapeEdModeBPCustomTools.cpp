// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Materials/MaterialInterface.h"
#include "AI/NavigationSystemBase.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UnrealWidgetFwd.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeEdMode.h"
#include "Containers/ArrayView.h"
#include "LandscapeEditorObject.h"
#include "ScopedTransaction.h"
#include "LandscapeEdit.h"
#include "LandscapeDataAccess.h"
#include "LandscapeRender.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "LandscapeEdModeTools.h"
#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeInfo.h"
#include "Landscape.h"
#include "ActorFactories/ActorFactory.h"
//#include "LandscapeDataAccess.h"

#define LOCTEXT_NAMESPACE "Landscape"

class FLandscapeToolBlueprintBrush : public FLandscapeTool
{
protected:
	FEdModeLandscape* EdMode;

public:
	FLandscapeToolBlueprintBrush(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
	{
	}

	virtual bool UsesTransformWidget() const { return true; }
	virtual bool OverrideWidgetLocation() const { return false; }
	virtual bool OverrideWidgetRotation() const { return false; }

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("BlueprintBrush"); }
	virtual FText GetDisplayName() const override { return FText(); };
	virtual FText GetDisplayMessage() const override { return FText(); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual void EnterTool() override
	{
	}

	virtual void ExitTool() override
	{
		GEditor->SelectNone(true, true);
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) 
	{
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		UClass* BrushClassPtr = EdMode->UISettings->BlueprintBrush.Get();
		if (BrushClassPtr == nullptr)
		{
			return false;
		}

		ALandscapeBlueprintBrushBase* DefaultObject = Cast<ALandscapeBlueprintBrushBase>(BrushClassPtr->GetDefaultObject(false));

		if (DefaultObject == nullptr)
		{
			return false;
		}

		const ULandscapeEditLayerBase* EditLayer = EdMode->GetCurrentEditLayerConst();
		// Brushes can only be placed if the BP Brush can affect the landscape and the current edit layer supports BP brushes
		if ((EditLayer && EditLayer->SupportsBlueprintBrushes()) && (DefaultObject->CanAffectHeightmap() || DefaultObject->CanAffectWeightmap() || DefaultObject->CanAffectVisibilityLayer()))
		{
			ULandscapeInfo* Info = EdMode->CurrentToolTarget.LandscapeInfo.Get();
			check(Info);

			FVector SpawnLocation = Info->GetLandscapeProxy()->LandscapeActorToWorld().TransformPosition(InHitLocation);

			FString BrushActorString = FString::Format(TEXT("{0}_{1}"), { Info->LandscapeActor.Get()->GetActorLabel(), BrushClassPtr->GetName() });
			FName BrushActorName = MakeUniqueObjectName(Info->LandscapeActor.Get()->GetLevel(), BrushClassPtr, FName(BrushActorString));

			FActorSpawnParameters SpawnInfo;
			SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
			SpawnInfo.bNoFail = true;
			SpawnInfo.OverrideLevel = Info->LandscapeActor.Get()->GetLevel(); // always spawn in the same level as the one containing the ALandscape
			SpawnInfo.Name = BrushActorName;

			// If brush overrides default behavior do not create one from the BP brush tool 
			if (!DefaultObject->SupportsBlueprintBrushTool())
			{
				return false;
			}

			FScopedTransaction Transaction(LOCTEXT("LandscapeEdModeBlueprintToolSpawn", "Create landscape brush"));

			// Use the class factory if there's one :
			UActorFactory* BrushActorFactory = GEditor->FindActorFactoryForActorClass(BrushClassPtr);

			UWorld* ActorWorld = ViewportClient->GetWorld();
			ALandscapeBlueprintBrushBase* Brush = (BrushActorFactory != nullptr)
				? CastChecked<ALandscapeBlueprintBrushBase>(BrushActorFactory->CreateActor(ActorWorld, SpawnInfo.OverrideLevel, FTransform(SpawnLocation), SpawnInfo))
				: ActorWorld->SpawnActor<ALandscapeBlueprintBrushBase>(BrushClassPtr, SpawnLocation, FRotator(0.0f), SpawnInfo);
			EdMode->UISettings->BlueprintBrush = nullptr;

			Brush->SetActorLabel(BrushActorString);

			GEditor->SelectNone(true, true);
			GEditor->SelectActor(Brush, true, true);

			EdMode->RefreshDetailPanel();
		}		
		
		return true;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		return false;
	}

	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override
	{
		if (InKey == EKeys::Enter && InEvent == IE_Pressed)
		{
		}

		return false;
	}

	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override
	{
		return false;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		// The editor can try to render the tool before the UpdateLandscapeEditorData command runs and the landscape editor realizes that the landscape has been hidden/deleted
		const ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		const ALandscapeProxy* const LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		if (LandscapeProxy)
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			int32 MinX, MinY, MaxX, MaxY;
			if (LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
			{
				// TODO if required
			}
		}
	}
};

//
// Toolset initialization
//
void FEdModeLandscape::InitializeTool_BlueprintBrush()
{
	auto Tool_BlueprintBrush = MakeUnique<FLandscapeToolBlueprintBrush>(this);
	Tool_BlueprintBrush->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_BlueprintBrush));
}

#undef LOCTEXT_NAMESPACE
