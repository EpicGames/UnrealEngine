// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Materials/Material.h"
#include "Algo/Transform.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInstanceConstant.h"
#include "EngineUtils.h"
#include "EditorViewportClient.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeGizmoActor.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "LandscapeUtils.h"
#include "LandscapeRender.h"

#include "LevelUtils.h"

// 
// FLandscapeBrush
//
bool GInLandscapeBrushTransaction = false;

void FLandscapeBrush::BeginStroke(float LandscapeX, float LandscapeY, FLandscapeTool* CurrentTool)
{
	if (!GInLandscapeBrushTransaction)
	{
		GEditor->BeginTransaction(FText::Format(NSLOCTEXT("UnrealEd", "LandscapeMode_EditTransaction", "Landscape Editing: {0}"), CurrentTool->GetDisplayName()));
		GInLandscapeBrushTransaction = true;
	}
}

void FLandscapeBrush::EndStroke()
{
	if (ensure(GInLandscapeBrushTransaction))
	{
		GEditor->EndTransaction();
		GInLandscapeBrushTransaction = false;
	}
}

// 
// FLandscapeBrushCircle
//

class FLandscapeBrushCircle : public FLandscapeBrush
{
	using Super = FLandscapeBrush;

	// Components which previously were under the area of the brush.
	TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> BrushMaterialComponents;
	
	// A Cache of previously created and now unused MIDs we can reuse if required.
	TArray<TWeakObjectPtr<UMaterialInstanceDynamic>> BrushMaterialFreeInstances;

protected:
	FVector2f LastMousePosition;
	TObjectPtr<UMaterialInterface> BrushMaterial;
	TMap<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtr<UMaterialInstanceDynamic>, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtr<UMaterialInstanceDynamic>>> BrushMaterialInstanceMap;

	// TODO [jonathan.bard] : This should move one level up (in FLandscapeBrush), so that FLandscapeBrushComponent can also disable painting on the edges and with the allow list
	bool bCanPaint;

	virtual float CalculateFalloff(float Distance, float Radius, float Falloff) = 0;

	/** Protected so that only subclasses can create instances of this class. */
	FLandscapeBrushCircle(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: LastMousePosition(0, 0)
		, BrushMaterial(UE::Landscape::CreateToolLandscapeMaterialInstanceConstant(InBrushMaterial))
		, bCanPaint(false)
		, EdMode(InEdMode)
	{
	}

public:
	FEdModeLandscape* EdMode;

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(BrushMaterial);
	}

	virtual void LeaveBrush() override
	{
		for (auto It = BrushMaterialInstanceMap.CreateIterator(); It; ++It)
		{
			if (ULandscapeComponent* Component = It.Key().Get())
			{
				Component->EditToolRenderData.ToolMaterial = nullptr;
				Component->UpdateEditToolRenderData();
			}
		}

		TArray<TWeakObjectPtr<UMaterialInstanceDynamic>> BrushMaterialInstances;
		ObjectPtrDecay(BrushMaterialInstanceMap).GenerateValueArray(BrushMaterialInstances);
		BrushMaterialFreeInstances += BrushMaterialInstances;
		BrushMaterialInstanceMap.Empty();
		BrushMaterialComponents.Empty();
		bCanPaint = false;
	}

	virtual void BeginStroke(float LandscapeX, float LandscapeY, FLandscapeTool* CurrentTool) override
	{
		Super::BeginStroke(LandscapeX, LandscapeY, CurrentTool);
		LastMousePosition = FVector2f(LandscapeX, LandscapeY);
	}

	virtual bool CanPaint(const TSet<ULandscapeComponent*>& InAffectedComponents, bool bInHasUnloadedComponents) const 
	{
		if (bInHasUnloadedComponents)
		{
			return false;
		}

		if (!EdMode->CanEditSelectedTargetLayer())
		{
			return false;
		}

		for (const ULandscapeComponent* Component : InAffectedComponents)
		{
			const ALandscapeProxy* LandscapeProxy = Component ? Component->GetLandscapeProxy() : nullptr;
			if (Component == nullptr || LandscapeProxy == nullptr)
			{
				return false;
			}

			const ULandscapeLayerInfoObject* LayerInfo = EdMode->CurrentToolTarget.LayerInfo.Get();
			if (((EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap) || (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility))
				&& (EdMode->UISettings->PaintingRestriction != ELandscapeLayerPaintingRestriction::None))
			{
				if ((EdMode->UISettings->PaintingRestriction == ELandscapeLayerPaintingRestriction::UseComponentAllowList) 
					&& !Component->LayerAllowList.Contains(LayerInfo))
				{
					return false;
				}
				else
				{
					const TArray<FWeightmapLayerAllocationInfo>& ComponentWeightmapLayerAllocations = Component->GetWeightmapLayerAllocations(true);

					bool bExisting = ComponentWeightmapLayerAllocations.ContainsByPredicate([LayerInfo](const FWeightmapLayerAllocationInfo& Allocation) { return Allocation.LayerInfo == LayerInfo; });
					if (!bExisting)
					{
						if (((EdMode->UISettings->PaintingRestriction == ELandscapeLayerPaintingRestriction::ExistingOnly) || (EdMode->UISettings->PaintingRestriction == ELandscapeLayerPaintingRestriction::UseMaxLayers))
							&& (LandscapeProxy->MaxPaintedLayersPerComponent > 0) 
							&& (ComponentWeightmapLayerAllocations.Num() >= LandscapeProxy->MaxPaintedLayersPerComponent))
						{
							return false;
						}
					}
				}
			}

			if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility && !Component->IsLandscapeHoleMaterialValid())
			{
				return false;
			}
		}

		return true;
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		ALandscapeProxy* Proxy = LandscapeInfo->GetLandscapeProxy();

		const float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
		const float TotalRadius = EdMode->UISettings->GetCurrentToolBrushRadius() / ScaleXY;
		const float Radius = (1.0f - EdMode->UISettings->GetCurrentToolBrushFalloff()) * TotalRadius;
		const float Falloff = EdMode->UISettings->GetCurrentToolBrushFalloff() * TotalRadius;

		FIntRect Bounds;
		Bounds.Min.X = FMath::FloorToInt(LastMousePosition.X - TotalRadius);
		Bounds.Min.Y = FMath::FloorToInt(LastMousePosition.Y - TotalRadius);
		Bounds.Max.X = FMath::CeilToInt( LastMousePosition.X + TotalRadius);
		Bounds.Max.Y = FMath::CeilToInt( LastMousePosition.Y + TotalRadius);

		TSet<ULandscapeComponent*> NewComponents;
		bool bHasUnloadedComponents = false;
		// Adjusting the brush may use the same keybind as moving the camera as they can be user-set, so we need this second check.
		if (!ViewportClient->IsMovingCamera() || EdMode->IsAdjustingBrush(ViewportClient))
		{
			// GetComponentsInRegion expects an inclusive max
			LandscapeInfo->GetComponentsInRegion(Bounds.Min.X, Bounds.Min.Y, Bounds.Max.X - 1, Bounds.Max.Y - 1, NewComponents);
			bHasUnloadedComponents = LandscapeInfo->HasUnloadedComponentsInRegion(Bounds.Min.X, Bounds.Min.Y, Bounds.Max.X - 1, Bounds.Max.Y - 1);
		}
		
		// Store components from the previous tick
		TSet<ULandscapeComponent*> PreviousComponents;
		PreviousComponents.Reserve(BrushMaterialComponents.Num());
		Algo::TransformIf(BrushMaterialComponents, PreviousComponents, [](TWeakObjectPtr<ULandscapeComponent> Component) { return Component.IsValid(); }, [](TWeakObjectPtr<ULandscapeComponent> Component) { return Component.Get(); });

		// We're dealing with weak object ptrs. Now is a good time to remove the null entries (weak pointers can be invalidated) from our containers if any : 
		// First, remove all components (the map's keys) that might have become stale (the map's value are also weak object ptrs to materials but they will be handled right after) :
		for (auto It= BrushMaterialInstanceMap.CreateIterator(); It; ++It)
		{
			if (!It.Key().IsValid())
			{
				BrushMaterialFreeInstances.Push(It.Value());
				It.RemoveCurrent();
			}
		}

		// Now remove all material instances that might have become stale :
		for (auto It = BrushMaterialFreeInstances.CreateIterator(); It; ++It)
		{
			if (!It->IsValid())
			{
				It.RemoveCurrent();
			}
		}

		// Remove the material from any old components that are no longer in the region
		TSet<ULandscapeComponent*> RemovedComponents = PreviousComponents.Difference(NewComponents);
		for (ULandscapeComponent* RemovedComponent : RemovedComponents)
		{
			TWeakObjectPtr<UMaterialInstanceDynamic> RemovedMaterialInstance;
			if (ensure(BrushMaterialInstanceMap.RemoveAndCopyValue(RemovedComponent, RemovedMaterialInstance)))
			{
				// We should have removed stale material instances already :
				check(RemovedMaterialInstance.IsValid());
				BrushMaterialFreeInstances.Push(RemovedMaterialInstance);
			}

			RemovedComponent->EditToolRenderData.ToolMaterial = nullptr;
			RemovedComponent->UpdateEditToolRenderData();
		}

		// Set brush material for components in new region
		TSet<ULandscapeComponent*> AddedComponents = NewComponents.Difference(PreviousComponents);
		for (ULandscapeComponent* AddedComponent : AddedComponents)
		{
			UMaterialInstanceDynamic* BrushMaterialInstance = nullptr;

			while (BrushMaterialFreeInstances.Num() > 0)
			{
				if (auto FreeBrushMaterialInstance = BrushMaterialFreeInstances.Pop(); FreeBrushMaterialInstance.IsValid())
				{
					BrushMaterialInstance = FreeBrushMaterialInstance.Get();
					break;
				}
			}
		
			if (!BrushMaterialInstance)
			{
				BrushMaterialInstance = UMaterialInstanceDynamic::Create(BrushMaterial, nullptr);
			}
			BrushMaterialInstanceMap.Add(AddedComponent, BrushMaterialInstance);
			AddedComponent->EditToolRenderData.ToolMaterial = BrushMaterialInstance;
			AddedComponent->UpdateEditToolRenderData();
		}

		BrushMaterialComponents.Empty(NewComponents.Num());
		Algo::Transform(NewComponents, BrushMaterialComponents, [](ULandscapeComponent* Component) { return TWeakObjectPtr<ULandscapeComponent>(Component); });

		// Set params for brush material.
		FVector WorldLocation = Proxy->LandscapeActorToWorld().TransformPosition(FVector(LastMousePosition.X, LastMousePosition.Y, 0));

		// Update whether the brush can actually paint : 
		bCanPaint = CanPaint(NewComponents, bHasUnloadedComponents);

		for (const auto& BrushMaterialInstancePair : BrushMaterialInstanceMap)
		{
			
			ULandscapeComponent* const Component = BrushMaterialInstancePair.Key.Get();
			UMaterialInstanceDynamic* const MaterialInstance = BrushMaterialInstancePair.Value.Get();

			if (Component && MaterialInstance)
			{
				// Painting can cause the EditToolRenderData to be destructed, so update it if necessary
				if (!AddedComponents.Contains(Component))
				{
					if (Component->EditToolRenderData.ToolMaterial == nullptr)
					{
						Component->EditToolRenderData.ToolMaterial = MaterialInstance;
						Component->UpdateEditToolRenderData();
					}
				}

				MaterialInstance->SetScalarParameterValue(FName(TEXT("LocalRadius")), Radius);
				MaterialInstance->SetScalarParameterValue(FName(TEXT("LocalFalloff")), Falloff);
				MaterialInstance->SetDoubleVectorParameterValue(FName(TEXT("WorldPosition")), FVector4(WorldLocation.X, WorldLocation.Y, WorldLocation.Z, ScaleXY));
				MaterialInstance->SetScalarParameterValue("CanPaint", bCanPaint ? 1.0f : 0.0f);
			}
		}
	}

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
		LastMousePosition = FVector2f(LandscapeX, LandscapeY);
	}

	virtual FLandscapeBrush::FOverlapInfo GetOverlapInfo(const TArray<FLandscapeToolInteractorPosition>& InInteractorPositions) const override
	{
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		const float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
		const float TotalRadius = EdMode->UISettings->GetCurrentToolBrushRadius() / ScaleXY;

		TArray<FLandscapeToolInteractorPosition> InteractorPositions = CapInteractorPositions(InInteractorPositions);
		FIntRect ExclusiveBounds;
		TSet<ULandscapeComponent*> OverlappedComponents;
		for (const FLandscapeToolInteractorPosition& InteractorPosition : InteractorPositions)
		{
			FIntRect SpotExclusiveBounds;
			SpotExclusiveBounds.Min.X = FMath::FloorToInt32(InteractorPosition.Position.X - TotalRadius);
			SpotExclusiveBounds.Min.Y = FMath::FloorToInt32(InteractorPosition.Position.Y - TotalRadius);
			SpotExclusiveBounds.Max.X = FMath::CeilToInt32(InteractorPosition.Position.X + TotalRadius) + 1;
			SpotExclusiveBounds.Max.Y = FMath::CeilToInt32(InteractorPosition.Position.Y + TotalRadius) + 1;

			if (ExclusiveBounds.IsEmpty())
			{
				ExclusiveBounds = SpotExclusiveBounds;
			}
			else
			{
				ExclusiveBounds.Union(SpotExclusiveBounds);
			}

			// Shrink bounds by 1,1 to avoid GetComponentsInRegion picking up extra components on all sides due to the overlap between components
			TSet<ULandscapeComponent*> SpotOverlappedComponents;
			LandscapeInfo->GetComponentsInRegion(SpotExclusiveBounds.Min.X + 1, SpotExclusiveBounds.Min.Y + 1, SpotExclusiveBounds.Max.X - 1, SpotExclusiveBounds.Max.Y - 1, SpotOverlappedComponents);
			OverlappedComponents.Append(SpotOverlappedComponents);
		}

		// Clamp to landscape bounds.  GetLandscapeExtent provides inclusive bounds.
		int32 MinX, MaxX, MinY, MaxY;
		if (!ensure(LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY)))
		{
			// Landscape has no components somehow
			return FLandscapeBrush::FOverlapInfo();
		}
		// Clip using exclusive bounds so we can detect empty result.
		ExclusiveBounds.Clip(FIntRect(MinX, MinY, MaxX + 1, MaxY + 1));
		if (ExclusiveBounds.Area() <= 0)
		{
			// Brush is completely outside the landscape extent
			return FLandscapeBrush::FOverlapInfo();
		}

		FLandscapeBrush::FOverlapInfo Result;
		Result.ExclusiveBounds = ExclusiveBounds;
		Result.OverlappedLoadedComponents = OverlappedComponents;

		// check if region to be modified overlaps with an unloaded component
		// TODO [chris.tchou] : The exclusive bounds here (InclusiveMax+1) shouldn't be necessary since the code looks like it expects inclusive values.  Without it, edges can be modified by the brush. Possibly a bug in the brush apply code.
		Result.bOverlapsUnloadedComponents = LandscapeInfo->HasUnloadedComponentsInRegion(ExclusiveBounds.Min.X, ExclusiveBounds.Min.Y, ExclusiveBounds.Max.X, ExclusiveBounds.Max.Y);

		return Result;
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InInteractorPositions) override
	{
		// bCanPaint is evaluated at an old mouse position, and also does not take interactor positions into account,
		// so cannot be relied upon to be correct vis-a-vis overlapping unloaded components.  We check those again below.
		if (!bCanPaint)
		{
			return FLandscapeBrushData();
		}

		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		const float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
		const float TotalRadius = EdMode->UISettings->GetCurrentToolBrushRadius() / ScaleXY;
		const float Radius = (1.0f - EdMode->UISettings->GetCurrentToolBrushFalloff()) * TotalRadius;
		const float Falloff = EdMode->UISettings->GetCurrentToolBrushFalloff() * TotalRadius;

		FLandscapeBrush::FOverlapInfo OverlapInfo = GetOverlapInfo(InInteractorPositions);
		// Disable brush if we don't overlap with anything or if it overlaps with some unloaded components (to prevent data from differing on the edges)
		if (OverlapInfo.ExclusiveBounds.IsEmpty() || OverlapInfo.bOverlapsUnloadedComponents)
		{
			return FLandscapeBrushData();
		}

		// Cap number of mouse positions to a sensible number
		TArray<FLandscapeToolInteractorPosition> InteractorPositions = CapInteractorPositions(InInteractorPositions);

		FLandscapeBrushData BrushData(OverlapInfo.ExclusiveBounds);

		for (const FLandscapeToolInteractorPosition& InteractorPosition : InteractorPositions)
		{
			FIntRect SpotExclusiveBounds;
			SpotExclusiveBounds.Min.X = FMath::FloorToInt32(InteractorPosition.Position.X - TotalRadius);
			SpotExclusiveBounds.Min.Y = FMath::FloorToInt32(InteractorPosition.Position.Y - TotalRadius);
			SpotExclusiveBounds.Max.X = FMath::CeilToInt32(InteractorPosition.Position.X + TotalRadius) + 1;
			SpotExclusiveBounds.Max.Y = FMath::CeilToInt32(InteractorPosition.Position.Y + TotalRadius) + 1;
			SpotExclusiveBounds.Clip(OverlapInfo.ExclusiveBounds);

			for (int32 Y = SpotExclusiveBounds.Min.Y; Y < SpotExclusiveBounds.Max.Y; Y++)
			{
				float* Scanline = BrushData.GetDataPtr(FIntPoint(0, Y));

				for (int32 X = SpotExclusiveBounds.Min.X; X < SpotExclusiveBounds.Max.X; X++)
				{
					float PrevAmount = Scanline[X];
					if (PrevAmount < 1.0f)
					{
						// Distance from mouse
						float MouseDist = static_cast<float>(FMath::Sqrt(FMath::Square(InteractorPosition.Position.X - X) + FMath::Square(InteractorPosition.Position.Y - Y)));

						float PaintAmount = CalculateFalloff(MouseDist, Radius, Falloff);

						if (PaintAmount > 0.0f)
						{
							if (EdMode->CurrentTool && EdMode->CurrentTool->GetToolType() != ELandscapeToolType::Mask
								&& EdMode->UISettings->bUseSelectedRegion && LandscapeInfo->SelectedRegion.Num() > 0)
							{
								float MaskValue = LandscapeInfo->SelectedRegion.FindRef(FIntPoint(X, Y));
								if (EdMode->UISettings->bUseNegativeMask)
								{
									MaskValue = 1.0f - MaskValue;
								}
								PaintAmount *= MaskValue;
							}

							if (PaintAmount > PrevAmount)
							{
								// Set the brush value for this vertex
								Scanline[X] = PaintAmount;
							}
						}
					}
				}
			}
		}

		return BrushData;
	}

private:
	static TArray<FLandscapeToolInteractorPosition> CapInteractorPositions(const TArray<FLandscapeToolInteractorPosition>& InInteractorPositions)
	{
		// Cap number of mouse positions to a sensible number
		static constexpr int32 MaxInteractorPositions = 10;
		TArray<FLandscapeToolInteractorPosition> OutInteractorPositions;
		if (InInteractorPositions.Num() > MaxInteractorPositions)
		{
			OutInteractorPositions.Reserve(MaxInteractorPositions);
			for (int32 i = 0; i < MaxInteractorPositions; ++i)
			{
				// Scale so we include the first and last of the input positions
				OutInteractorPositions.Add(InInteractorPositions[(i * (InInteractorPositions.Num() - 1)) / (MaxInteractorPositions - 1)]);
			}
		}
		else
		{
			OutInteractorPositions = InInteractorPositions;
		}
		return OutInteractorPositions;
	}
};

// 
// FLandscapeBrushComponent
//

class FLandscapeBrushComponent : public FLandscapeBrush
{
	using Super = FLandscapeBrush;

	TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> BrushMaterialComponents;

	virtual const TCHAR* GetBrushName() override { return TEXT("Component"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Component", "Component"); };

protected:
	FVector2D LastMousePosition;
	TObjectPtr<UMaterialInterface> BrushMaterial;
	FIntRect BrushExtentsInclusive; // True extents of the brush (in landscape coordinates)

public:
	FEdModeLandscape* EdMode;

	FLandscapeBrushComponent(FEdModeLandscape* InEdMode)
		: BrushMaterial(nullptr)
		, EdMode(InEdMode)
	{
		UMaterial* BaseBrushMaterial = LoadObject<UMaterial>(nullptr, TEXT("/Engine/EditorLandscapeResources/SelectBrushMaterial.SelectBrushMaterial"));
		BrushMaterial = UE::Landscape::CreateToolLandscapeMaterialInstanceConstant(BaseBrushMaterial);
	}

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(BrushMaterial);
	}

	virtual ELandscapeBrushType GetBrushType() override { return ELandscapeBrushType::Component; }

	virtual void LeaveBrush() override
	{
		for (TWeakObjectPtr<ULandscapeComponent> ComponentWeakPtr : BrushMaterialComponents)
		{
			if (ULandscapeComponent* Component = ComponentWeakPtr.Get())
			{
				Component->EditToolRenderData.ToolMaterial = nullptr;
				Component->UpdateEditToolRenderData();
			}
		}
		BrushMaterialComponents.Empty();
	}

	virtual void BeginStroke(float LandscapeX, float LandscapeY, FLandscapeTool* CurrentTool) override
	{
		Super::BeginStroke(LandscapeX, LandscapeY, CurrentTool);
		LastMousePosition = FVector2D(LandscapeX, LandscapeY);
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> NewComponents;

		// Adjusting the brush may use the same keybind as moving the camera as they can be user-set, so we need this second check.
		if (!ViewportClient->IsMovingCamera() || EdMode->IsAdjustingBrush(ViewportClient))
		{
			ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
			if (LandscapeInfo && LandscapeInfo->ComponentSizeQuads > 0)
			{
				const int32 BrushSize = FMath::Max(EdMode->UISettings->BrushComponentSize, 0);

				const float BrushOriginX = static_cast<float>(LastMousePosition.X / LandscapeInfo->ComponentSizeQuads - (BrushSize - 1) / 2.0f);
				const float BrushOriginY = static_cast<float>(LastMousePosition.Y / LandscapeInfo->ComponentSizeQuads - (BrushSize - 1) / 2.0f);
				const int32 ComponentIndexX = FMath::FloorToInt(BrushOriginX);
				const int32 ComponentIndexY = FMath::FloorToInt(BrushOriginY);
				BrushExtentsInclusive.Min = FIntPoint(ComponentIndexX * LandscapeInfo->ComponentSizeQuads, ComponentIndexY * LandscapeInfo->ComponentSizeQuads);
				BrushExtentsInclusive.Max = FIntPoint((ComponentIndexX + BrushSize) * LandscapeInfo->ComponentSizeQuads + 1, (ComponentIndexY + BrushSize) * LandscapeInfo->ComponentSizeQuads + 1);

				for (int32 YIndex = 0; YIndex < BrushSize; ++YIndex)
				{
					for (int32 XIndex = 0; XIndex < BrushSize; ++XIndex)
					{
						ULandscapeComponent* Component = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint((ComponentIndexX + XIndex), (ComponentIndexY + YIndex)));
						if (Component && FLevelUtils::IsLevelVisible(Component->GetLandscapeProxy()->GetLevel()))
						{
							// For MoveToLevel
							if (EdMode->CurrentTool->GetToolName() == FName("MoveToLevel"))
							{
								if (Component->GetLandscapeProxy() && !Component->GetLandscapeProxy()->GetLevel()->IsCurrentLevel())
								{
									NewComponents.Add(Component);
								}
							}
							else
							{
								NewComponents.Add(Component);
							}
						}
					}
				}

				// Set brush material for components in new region
				for (TWeakObjectPtr<ULandscapeComponent> NewComponent : NewComponents)
				{
					if (NewComponent.IsValid())
					{
						NewComponent->EditToolRenderData.ToolMaterial = BrushMaterial;
						NewComponent->UpdateEditToolRenderData();
					}
				}
			}
		}

		// Remove the material from any old components that are no longer in the region
		TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> RemovedComponents = BrushMaterialComponents.Difference(NewComponents);
		for (TWeakObjectPtr<ULandscapeComponent> RemovedComponent : RemovedComponents)
		{
			if (RemovedComponent.IsValid())
			{
				RemovedComponent->EditToolRenderData.ToolMaterial = nullptr;
				RemovedComponent->UpdateEditToolRenderData();
			}
		}

		BrushMaterialComponents = MoveTemp(NewComponents);
	}

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
		LastMousePosition = FVector2D(LandscapeX, LandscapeY);
	}

	virtual TOptional<FVector2D> GetLastMousePosition() const override
	{ 
		return LastMousePosition;
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		// Selection Brush only works for 
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();

		FIntRect ExclusiveBounds(INT_MAX, INT_MAX, INT_MIN, INT_MIN);

		// The add component tool needs the raw bounds of the brush rather than the bounds of the actually existing components under the brush
		if (EdMode->CurrentTool->GetToolName() == FName("AddComponent"))
		{
			const int32 BrushSize = FMath::Max(EdMode->UISettings->BrushComponentSize, 0);

			const float BrushOriginX = static_cast<float>(LastMousePosition.X / LandscapeInfo->ComponentSizeQuads - (BrushSize - 1) / 2.0f);
			const float BrushOriginY = static_cast<float>(LastMousePosition.Y / LandscapeInfo->ComponentSizeQuads - (BrushSize - 1) / 2.0f);
			const int32 ComponentIndexX = FMath::FloorToInt(BrushOriginX);
			const int32 ComponentIndexY = FMath::FloorToInt(BrushOriginY);

			ExclusiveBounds.Min.X = (ComponentIndexX) * LandscapeInfo->ComponentSizeQuads;
			ExclusiveBounds.Min.Y = (ComponentIndexY) * LandscapeInfo->ComponentSizeQuads;
			ExclusiveBounds.Max.X = (ComponentIndexX + BrushSize) * LandscapeInfo->ComponentSizeQuads + 1;
			ExclusiveBounds.Max.Y = (ComponentIndexY + BrushSize) * LandscapeInfo->ComponentSizeQuads + 1;
		}
		else
		{
			// Make sure to remove all stale weak ptrs (in case ApplyBrush is called twice before a Tick) 
			for (auto It = BrushMaterialComponents.CreateIterator(); It; ++It)
			{
				if (!It->IsValid())
				{
					It.RemoveCurrent();
				}
			}
			if (BrushMaterialComponents.Num() == 0)
			{
				return FLandscapeBrushData();
			}

			// Get extent for all components
			for (TWeakObjectPtr<ULandscapeComponent> Component : BrushMaterialComponents)
			{
				check(Component.IsValid());
				Component->GetComponentExtent(ExclusiveBounds.Min.X, ExclusiveBounds.Min.Y, ExclusiveBounds.Max.X, ExclusiveBounds.Max.Y);
			}

			check((ExclusiveBounds.Min.X != INT_MAX) && (ExclusiveBounds.Min.Y != INT_MAX) && (ExclusiveBounds.Max.X != INT_MIN) && (ExclusiveBounds.Max.Y != INT_MIN));
			
			// GetComponentExtent returns an inclusive max bound
			ExclusiveBounds.Max += FIntPoint(1, 1);
		}

		FLandscapeBrushData BrushData(ExclusiveBounds);

		for (int32 Y = ExclusiveBounds.Min.Y; Y < ExclusiveBounds.Max.Y; Y++)
		{
			float* Scanline = BrushData.GetDataPtr(FIntPoint(0, Y));

			for (int32 X = ExclusiveBounds.Min.X; X < ExclusiveBounds.Max.X; X++)
			{
				const int32 ComponentIndexX = FMath::FloorToInt(static_cast<float>(X) / LandscapeInfo->ComponentSizeQuads);
				const int32 ComponentIndexY = FMath::FloorToInt(static_cast<float>(Y) / LandscapeInfo->ComponentSizeQuads);
				// Skip the pixels from the border if requested :
				if (EdMode->UISettings->bBrushComponentIncludeBorder 
					|| ((X > BrushExtentsInclusive.Min.X)
						&& (X < (BrushExtentsInclusive.Max.X - 1))
						&& (Y > BrushExtentsInclusive.Min.Y)
						&& (Y < (BrushExtentsInclusive.Max.Y - 1))))
				{
					float PaintAmount = 1.0f;
					if (EdMode->CurrentTool && EdMode->CurrentTool->GetToolType() != ELandscapeToolType::Mask
						&& EdMode->UISettings->bUseSelectedRegion && LandscapeInfo->SelectedRegion.Num() > 0)
					{
						float MaskValue = LandscapeInfo->SelectedRegion.FindRef(FIntPoint(X, Y));
						if (EdMode->UISettings->bUseNegativeMask)
						{
							MaskValue = 1.0f - MaskValue;
						}
						PaintAmount *= MaskValue;
					}

					// Set the brush value for this vertex
					Scanline[X] = PaintAmount;
				}
			}
		}

		return BrushData;
	}
};

// 
// FLandscapeBrushGizmo
//

class FLandscapeBrushGizmo : public FLandscapeBrush
{
	using Super = FLandscapeBrush;

	TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> BrushMaterialComponents;

	const TCHAR* GetBrushName() override { return TEXT("Gizmo"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Gizmo", "Gizmo"); };

protected:
	TObjectPtr<UMaterialInstanceDynamic> BrushMaterial;
public:
	FEdModeLandscape* EdMode;

	FLandscapeBrushGizmo(FEdModeLandscape* InEdMode)
		: BrushMaterial(nullptr)
		, EdMode(InEdMode)
	{
		UMaterialInterface* GizmoMaterial = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/MaskBrushMaterial_Gizmo.MaskBrushMaterial_Gizmo"));
		BrushMaterial = UMaterialInstanceDynamic::Create(UE::Landscape::CreateToolLandscapeMaterialInstanceConstant(GizmoMaterial), nullptr);
	}

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(BrushMaterial);
	}

	virtual ELandscapeBrushType GetBrushType() override { return ELandscapeBrushType::Gizmo; }

	virtual void EnterBrush() override
	{
		// Make sure gizmo actor is selected
		ALandscapeGizmoActiveActor* Gizmo = EdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo)
		{
			GEditor->SelectNone(false, true);
			GEditor->SelectActor(Gizmo, true, false, true);
		}
	}

	virtual void LeaveBrush() override
	{
		for (TWeakObjectPtr<ULandscapeComponent> ComponentWeakPtr : BrushMaterialComponents)
		{
			if (ULandscapeComponent* Component = ComponentWeakPtr.Get())
			{
				Component->EditToolRenderData.ToolMaterial = nullptr;
				Component->UpdateEditToolRenderData();
			}
		}
		BrushMaterialComponents.Empty();
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		if (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo || GLandscapeEditRenderMode & ELandscapeEditRenderMode::Select)
		{
			ALandscapeGizmoActiveActor* Gizmo = EdMode->UISettings->CurrentGizmoActor.Get();

			if (Gizmo && Gizmo->TargetLandscapeInfo.IsValid() && (Gizmo->TargetLandscapeInfo == EdMode->CurrentToolTarget.LandscapeInfo.Get()) && Gizmo->GizmoTexture && Gizmo->GetRootComponent())
			{
				ULandscapeInfo* LandscapeInfo = Gizmo->TargetLandscapeInfo.Get();
				if (LandscapeInfo && LandscapeInfo->GetLandscapeProxy())
				{
					float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
					FMatrix LToW = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().ToMatrixWithScale();
					FMatrix WToL = LToW.InverseFast();

					UTexture2D* DataTexture = Gizmo->GizmoTexture;
					FIntRect Bounds(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
					FVector LocalPos[4];
					//FMatrix WorldToLocal = Proxy->LocalToWorld().Inverse();
					for (int32 i = 0; i < 4; ++i)
					{
						//LocalPos[i] = WorldToLocal.TransformPosition(Gizmo->FrustumVerts[i]);
						LocalPos[i] = WToL.TransformPosition(Gizmo->FrustumVerts[i]);
						Bounds.Min.X = FMath::Min(Bounds.Min.X, (int32)LocalPos[i].X);
						Bounds.Min.Y = FMath::Min(Bounds.Min.Y, (int32)LocalPos[i].Y);
						Bounds.Max.X = FMath::Max(Bounds.Max.X, (int32)LocalPos[i].X);
						Bounds.Max.Y = FMath::Max(Bounds.Max.Y, (int32)LocalPos[i].Y);
					}

					// GetComponentsInRegion expects an inclusive max
					TSet<ULandscapeComponent*> NewComponents;
					LandscapeInfo->GetComponentsInRegion(Bounds.Min.X, Bounds.Min.Y, Bounds.Max.X - 1, Bounds.Max.Y - 1, NewComponents);

					float SquaredScaleXY = FMath::Square(ScaleXY);
					FLinearColor AlphaScaleBias(
						SquaredScaleXY / FMath::Max<float>(1.f, Gizmo->GetWidth() * DataTexture->GetSizeX()),
						SquaredScaleXY / FMath::Max<float>(1.f, Gizmo->GetHeight() * DataTexture->GetSizeY()),
						static_cast<float>(Gizmo->TextureScale.X),
						static_cast<float>(Gizmo->TextureScale.Y)
						);
					BrushMaterial->SetVectorParameterValue(FName(TEXT("AlphaScaleBias")), AlphaScaleBias);

					float Angle = static_cast<float>(-EdMode->UISettings->CurrentGizmoActor->GetActorRotation().Euler().Z * PI / 180.0f);
					FVector4 LandscapeLocation(EdMode->UISettings->CurrentGizmoActor->GetActorLocation().X, EdMode->UISettings->CurrentGizmoActor->GetActorLocation().Y, EdMode->UISettings->CurrentGizmoActor->GetActorLocation().Z, Angle);

					BrushMaterial->SetDoubleVectorParameterValue(FName(TEXT("LandscapeLocation")), LandscapeLocation);
					BrushMaterial->SetTextureParameterValue(FName(TEXT("AlphaTexture")), DataTexture);

					// Set brush material for components in new region
					for (ULandscapeComponent* NewComponent : NewComponents)
					{
						NewComponent->EditToolRenderData.GizmoMaterial = ((Gizmo->DataType != LGT_None) && (GLandscapeEditRenderMode & ELandscapeEditRenderMode::Gizmo) ? BrushMaterial : nullptr);
						NewComponent->UpdateEditToolRenderData();
					}

					// Convert NewComponents to TWeakObjectPtr TSet
					TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> NewComponentsWeakSet;
					NewComponentsWeakSet.Reserve(NewComponents.Num());
					Algo::Transform(NewComponents, NewComponentsWeakSet, [](ULandscapeComponent* Component) { return TWeakObjectPtr<ULandscapeComponent>(Component); });

					// Remove the material from any old components that are no longer in the region
					TSet<TWeakObjectPtr<ULandscapeComponent>, TWeakObjectPtrSetKeyFuncs<TWeakObjectPtr<ULandscapeComponent>>> RemovedComponents = BrushMaterialComponents.Difference(NewComponentsWeakSet);
					for (TWeakObjectPtr<ULandscapeComponent> RemovedComponent : RemovedComponents)
					{
						if (RemovedComponent.IsValid())
						{
							RemovedComponent->EditToolRenderData.GizmoMaterial = nullptr;
							RemovedComponent->UpdateEditToolRenderData();
						}
					}

					BrushMaterialComponents = MoveTemp(NewComponentsWeakSet);
				}
			}
		}
	}

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
	}

	virtual TOptional<bool> InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override
	{

		if (InKey == EKeys::LeftMouseButton && InEvent == IE_Pressed)
		{
			int32 HitX = InViewport->GetMouseX();
			int32 HitY = InViewport->GetMouseY();
			HHitProxy* HitProxy = InViewport->GetHitProxy(HitX, HitY);

			HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy);
			if (ActorHitProxy && ActorHitProxy->Actor->IsA<ALandscapeGizmoActor>())
			{
				// don't treat clicks on a landscape gizmo as a tool invocation
				return TOptional<bool>(false);
			}
		}

		// default behaviour
		return TOptional<bool>();
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		// Selection Brush only works for 
		ALandscapeGizmoActiveActor* Gizmo = EdMode->UISettings->CurrentGizmoActor.Get();
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();

		if (!Gizmo || !Gizmo->GetRootComponent())
		{
			return FLandscapeBrushData();
		}

		Gizmo->TargetLandscapeInfo = LandscapeInfo;
		float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));

		// Get extent for all components
		FIntRect ExclusiveBounds(INT_MAX, INT_MAX, INT_MIN, INT_MIN);
		
		// Make sure to remove all stale weak ptrs (in case ApplyBrush is called twice before a Tick) 
		for (auto It = BrushMaterialComponents.CreateIterator(); It; ++It)
		{
			if (!It->IsValid())
			{
				It.RemoveCurrent();
			}
		}
		if (BrushMaterialComponents.Num() == 0)
		{
			return FLandscapeBrushData();
		}
		
		for (TWeakObjectPtr<ULandscapeComponent> Component : BrushMaterialComponents)
		{
			check(Component.IsValid());
			Component->GetComponentExtent(ExclusiveBounds.Min.X, ExclusiveBounds.Min.Y, ExclusiveBounds.Max.X, ExclusiveBounds.Max.Y);
		}

		check((ExclusiveBounds.Min.X != INT_MAX) && (ExclusiveBounds.Min.Y != INT_MAX) && (ExclusiveBounds.Max.X != INT_MIN) && (ExclusiveBounds.Max.Y != INT_MIN));
		
		// GetComponentExtent returns an inclusive max bound
		ExclusiveBounds.Max += FIntPoint(1, 1);
		
		FLandscapeBrushData BrushData(ExclusiveBounds);

		//FMatrix LandscapeToGizmoLocal = Landscape->LocalToWorld() * Gizmo->WorldToLocal();
		const float LW = Gizmo->GetWidth() / (2 * ScaleXY);
		const float LH = Gizmo->GetHeight() / (2 * ScaleXY);

		FMatrix WToL = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().ToMatrixWithScale().InverseFast();
		FVector BaseLocation = WToL.TransformPosition(Gizmo->GetActorLocation());
		FMatrix LandscapeToGizmoLocal =
			(FTranslationMatrix(FVector(-LW + 0.5, -LH + 0.5, 0)) * FRotationTranslationMatrix(FRotator(0, Gizmo->GetActorRotation().Yaw, 0), FVector(BaseLocation.X, BaseLocation.Y, 0))).InverseFast();

		float W = Gizmo->GetWidth() / ScaleXY; //Gizmo->GetWidth() / (Gizmo->DrawScale * Gizmo->DrawScale3D.X);
		float H = Gizmo->GetHeight() / ScaleXY; //Gizmo->GetHeight() / (Gizmo->DrawScale * Gizmo->DrawScale3D.Y);

		for (int32 Y = ExclusiveBounds.Min.Y; Y < ExclusiveBounds.Max.Y; Y++)
		{
			float* Scanline = BrushData.GetDataPtr(FIntPoint(0, Y));

			for (int32 X = ExclusiveBounds.Min.X; X < ExclusiveBounds.Max.X; X++)
			{
				FVector GizmoLocal = LandscapeToGizmoLocal.TransformPosition(FVector(X, Y, 0));
				if (GizmoLocal.X < W && GizmoLocal.X > 0 && GizmoLocal.Y < H && GizmoLocal.Y > 0)
				{
					float PaintAmount = 1.0f;
					// Transform in 0,0 origin LW radius
					if (EdMode->UISettings->bSmoothGizmoBrush)
					{
						FVector TransformedLocal(FMath::Abs(GizmoLocal.X - LW), FMath::Abs(GizmoLocal.Y - LH) * (W / H), 0);
						float FalloffRadius = LW * EdMode->UISettings->GetCurrentToolBrushFalloff();
						float SquareRadius = LW - FalloffRadius;
						float Cos = static_cast<float>(FMath::Abs(TransformedLocal.X) / TransformedLocal.Size2D());
						float Sin = static_cast<float>(FMath::Abs(TransformedLocal.Y) / TransformedLocal.Size2D());
						float RatioX = FalloffRadius > 0.0f ? 1.0f - FMath::Clamp<float>(static_cast<float>((FMath::Abs(TransformedLocal.X) - Cos*SquareRadius) / FalloffRadius), 0.0f, 1.0f) : 1.0f;
						float RatioY = FalloffRadius > 0.0f ? 1.0f - FMath::Clamp<float>(static_cast<float>((FMath::Abs(TransformedLocal.Y) - Sin*SquareRadius) / FalloffRadius), 0.0f, 1.0f) : 1.0f;
						float Ratio = TransformedLocal.Size2D() > SquareRadius ? RatioX * RatioY : 1.0f; //TransformedLocal.X / LW * TransformedLocal.Y / LW;
						PaintAmount = Ratio*Ratio*(3 - 2 * Ratio); //FMath::Lerp(SquareFalloff, RectFalloff*RectFalloff, Ratio);
					}

					if (PaintAmount)
					{
						if (EdMode->CurrentTool && EdMode->CurrentTool->GetToolType() != ELandscapeToolType::Mask
							&& EdMode->UISettings->bUseSelectedRegion && LandscapeInfo->SelectedRegion.Num() > 0)
						{
							float MaskValue = LandscapeInfo->SelectedRegion.FindRef(FIntPoint(X, Y));
							if (EdMode->UISettings->bUseNegativeMask)
							{
								MaskValue = 1.0f - MaskValue;
							}
							PaintAmount *= MaskValue;
						}

						// Set the brush value for this vertex
						Scanline[X] = PaintAmount;
					}
				}
			}
		}

		return BrushData;
	}
};

// 
// FLandscapeBrushSplines
//
class FLandscapeBrushSplines : public FLandscapeBrush
{
	using Super = FLandscapeBrush;

public:
	const TCHAR* GetBrushName() override { return TEXT("Splines"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Splines", "Splines"); };

	FEdModeLandscape* EdMode;

	FLandscapeBrushSplines(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
	{
	}

	virtual ~FLandscapeBrushSplines()
	{
	}

	virtual ELandscapeBrushType GetBrushType() override { return ELandscapeBrushType::Splines; }

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		return FLandscapeBrushData();
	}
};

// 
// FLandscapeBrushDummy
//
class FLandscapeBrushDummy : public FLandscapeBrush
{
	using Super = FLandscapeBrush;

public:
	const TCHAR* GetBrushName() override { return TEXT("Circle_Dummy"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_None", "None"); };

	FEdModeLandscape* EdMode;

	FLandscapeBrushDummy(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
	{
	}

	virtual ~FLandscapeBrushDummy()
	{
	}

	virtual ELandscapeBrushType GetBrushType() override { return ELandscapeBrushType::Normal; }

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		return FLandscapeBrushData();
	}
};

class FLandscapeBrushCircle_Linear : public FLandscapeBrushCircle
{
	using Super = FLandscapeBrushCircle;

protected:
	/** Protected so only subclasses can create instances. */
	FLandscapeBrushCircle_Linear(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushCircle(InEdMode, InBrushMaterial)
	{
	}

	virtual float CalculateFalloff(float Distance, float Radius, float Falloff) override
	{
		return Distance < Radius ? 1.0f :
			Falloff > 0.0f ? FMath::Max<float>(0.0f, 1.0f - (Distance - Radius) / Falloff) :
			0.0f;
	}

public:
	static FLandscapeBrushCircle_Linear* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* CircleBrushMaterial_Linear = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/CircleBrushMaterial_Linear.CircleBrushMaterial_Linear"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushCircle_Linear(InEdMode, CircleBrushMaterial_Linear);
	}
	virtual const TCHAR* GetBrushName() override { return TEXT("Circle_Linear"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Falloff_Linear", "Linear falloff"); };

};

class FLandscapeBrushCircle_Smooth : public FLandscapeBrushCircle_Linear
{
	using Super = FLandscapeBrushCircle_Linear;

protected:
	FLandscapeBrushCircle_Smooth(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushCircle_Linear(InEdMode, InBrushMaterial)
	{
	}

	virtual float CalculateFalloff(float Distance, float Radius, float Falloff) override
	{
		float Y = Super::CalculateFalloff(Distance, Radius, Falloff);
		// Smooth-step it
		return Y * Y * (3 - 2 * Y);
	}

public:
	static FLandscapeBrushCircle_Smooth* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* CircleBrushMaterial_Smooth = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/CircleBrushMaterial_Smooth.CircleBrushMaterial_Smooth"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushCircle_Smooth(InEdMode, CircleBrushMaterial_Smooth);
	}
	virtual const TCHAR* GetBrushName() override { return TEXT("Circle_Smooth"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Falloff_Smooth", "Smooth falloff"); };

};

class FLandscapeBrushCircle_Spherical : public FLandscapeBrushCircle
{
	using Super = FLandscapeBrushCircle;

protected:
	FLandscapeBrushCircle_Spherical(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushCircle(InEdMode, InBrushMaterial)
	{
	}

	virtual float CalculateFalloff(float Distance, float Radius, float Falloff) override
	{
		if (Distance <= Radius)
		{
			return 1.0f;
		}

		if (Distance > Radius + Falloff)
		{
			return 0.0f;
		}

		// Elliptical falloff
		return FMath::Sqrt(1.0f - FMath::Square((Distance - Radius) / Falloff));
	}

public:
	static FLandscapeBrushCircle_Spherical* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* CircleBrushMaterial_Spherical = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/CircleBrushMaterial_Spherical.CircleBrushMaterial_Spherical"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushCircle_Spherical(InEdMode, CircleBrushMaterial_Spherical);
	}
	virtual const TCHAR* GetBrushName() override { return TEXT("Circle_Spherical"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Falloff_Spherical", "Spherical falloff"); };
};

class FLandscapeBrushCircle_Tip : public FLandscapeBrushCircle
{
	using Super = FLandscapeBrushCircle_Tip;

protected:
	FLandscapeBrushCircle_Tip(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushCircle(InEdMode, InBrushMaterial)
	{
	}

	virtual float CalculateFalloff(float Distance, float Radius, float Falloff) override
	{
		if (Distance <= Radius)
		{
			return 1.0f;
		}

		if (Distance > Radius + Falloff)
		{
			return 0.0f;
		}

		// inverse elliptical falloff
		return 1.0f - FMath::Sqrt(1.0f - FMath::Square((Falloff + Radius - Distance) / Falloff));
	}

public:
	static FLandscapeBrushCircle_Tip* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* CircleBrushMaterial_Tip = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/CircleBrushMaterial_Tip.CircleBrushMaterial_Tip"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushCircle_Tip(InEdMode, CircleBrushMaterial_Tip);
	}
	virtual const TCHAR* GetBrushName() override { return TEXT("Circle_Tip"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Falloff_Tip", "Tip falloff"); };
};


// FLandscapeBrushAlphaBase
class FLandscapeBrushAlphaBase : public FLandscapeBrushCircle_Smooth
{
	using Super = FLandscapeBrushCircle_Smooth;

public:
	FLandscapeBrushAlphaBase(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushCircle_Smooth(InEdMode, InBrushMaterial)
	{
	}

	float GetAlphaSample(float SampleX, float SampleY)
	{
		if (!EdMode->UISettings->HasValidAlphaTextureData())
		{
			return 1.0f;
		}

		int32 SizeX = EdMode->UISettings->AlphaTextureSizeX;
		int32 SizeY = EdMode->UISettings->AlphaTextureSizeY;
		check((SizeX > 0) && (SizeY > 0));

		// Bilinear interpolate the values from the alpha texture
		int32 SampleX0 = FMath::FloorToInt(SampleX);
		int32 SampleX1 = (SampleX0 + 1) % SizeX;
		int32 SampleY0 = FMath::FloorToInt(SampleY);
		int32 SampleY1 = (SampleY0 + 1) % SizeY;

		check(!EdMode->UISettings->AlphaTextureData.IsEmpty());
		const uint8* AlphaData = EdMode->UISettings->AlphaTextureData.GetData();

		float Alpha00 = (float)AlphaData[SampleX0 + SampleY0 * SizeX] / 255.0f;
		float Alpha01 = (float)AlphaData[SampleX0 + SampleY1 * SizeX] / 255.0f;
		float Alpha10 = (float)AlphaData[SampleX1 + SampleY0 * SizeX] / 255.0f;
		float Alpha11 = (float)AlphaData[SampleX1 + SampleY1 * SizeX] / 255.0f;

		return FMath::Lerp(
			FMath::Lerp(Alpha00, Alpha01, FMath::Fractional(SampleX)),
			FMath::Lerp(Alpha10, Alpha11, FMath::Fractional(SampleX)),
			FMath::Fractional(SampleY)
			);
	}

};

//
// FLandscapeBrushAlphaPattern
//
class FLandscapeBrushAlphaPattern : public FLandscapeBrushAlphaBase
{
	using Super = FLandscapeBrushAlphaBase;

protected:
	FLandscapeBrushAlphaPattern(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushAlphaBase(InEdMode, InBrushMaterial)
	{
	}

public:
	static FLandscapeBrushAlphaPattern* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* PatternBrushMaterial = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/PatternBrushMaterial_Smooth.PatternBrushMaterial_Smooth"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushAlphaPattern(InEdMode, PatternBrushMaterial);
	}

	virtual ELandscapeBrushType GetBrushType() override { return ELandscapeBrushType::Alpha; }

	virtual bool CanPaint(const TSet<ULandscapeComponent*>& InAffectedComponents, bool bInHasUnloadedComponents) const override
	{
		if (!Super::CanPaint(InAffectedComponents, bInHasUnloadedComponents))
		{
			return false;
		}

		return EdMode->UISettings->HasValidAlphaTextureData();
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		if (!bCanPaint)
		{
			return FLandscapeBrushData();
		}

		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		const float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
		const float TotalRadius = EdMode->UISettings->GetCurrentToolBrushRadius() / ScaleXY;
		const float Radius = (1.0f - EdMode->UISettings->GetCurrentToolBrushFalloff()) * TotalRadius;
		const float Falloff = EdMode->UISettings->GetCurrentToolBrushFalloff() * TotalRadius;

		int32 SizeX = EdMode->UISettings->AlphaTextureSizeX;
		int32 SizeY = EdMode->UISettings->AlphaTextureSizeY;
		check(EdMode->UISettings->HasValidAlphaTextureData() && (SizeX > 0) && (SizeY > 0)); // See CanPaint() above : if we can paint, these must all be valid

		FIntRect ExclusiveBounds;
		ExclusiveBounds.Min.X = FMath::FloorToInt(LastMousePosition.X - TotalRadius);
		ExclusiveBounds.Min.Y = FMath::FloorToInt(LastMousePosition.Y - TotalRadius);
		ExclusiveBounds.Max.X = FMath::CeilToInt(LastMousePosition.X + TotalRadius) + 1;
		ExclusiveBounds.Max.Y = FMath::CeilToInt(LastMousePosition.Y + TotalRadius) + 1;

		// Clamp to landscape bounds.  GetLandscapeExtent provides inclusive bounds.
		int32 MinX, MaxX, MinY, MaxY;
		if (!ensure(LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY)))
		{
			// Landscape has no components somehow
			return FLandscapeBrushData();
		}
		// Clip using exclusive bounds so we can detect empty result.
		ExclusiveBounds.Clip(FIntRect(MinX, MinY, MaxX + 1, MaxY + 1));
		if (ExclusiveBounds.Area() <= 0)
		{
			// Brush is completely outside the landscape extent
			return FLandscapeBrushData();
		}

		FLandscapeBrushData BrushData(ExclusiveBounds);

		for (int32 Y = ExclusiveBounds.Min.Y; Y < ExclusiveBounds.Max.Y; Y++)
		{
			float* Scanline = BrushData.GetDataPtr(FIntPoint(0, Y));

			for (int32 X = ExclusiveBounds.Min.X; X < ExclusiveBounds.Max.X; X++)
			{
				float Angle;
				FVector2D Scale;
				FVector2D Bias;
				if (EdMode->UISettings->bUseWorldSpacePatternBrush)
				{
					FVector2D LocalOrigin = -FVector2D(LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().InverseTransformPosition(FVector(EdMode->UISettings->WorldSpacePatternBrushSettings.Origin, 0.0f)));
					const FVector2D LocalScale = FVector2D(
						ScaleXY / (EdMode->UISettings->WorldSpacePatternBrushSettings.RepeatSize * ((float)SizeX / SizeY)),
						ScaleXY / (EdMode->UISettings->WorldSpacePatternBrushSettings.RepeatSize));
					LocalOrigin *= LocalScale;
					Angle = -EdMode->UISettings->WorldSpacePatternBrushSettings.Rotation;
					if (EdMode->UISettings->WorldSpacePatternBrushSettings.bCenterTextureOnOrigin)
					{
						LocalOrigin += FVector2D(0.5f, 0.5f).GetRotated(-Angle);
					}
					Scale = FVector2D(SizeX, SizeY) * LocalScale;
					Bias = FVector2D(SizeX, SizeY) * LocalOrigin;
				}
				else
				{
					Scale.X = 1.0f / EdMode->UISettings->AlphaBrushScale;
					Scale.Y = 1.0f / EdMode->UISettings->AlphaBrushScale;
					Bias.X = SizeX * EdMode->UISettings->AlphaBrushPanU;
					Bias.Y = SizeY * EdMode->UISettings->AlphaBrushPanV;
					Angle = EdMode->UISettings->AlphaBrushRotation;
				}

				// Find alphamap sample location
				FVector2D SamplePos = FVector2D(X, Y) * Scale + Bias;
				SamplePos = SamplePos.GetRotated(Angle);

				float ModSampleX = FMath::Fmod(static_cast<float>(SamplePos.X), static_cast<float>(SizeX));
				float ModSampleY = FMath::Fmod(static_cast<float>(SamplePos.Y), static_cast<float>(SizeY));

				if (ModSampleX < 0.0f)
				{
					ModSampleX += (float)SizeX;
				}
				if (ModSampleY < 0.0f)
				{
					ModSampleY += (float)SizeY;
				}

				// Sample the alpha texture
				float Alpha = GetAlphaSample(ModSampleX, ModSampleY);

				// Distance from mouse
				float MouseDist = FMath::Sqrt(FMath::Square(LastMousePosition.X - (float)X) + FMath::Square(LastMousePosition.Y - (float)Y));

				float PaintAmount = CalculateFalloff(MouseDist, Radius, Falloff) * Alpha;

				if (PaintAmount > 0.0f)
				{
					if (EdMode->CurrentTool && EdMode->CurrentTool->GetToolType() != ELandscapeToolType::Mask
						&& EdMode->UISettings->bUseSelectedRegion && LandscapeInfo->SelectedRegion.Num() > 0)
					{
						float MaskValue = LandscapeInfo->SelectedRegion.FindRef(FIntPoint(X, Y));
						if (EdMode->UISettings->bUseNegativeMask)
						{
							MaskValue = 1.0f - MaskValue;
						}
						PaintAmount *= MaskValue;
					}
					// Set the brush value for this vertex
					Scanline[X] = PaintAmount;
				}
			}
		}
		return BrushData;
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		Super::Tick(ViewportClient, DeltaTime);

		if (const ALandscapeProxy* Proxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr)
		{
			const float ScaleXY = static_cast<float>(FMath::Abs(EdMode->CurrentToolTarget.LandscapeInfo->DrawScale.X));

			const bool bHasValidAlphaTextureData = EdMode->UISettings->HasValidAlphaTextureData();
			int32 SizeX = EdMode->UISettings->AlphaTextureSizeX;
			int32 SizeY = EdMode->UISettings->AlphaTextureSizeY;
			check(!bHasValidAlphaTextureData || ((SizeX > 0) && (SizeY > 0)));

			FLinearColor AlphaScaleBias(ForceInitToZero);
			float Angle;
			if (EdMode->UISettings->bUseWorldSpacePatternBrush)
			{
				FVector2D LocalOrigin = -FVector2D(Proxy->LandscapeActorToWorld().InverseTransformPosition(FVector(EdMode->UISettings->WorldSpacePatternBrushSettings.Origin, 0.0f)));
				FVector2D Scale(ForceInitToZero);
				if (bHasValidAlphaTextureData)
				{
					Scale = FVector2D(
						ScaleXY / (EdMode->UISettings->WorldSpacePatternBrushSettings.RepeatSize * ((float)SizeX / SizeY)),
						ScaleXY / (EdMode->UISettings->WorldSpacePatternBrushSettings.RepeatSize));
				}
				LocalOrigin *= Scale;
				Angle = -EdMode->UISettings->WorldSpacePatternBrushSettings.Rotation;
				if (EdMode->UISettings->WorldSpacePatternBrushSettings.bCenterTextureOnOrigin)
				{
					LocalOrigin += FVector2D(0.5f, 0.5f).GetRotated(-Angle);
				}
				AlphaScaleBias = FLinearColor(
					static_cast<float>(Scale.X),
					static_cast<float>(Scale.Y),
					static_cast<float>(LocalOrigin.X),
					static_cast<float>(LocalOrigin.Y));
			}
			else 
			{
				const FVector2D Scale(EdMode->UISettings->AlphaBrushScale * SizeX, EdMode->UISettings->AlphaBrushScale * SizeY);
				AlphaScaleBias = FLinearColor(
					static_cast<float>(FMath::IsNearlyZero(Scale.X) ? 1.0 : 1.0 / (Scale.X)),
					static_cast<float>(FMath::IsNearlyZero(Scale.Y) ? 1.0 : 1.0 / (Scale.Y)),
					EdMode->UISettings->AlphaBrushPanU,
					EdMode->UISettings->AlphaBrushPanV);
				Angle = EdMode->UISettings->AlphaBrushRotation;
			}
			Angle = FMath::DegreesToRadians(Angle);

			FVector LandscapeLocation = Proxy->LandscapeActorToWorld().GetTranslation();
			FVector4 LandscapeLocationParam(LandscapeLocation.X, LandscapeLocation.Y, LandscapeLocation.Z, Angle);

			FLinearColor AlphaTextureMask(
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Red ? 1.0f : 0.0f, 
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Green ? 1.0f : 0.0f, 
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Blue ? 1.0f : 0.0f, 
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Alpha ? 1.0f : 0.0f);

			for (const auto& BrushMaterialInstancePair : BrushMaterialInstanceMap)
			{
				UMaterialInstanceDynamic* const MaterialInstance = BrushMaterialInstancePair.Value.Get();
				if (!MaterialInstance)
				{
					continue;
				}
				
				MaterialInstance->SetVectorParameterValue(FName(TEXT("AlphaScaleBias")), AlphaScaleBias);
				MaterialInstance->SetDoubleVectorParameterValue(FName(TEXT("LandscapeLocation")), LandscapeLocationParam);
				MaterialInstance->SetVectorParameterValue(FName(TEXT("AlphaTextureMask")), AlphaTextureMask);
				MaterialInstance->SetTextureParameterValue(FName(TEXT("AlphaTexture")), EdMode->UISettings->AlphaTexture);
			}
		}
	}

	virtual const TCHAR* GetBrushName() override { return TEXT("Pattern"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_PatternAlpha", "Pattern Alpha"); };
};


//
// FLandscapeBrushAlpha
//
class FLandscapeBrushAlpha : public FLandscapeBrushAlphaBase
{
	using Super = FLandscapeBrushAlphaBase;

	float LastMouseAngle;
	FVector2f OldMousePosition;	// a previous mouse position, kept until we move a certain distance away, for smoothing deltas
	double LastMouseSampleTime;

protected:
	FLandscapeBrushAlpha(FEdModeLandscape* InEdMode, UMaterialInterface* InBrushMaterial)
		: FLandscapeBrushAlphaBase(InEdMode, InBrushMaterial)
		, LastMouseAngle(0.0f)
		, OldMousePosition(0.0f, 0.0f)
		, LastMouseSampleTime(FPlatformTime::Seconds())
	{
	}

public:
	static FLandscapeBrushAlpha* Create(FEdModeLandscape* InEdMode)
	{
		UMaterialInstanceConstant* AlphaBrushMaterial = LoadObject<UMaterialInstanceConstant>(nullptr, TEXT("/Engine/EditorLandscapeResources/AlphaBrushMaterial_Smooth.AlphaBrushMaterial_Smooth"), nullptr, LOAD_None, nullptr);
		return new FLandscapeBrushAlpha(InEdMode, AlphaBrushMaterial);
	}

	bool ComputeAlphaBrushScaleAndRadius(float& OutAlphaBrushScale, float& OutRadius, int32& OutSizeX, int32& OutSizeY) const
	{
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();

		int32 SizeX = EdMode->UISettings->AlphaTextureSizeX;
		int32 SizeY = EdMode->UISettings->AlphaTextureSizeY;
		bool bIsValidTextureSize = (SizeX > 0) && (SizeY > 0);

		float ScaleXY = static_cast<float>(FMath::Abs(LandscapeInfo->DrawScale.X));
		float Radius = FMath::IsNearlyZero(ScaleXY) ? 0.0f : EdMode->UISettings->GetCurrentToolBrushRadius() / ScaleXY;

		float MaxSize = 2.0f * FMath::Sqrt(FMath::Square(Radius) / 2.0f);
		float FinalScale = bIsValidTextureSize ? MaxSize / (float)FMath::Max<int32>(SizeX, SizeY) : 0.0f;

		if (FMath::IsNearlyZero(FinalScale))
		{
			return false;
		}

		check(bIsValidTextureSize);
		OutAlphaBrushScale = FinalScale;
		OutRadius = Radius;
		OutSizeX = SizeX;
		OutSizeY = SizeY;
		return true;
	}

	virtual bool CanPaint(const TSet<ULandscapeComponent*>& InAffectedComponents, bool bInHasUnloadedComponents) const override
	{
		if (!Super::CanPaint(InAffectedComponents, bInHasUnloadedComponents))
		{
			return false;
		}

		if (!EdMode->UISettings->HasValidAlphaTextureData())
		{
			return false;
		}

		float AlphaBrushScale; 
		float Radius;
		int32 SizeX;
		int32 SizeY;
		if (!ComputeAlphaBrushScaleAndRadius(AlphaBrushScale, Radius, SizeX, SizeY))
		{
			return false;
		}

		return true;
	}

	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) override
	{
		if (!bCanPaint)
		{
			return FLandscapeBrushData();
		}

		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (EdMode->UISettings->bAlphaBrushAutoRotate && OldMousePosition.IsZero())
		{
			OldMousePosition = LastMousePosition;
			LastMouseAngle = 0.0f;
			LastMouseSampleTime = FPlatformTime::Seconds();
			return FLandscapeBrushData();
		}
		else
		{
			float AlphaBrushScale;
			float Radius;
			int32 SizeX;
			int32 SizeY;
			bool bIsValid = ComputeAlphaBrushScaleAndRadius(AlphaBrushScale, Radius, SizeX, SizeY);
			check(bIsValid && !FMath::IsNearlyZero(AlphaBrushScale) && !FMath::IsNearlyZero(Radius) && (SizeX > 0) && (SizeY > 0)); // See CanPaint() function above : if bCanPaint, AlphaBrushScale should be non-zero

			const float BrushAngle = EdMode->UISettings->bAlphaBrushAutoRotate ? LastMouseAngle : FMath::DegreesToRadians(EdMode->UISettings->AlphaBrushRotation);
			FIntRect ExclusiveBounds;
			ExclusiveBounds.Min.X = FMath::FloorToInt(LastMousePosition.X - Radius);
			ExclusiveBounds.Min.Y = FMath::FloorToInt(LastMousePosition.Y - Radius);
			ExclusiveBounds.Max.X = FMath::CeilToInt(LastMousePosition.X + Radius) + 1;
			ExclusiveBounds.Max.Y = FMath::CeilToInt(LastMousePosition.Y + Radius) + 1;

			// Clamp to landscape bounds.  GetLandscapeExtent provides inclusive bounds.
			int32 MinX, MaxX, MinY, MaxY;
			if (!ensure(LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY)))
			{
				// Landscape has no components somehow
				return FLandscapeBrushData();
			}
			// Clip using exclusive bounds so we can detect empty result.
			ExclusiveBounds.Clip(FIntRect(MinX, MinY, MaxX + 1, MaxY + 1));
			if (ExclusiveBounds.Area() <= 0)
			{
				// Brush is completely outside the landscape extent
				return FLandscapeBrushData();
			}

			FLandscapeBrushData BrushData(FIntRect(ExclusiveBounds.Min, ExclusiveBounds.Max));

			for (int32 Y = ExclusiveBounds.Min.Y; Y < ExclusiveBounds.Max.Y; Y++)
			{
				float* Scanline = BrushData.GetDataPtr(FIntPoint(0, Y));

				for (int32 X = ExclusiveBounds.Min.X; X < ExclusiveBounds.Max.X; X++)
				{
					// Find alphamap sample location
					float ScaleSampleX = ((float)X - LastMousePosition.X) / AlphaBrushScale;
					float ScaleSampleY = ((float)Y - LastMousePosition.Y) / AlphaBrushScale;

					// Rotate around center to match angle
					float SampleX = ScaleSampleX * FMath::Cos(BrushAngle) - ScaleSampleY * FMath::Sin(BrushAngle);
					float SampleY = ScaleSampleY * FMath::Cos(BrushAngle) + ScaleSampleX * FMath::Sin(BrushAngle);

					SampleX += (float)SizeX * 0.5f;
					SampleY += (float)SizeY * 0.5f;

					if (SampleX >= 0 && SampleX <= (SizeX - 1) &&
						SampleY >= 0 && SampleY <= (SizeY - 1))
					{
						// Sample the alpha texture
						float Alpha = GetAlphaSample(SampleX, SampleY);

						if (Alpha > 0.0f)
						{
							// Set the brush value for this vertex
							FIntPoint VertexKey = FIntPoint(X, Y);

							if (EdMode->CurrentTool && EdMode->CurrentTool->GetToolType() != ELandscapeToolType::Mask
								&& EdMode->UISettings->bUseSelectedRegion && LandscapeInfo->SelectedRegion.Num() > 0)
							{
								float MaskValue = LandscapeInfo->SelectedRegion.FindRef(VertexKey);
								if (EdMode->UISettings->bUseNegativeMask)
								{
									MaskValue = 1.0f - MaskValue;
								}
								Alpha *= MaskValue;
							}

							Scanline[X] = Alpha;
						}
					}
				}
			}

			return BrushData;
		}
	}

	virtual void MouseMove(float LandscapeX, float LandscapeY) override
	{
		Super::MouseMove(LandscapeX, LandscapeY);

		if (EdMode->UISettings->bAlphaBrushAutoRotate)
		{
			// don't do anything with the angle unless we move at least 0.1 units.
			FVector2f MouseDelta = LastMousePosition - OldMousePosition;
			if (MouseDelta.SizeSquared() >= FMath::Square(0.5f))
			{
				double SampleTime = FPlatformTime::Seconds();
				float DeltaTime = (float)(SampleTime - LastMouseSampleTime);
				FVector2f MouseDirection = MouseDelta.GetSafeNormal();
				float MouseAngle = FMath::Lerp(LastMouseAngle, FMath::Atan2(-MouseDirection.Y, MouseDirection.X), FMath::Min<float>(10.0f * DeltaTime, 1.0f));		// lerp over 100ms
				LastMouseAngle = MouseAngle;
				LastMouseSampleTime = SampleTime;
				OldMousePosition = LastMousePosition;
				// UE_LOGF(LogLandscape, Log, "(%f,%f) delta (%f,%f) angle %f", LandscapeX, LandscapeY, MouseDirection.X, MouseDirection.Y, MouseAngle);
			}
		}
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		Super::Tick(ViewportClient, DeltaTime);

		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo)
		{
			FVector2f Scale(1.0f, 1.0f);
			float AlphaBrushScale;
			float Radius;
			int32 SizeX;
			int32 SizeY;
			bool bIsValid = ComputeAlphaBrushScaleAndRadius(AlphaBrushScale, Radius, SizeX, SizeY);
			if (bIsValid)
			{
				check(!FMath::IsNearlyZero(AlphaBrushScale) && !FMath::IsNearlyZero(Radius) && (SizeX > 0) && (SizeY > 0)); // See CanPaint() function above : if bCanPaint, AlphaBrushScale should be non-zero
				Scale = FVector2f(1.0f / (AlphaBrushScale * SizeX), 1.0f / (AlphaBrushScale * SizeY));
			}

			FLinearColor BrushScaleRot(
				Scale.X,
				Scale.Y,
				0.0f,
				EdMode->UISettings->bAlphaBrushAutoRotate ? LastMouseAngle : FMath::DegreesToRadians(EdMode->UISettings->AlphaBrushRotation)
				);

			FLinearColor AlphaTextureMask(EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Red ? 1.0f : 0.0f,
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Green ? 1.0f : 0.0f,
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Blue ? 1.0f : 0.0f,
				EdMode->UISettings->AlphaTextureChannel == ELandscapeTextureColorChannel::Alpha ? 1.0f : 0.0f);


			for (const auto& BrushMaterialInstancePair : BrushMaterialInstanceMap)
			{
				UMaterialInstanceDynamic* const MaterialInstance = BrushMaterialInstancePair.Value.Get();

				if (!MaterialInstance)
				{
					continue;
				}
				
				MaterialInstance->SetVectorParameterValue(FName(TEXT("BrushScaleRot")), BrushScaleRot);
				MaterialInstance->SetVectorParameterValue(FName(TEXT("AlphaTextureMask")), AlphaTextureMask);
				MaterialInstance->SetTextureParameterValue(FName(TEXT("AlphaTexture")), EdMode->UISettings->AlphaTexture);
			}
		}
	}

	virtual const TCHAR* GetBrushName() override { return TEXT("Alpha"); }
	virtual FText GetDisplayName() override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Brush_Alpha", "Alpha"); };

};


void FEdModeLandscape::InitializeBrushes()
{
	FLandscapeBrushSet* BrushSet;
	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Circle"));
	BrushSet->Brushes.Add(FLandscapeBrushCircle_Smooth::Create(this));
	BrushSet->Brushes.Add(FLandscapeBrushCircle_Linear::Create(this));
	BrushSet->Brushes.Add(FLandscapeBrushCircle_Spherical::Create(this));
	BrushSet->Brushes.Add(FLandscapeBrushCircle_Tip::Create(this));

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Alpha"));
	BrushSet->Brushes.Add(FLandscapeBrushAlpha::Create(this));

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Pattern"));
	BrushSet->Brushes.Add(FLandscapeBrushAlphaPattern::Create(this));

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Component"));
	BrushSet->Brushes.Add(new FLandscapeBrushComponent(this));

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Gizmo"));
	GizmoBrush = new FLandscapeBrushGizmo(this);
	BrushSet->Brushes.Add(GizmoBrush);

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Splines"));
	BrushSet->Brushes.Add(new FLandscapeBrushSplines(this));

	BrushSet = new(LandscapeBrushSets)FLandscapeBrushSet(TEXT("BrushSet_Dummy"));
	BrushSet->Brushes.Add(new FLandscapeBrushDummy(this));
}
