// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Engine/EngineTypes.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeGizmoActiveActor.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "Landscape.h"
#include "LandscapeStreamingProxy.h"
#include "ObjectTools.h"
#include "LandscapeEdit.h"
#include "LandscapeComponent.h"
#include "LandscapeRender.h"
#include "PropertyEditorModule.h"
#include "InstancedFoliageActor.h"
#include "LandscapeEdModeTools.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Algo/Copy.h"
#include "LandscapeSubsystem.h"

#define LOCTEXT_NAMESPACE "Landscape"

//
// FLandscapeToolSelect
//
class FLandscapeToolStrokeSelect : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

	bool bInitializedComponentInvert;
	bool bInvert;
	bool bNeedsSelectionUpdate;

public:
	FLandscapeToolStrokeSelect(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, bInitializedComponentInvert(false)
		, bNeedsSelectionUpdate(false)
	{
	}

	~FLandscapeToolStrokeSelect()
	{
		if (bNeedsSelectionUpdate)
		{
			TArray<UObject*> Objects;
			if (LandscapeInfo)
			{
				TSet<ULandscapeComponent*> SelectedComponents = LandscapeInfo->GetSelectedComponents();
				Objects.Reset(SelectedComponents.Num());
				Algo::Copy(SelectedComponents, Objects);
			}
			FPropertyEditorModule& PropertyModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			PropertyModule.UpdatePropertyViews(Objects);
		}
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeSelect_Apply);

		if (LandscapeInfo)
		{
			LandscapeInfo->Modify();

			// TODO - only retrieve bounds as we don't need the data
			const FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			// Shrink bounds by 1,1 to avoid GetComponentsInRegion picking up extra components on all sides due to the overlap between components
			TSet<ULandscapeComponent*> NewComponents;
			LandscapeInfo->GetComponentsInRegion(X1 + 1, Y1 + 1, X2 - 1, Y2 - 1, NewComponents);

			if (!bInitializedComponentInvert)
			{
				// Get the component under the mouse location. Copied from FLandscapeBrushComponent::ApplyBrush()
				const float MouseX = static_cast<float>(InteractorPositions[0].Position.X);
				const float MouseY = static_cast<float>(InteractorPositions[0].Position.Y);
				const int32 MouseComponentIndexX = (MouseX >= 0.0f) ? FMath::FloorToInt(MouseX / LandscapeInfo->ComponentSizeQuads) : FMath::CeilToInt(MouseX / LandscapeInfo->ComponentSizeQuads);
				const int32 MouseComponentIndexY = (MouseY >= 0.0f) ? FMath::FloorToInt(MouseY / LandscapeInfo->ComponentSizeQuads) : FMath::CeilToInt(MouseY / LandscapeInfo->ComponentSizeQuads);
				ULandscapeComponent* MouseComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(MouseComponentIndexX, MouseComponentIndexY));

				if (MouseComponent != nullptr)
				{
					bInvert = LandscapeInfo->GetSelectedComponents().Contains(MouseComponent);
				}
				else
				{
					bInvert = false;
				}

				bInitializedComponentInvert = true;
			}

			TSet<ULandscapeComponent*> NewSelection;
			if (bInvert)
			{
				NewSelection = LandscapeInfo->GetSelectedComponents().Difference(NewComponents);
			}
			else
			{
				NewSelection = LandscapeInfo->GetSelectedComponents().Union(NewComponents);
			}

			LandscapeInfo->Modify();
			LandscapeInfo->UpdateSelectedComponents(NewSelection);

			// Update Details tab with selection
			bNeedsSelectionUpdate = true;
		}
	}
};

class FLandscapeToolSelect : public FLandscapeToolBase<FLandscapeToolStrokeSelect>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeSelect>;

public:
	FLandscapeToolSelect(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual bool AffectsEditLayers() const { return false; }

	virtual ELandscapeLayerUpdateMode GetBeginToolContentUpdateFlag() const override
	{
		return ELandscapeLayerUpdateMode::Update_None;
	}

	virtual ELandscapeLayerUpdateMode GetTickToolContentUpdateFlag() const override
	{
		return ELandscapeLayerUpdateMode::Update_None;
	}

	virtual ELandscapeLayerUpdateMode GetEndToolContentUpdateFlag() const override
	{
		return ELandscapeLayerUpdateMode::Update_None;
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Select"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Selection", "Component Selection"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Selection_Message", "Paint a mask on the Landscape to protect areas from editing."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectComponent | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }
};

//
// FLandscapeToolMask
//
class FLandscapeToolStrokeMask : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeMask(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, Cache(InTarget.LandscapeInfo.Get())
	{
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeMask_Apply);

		if (LandscapeInfo)
		{
			LandscapeInfo->Modify();

			// Invert when holding Shift
			bool bInvert = InteractorPositions[ InteractorPositions.Num() - 1].bModifierPressed;

			const FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			// Tablet pressure
			float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

			Cache.CacheData(X1, Y1, X2, Y2);
			TArray<uint8> Data;
			Cache.GetCachedData(X1, Y1, X2, Y2, Data);

			TSet<ULandscapeComponent*> NewComponents;
			LandscapeInfo->GetComponentsInRegion(X1, Y1, X2, Y2, NewComponents);
			LandscapeInfo->UpdateSelectedComponents(NewComponents, false);

			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				uint8* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const FIntPoint Key = FIntPoint(X, Y);
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f && LandscapeInfo->IsValidPosition(X, Y))
					{
						float PaintValue = BrushValue * UISettings->GetCurrentToolStrength() * Pressure;
						float Value = DataScanline[X] / 255.0f;
						checkSlow(FMath::IsNearlyEqual(Value, LandscapeInfo->SelectedRegion.FindRef(Key), 1 / 255.0f));
						if (bInvert)
						{
							Value = FMath::Max(Value - PaintValue, 0.0f);
						}
						else
						{
							Value = FMath::Min(Value + PaintValue, 1.0f);
						}
						if (Value > 0.0f)
						{
							LandscapeInfo->SelectedRegion.Add(Key, Value);
						}
						else
						{
							LandscapeInfo->SelectedRegion.Remove(Key);
						}

						DataScanline[X] = static_cast<uint8>(FMath::Clamp<int32>(FMath::RoundToInt(Value * 255), 0, 255));
					}
				}
			}

			Cache.SetCachedData(X1, Y1, X2, Y2, Data);
			Cache.Flush();
		}
	}

protected:
	FLandscapeDataCache Cache;
};

class FLandscapeToolMask : public FLandscapeToolBase<FLandscapeToolStrokeMask>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeMask>;

public:
	FLandscapeToolMask(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual bool AffectsEditLayers() const { return false; }

	virtual const TCHAR* GetToolName() const override { return TEXT("Mask"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Mask", "Region Selection"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Mask_Message", "Region Selection"); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectRegion | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return true; }

	virtual ELandscapeToolType GetToolType() const override { return ELandscapeToolType::Mask; }
};

//
// FLandscapeToolVisibility
//
class FLandscapeToolStrokeVisibility : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeVisibility(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, Cache(InTarget.LandscapeInfo.Get())
	{
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeVisibility_Apply);

		if (LandscapeInfo)
		{
			LandscapeInfo->Modify();
			// Get list of verts to update
			FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			// Invert when holding Shift
			bool bInvert = InteractorPositions.Last().bModifierPressed;

			// Tablet pressure
			float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

			Cache.CacheData(X1, Y1, X2, Y2);
			TArray<uint8> Data;
			Cache.GetCachedData(X1, Y1, X2, Y2, Data);

			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				uint8* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						uint8 Value = bInvert ? 0 : 255; // Just on and off for visibility, for masking...
						DataScanline[X] = Value;
					}
				}
			}

			Cache.SetCachedData(X1, Y1, X2, Y2, Data);
			Cache.Flush();

			// Dirty any runtime virtual textures that our landscape components write to.
			LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
		}
	}

protected:
	FLandscapeVisCache Cache;
};

class FLandscapeToolVisibility : public FLandscapeToolBase<FLandscapeToolStrokeVisibility>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeVisibility>;

public:
	FLandscapeToolVisibility(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& InTarget, const FVector& InHitLocation) override
	{
		return Super::BeginTool(ViewportClient, InTarget, InHitLocation);
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Visibility"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Visibility", "Visibility"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Visibility_Message", "This tool will allow you to mask out the visibility and collision of areas of your Landscape when used in conjunction with the Landscape Hole Material."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ELandscapeToolTargetType::Visibility;
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		if (InLayerInfo != ALandscapeProxy::VisibilityLayer)
		{
			return ELandscapeTargetLayerActionType::None;
		}

		const bool bInvert = bInIsModifierPressed;
		return bInvert ? ELandscapeTargetLayerActionType::Inverted : ELandscapeTargetLayerActionType::Normal;
	}
};

//
// FLandscapeToolMoveToLevel
//
class FLandscapeToolStrokeMoveToLevel : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeMoveToLevel(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeMoveToLevel_Apply);

		ALandscape* Landscape = LandscapeInfo ? LandscapeInfo->LandscapeActor.Get() : nullptr;

		if (Landscape)
		{
			Landscape->Modify();
			LandscapeInfo->Modify();

			TArray<UObject*> RenameObjects;
			FString MsgBoxList;

			// Check the Physical Material is same package with Landscape
			if (Landscape->DefaultPhysMaterial && Landscape->DefaultPhysMaterial->GetOutermost() == Landscape->GetOutermost())
			{
				//FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "LandscapePhyMaterial_Warning", "Landscape's DefaultPhysMaterial is in the same package as the Landscape Actor. To support streaming levels, you must move the PhysicalMaterial to another package.") );
				RenameObjects.AddUnique(Landscape->DefaultPhysMaterial);
				MsgBoxList += Landscape->DefaultPhysMaterial->GetPathName();
				MsgBoxList += FString::Printf(TEXT("\n"));
			}

			// Check the LayerInfoObjects are same package with Landscape
			for (int32 i = 0; i < LandscapeInfo->Layers.Num(); ++i)
			{
				ULandscapeLayerInfoObject* LayerInfo = LandscapeInfo->Layers[i].LayerInfoObj;
				if (LayerInfo && LayerInfo->GetOutermost() == Landscape->GetOutermost())
				{
					RenameObjects.AddUnique(LayerInfo);
					MsgBoxList += LayerInfo->GetPathName();
					MsgBoxList += FString::Printf(TEXT("\n"));
				}
			}

			auto SelectedComponents = LandscapeInfo->GetSelectedComponents();
			bool bBrush = false;
			if (!SelectedComponents.Num())
			{
				// Get list of verts to update
				// TODO - only retrieve bounds as we don't need the data
				FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
				if (!BrushInfo)
				{
					return;
				}

				int32 X1, Y1, X2, Y2;
				BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

				// Shrink bounds by 1,1 to avoid GetComponentsInRegion picking up extra components on all sides due to the overlap between components
				LandscapeInfo->GetComponentsInRegion(X1 + 1, Y1 + 1, X2 - 1, Y2 - 1, SelectedComponents);
				bBrush = true;
			}

			check(ViewportClient->GetScene());
			UWorld* World = ViewportClient->GetScene()->GetWorld();
			check(World);
			if (SelectedComponents.Num())
			{
				bool bIsAllCurrentLevel = true;
				for (ULandscapeComponent* Component : SelectedComponents)
				{
					if (Component->GetLandscapeProxy()->GetLevel() != World->GetCurrentLevel())
					{
						bIsAllCurrentLevel = false;
					}
				}

				if (bIsAllCurrentLevel)
				{
					// Need to fix double WM
					if (!bBrush)
					{
						// Remove Selection
						LandscapeInfo->ClearSelectedRegion(true);
					}
					return;
				}

				for (ULandscapeComponent* Component : SelectedComponents)
				{
					UMaterialInterface* LandscapeMaterial = Component->GetLandscapeMaterial();
					if (LandscapeMaterial && LandscapeMaterial->GetOutermost() == Component->GetOutermost())
					{
						RenameObjects.AddUnique(LandscapeMaterial);
						MsgBoxList += Component->GetName() + TEXT("'s ") + LandscapeMaterial->GetPathName();
						MsgBoxList += FString::Printf(TEXT("\n"));
						//It.RemoveCurrent();
					}
				}

				if (RenameObjects.Num())
				{
					if (FMessageDialog::Open(EAppMsgType::OkCancel,
						FText::Format(
						NSLOCTEXT("UnrealEd", "LandscapeMoveToStreamingLevel_SharedResources", "The following items must be moved out of the persistent level and into a package that can be shared between multiple levels:\n\n{0}"),
						FText::FromString(MsgBoxList))) == EAppReturnType::Type::Ok)
					{
						FString Path = Landscape->GetOutermost()->GetName() + TEXT("_sharedassets/");
						bool bSucceed = ObjectTools::RenameObjects(RenameObjects, false, TEXT(""), Path);
						if (!bSucceed)
						{
							FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "LandscapeMoveToStreamingLevel_RenameFailed", "Move To Streaming Level did not succeed because shared resources could not be moved to a new package."));
							return;
						}
					}
					else
					{
						return;
					}
				}

				FScopedSlowTask SlowTask(0, LOCTEXT("BeginMovingLandscapeComponentsToCurrentLevelTask", "Moving Landscape components to current level"));
				SlowTask.MakeDialogDelayed(10); // show slow task dialog after 10 seconds

				if (ALandscapeProxy* LandscapeProxy = LandscapeInfo->MoveComponentsToLevel(SelectedComponents.Array(), World->GetCurrentLevel()))
				{
					GEditor->SelectNone(false, true);
					GEditor->SelectActor(LandscapeProxy, true, false, true);

					GEditor->SelectNone(false, true);

					// Remove Selection
					LandscapeInfo->ClearSelectedRegion(true);
				}
			}
		}
	}
};

class FLandscapeToolMoveToLevel : public FLandscapeToolBase<FLandscapeToolStrokeMoveToLevel>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeMoveToLevel>;

public:
	FLandscapeToolMoveToLevel(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}
	virtual bool AffectsEditLayers() const override { return false; }

	virtual const TCHAR* GetToolName() const override { return TEXT("MoveToLevel"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_MoveToLevel", "Move to Streaming Level"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_MoveToLevel_Message", "Move the selected components, via using the Selection tool, to the current streaming level.  This makes it possible to move sections of a Landscape into a streaming level so that they will be streamed in and out with that level, optimizing the performance of the Landscape."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectComponent | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }
};

//
// FLandscapeToolAddComponent
//

class FLandscapeToolStrokeAddComponent : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeAddComponent(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	virtual ~FLandscapeToolStrokeAddComponent()
	{
	}

	virtual void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeAddComponent_Apply);

		if (LandscapeInfo)
		{
			check(Brush->GetBrushType() == ELandscapeBrushType::Component);

			// Get list of verts to update
			// TODO - only retrieve bounds as we don't need the data
			FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			// Find component range for this block of data, non shared vertices
			int32 ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2;
			ALandscape::CalcComponentIndicesNoOverlap(X1, Y1, X2, Y2, LandscapeInfo->ComponentSizeQuads, ComponentIndexX1, ComponentIndexY1, ComponentIndexX2, ComponentIndexY2);

			// expand the area by one vertex in each direction to capture the height of the corner vertices these components will attach to
			X1 -= 1;
			Y1 -= 1;
			X2 += 1;
			Y2 += 1;

			UWorld* World = ViewportClient->GetScene()->GetWorld();
			
			ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
			check(Landscape != nullptr);

			TMap<FGuid, TArray<uint16>> PersistentEditLayerData;
			// Copy the persistent edit layer data using FLandscapeHeightCache on the region. That will extrapolate the data for the missing components : 
			for (const ULandscapeEditLayerBase* LandscapeEditLayer : Landscape->GetEditLayersConst())
			{
				check(LandscapeEditLayer != nullptr);
				if (!LandscapeEditLayer->NeedsPersistentTextures())
				{
					continue;
				}

				TArray<uint16>& Data = PersistentEditLayerData.FindOrAdd(LandscapeEditLayer->GetGuid());

				// No need to copy the persistent textures if they are not manually edited (e.g. fully procedural layers with persistent textures : e.g. the splines layer) :
				//  those will have a heightmap created and initialized via AddDefaultLayerData but we won't copy any data into it :
				if (LandscapeEditLayer->SupportsEditingTools())
				{
					Data.AddZeroed((1 + X2 - X1) * (1 + Y2 - Y1));
					// Use FLandscapeEditDataInterface to cache / interpolate the existing edit layer's height data, that allows to use SetShouldDirtyPackage(false)
					FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo, LandscapeEditLayer->GetGuid(), /*bInUploadTextureChangesToGPU = */false);
					// Ideally, we wouldn't even need to use FLandscapeEditDataInterface to simply cache the data, since that is intended to mutate the data 
					//  but it's the class that allows extrapolating the data for missing components... At least, we can force it not to dirty the package when doing so : 
					LandscapeEdit.SetShouldDirtyPackage(false);

					int32 TempX1 = X1; // GetHeightData overwrites its input min/max x/y
					int32 TempX2 = X2;
					int32 TempY1 = Y1;
					int32 TempY2 = Y2;
					LandscapeEdit.GetHeightData(TempX1, TempY1, TempX2, TempY2, Data.GetData(), /*Stride = */0);
				}
			}

			TArray<ULandscapeComponent*> NewComponents;
			LandscapeInfo->Modify();
			ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
			for (int32 ComponentIndexY = ComponentIndexY1; ComponentIndexY <= ComponentIndexY2; ComponentIndexY++)
			{
				for (int32 ComponentIndexX = ComponentIndexX1; ComponentIndexX <= ComponentIndexX2; ComponentIndexX++)
				{
					ULandscapeComponent* LandscapeComponent = LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(ComponentIndexX, ComponentIndexY));
					if (!LandscapeComponent)
					{
						// Add New component...
						FIntPoint ComponentBase = FIntPoint(ComponentIndexX, ComponentIndexY)* LandscapeInfo->ComponentSizeQuads;

						if (ALandscapeProxy* LandscapeProxy = LandscapeSubsystem->FindOrAddLandscapeProxy(LandscapeInfo, ComponentBase))
						{
							LandscapeComponent = NewObject<ULandscapeComponent>(LandscapeProxy, NAME_None, RF_Transactional);
							NewComponents.Add(LandscapeComponent);
							LandscapeComponent->Init(
								ComponentBase.X, ComponentBase.Y,
								LandscapeProxy->ComponentSizeQuads,
								LandscapeProxy->NumSubsections,
								LandscapeProxy->SubsectionSizeQuads
							);

							// Let the component be known to the ULandscapeInfo. This needs to be done before restoring the edit layers data via FLandscapeHeightCache 
							LandscapeInfo->XYtoComponentMap.Add(FIntPoint(ComponentIndexX, ComponentIndexY), LandscapeComponent);
							LandscapeInfo->XYtoAddCollisionMap.Remove(FIntPoint(ComponentIndexX, ComponentIndexY));

							// First create the component's final heightmap, it's needed for AddDefaultLayerData... 
							TArray<FColor> HeightData;
							const int32 ComponentVerts = (LandscapeComponent->SubsectionSizeQuads + 1) * LandscapeComponent->NumSubsections;
							HeightData.AddZeroed(FMath::Square(ComponentVerts));
							LandscapeComponent->InitHeightmapData(HeightData, /*bUpdateCollision = */false);

							// For each layer that needs some persistent height data
							for (TPair<FGuid, TArray<uint16>> LayerDataPair : PersistentEditLayerData)
							{
								const FGuid& LayerGuid = LayerDataPair.Key;
								TMap<UTexture2D*, UTexture2D*> CreatedTextures;
								LandscapeComponent->AddDefaultLayerData(LayerGuid, { LandscapeComponent }, CreatedTextures);

								TArray<uint16>& CachedExtrapolatedData = LayerDataPair.Value;
								if (!CachedExtrapolatedData.IsEmpty())
								{
									// Finally, copy our (previously extrapolated) edit layer data onto this new component :
									FScopedSetLandscapeEditingLayer LayerScope(Landscape, LayerGuid);
									// We have cached the data for the entire X1, Y1, X2, Y2 region, which is larger than this new component's area so we need
									//  to extract the component's data into another array : 
									TArray<uint16> ComponentExtrapolatedCachedData;
									{
										const int32 SrcDataStride = X2 - X1 + 1;
										const int32 DstDataStride = ComponentVerts;
										ComponentExtrapolatedCachedData.AddZeroed(FMath::Square(ComponentVerts));
										uint16* DstDataPtr = ComponentExtrapolatedCachedData.GetData();
										for (int32 Y = ComponentBase.Y; Y < ComponentBase.Y + ComponentVerts; ++Y)
										{
											const FIntPoint SrcDataOffset(ComponentBase.X - X1, Y - Y1);
											check((SrcDataOffset.X >= 0) && (SrcDataOffset.Y >= 0));
											uint16* SrcDataPtr = CachedExtrapolatedData.GetData() + SrcDataOffset.X + SrcDataOffset.Y * SrcDataStride;
											check(SrcDataPtr + DstDataStride < SrcDataPtr + SrcDataStride);
											FMemory::Memcpy(DstDataPtr, SrcDataPtr, DstDataStride * sizeof(uint16));
											DstDataPtr += DstDataStride;
										}
									}

									FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo, LayerGuid);
									LandscapeEdit.SetHeightData(ComponentBase.X, ComponentBase.Y, ComponentBase.X + ComponentVerts - 1, ComponentBase.Y + ComponentVerts - 1, ComponentExtrapolatedCachedData.GetData(), /*InStride = */0, /*InCalcNormals = */false);
								}
							}

							LandscapeComponent->UpdateMaterialInstances();
							// We need to update the bounds before registering the component, otherwise, the transform verification that are done on register will fail
							LandscapeComponent->UpdateCachedBounds();
							LandscapeComponent->UpdateBounds();
							LandscapeComponent->RegisterComponent();
							// The collisions will be generated after the edit layers merge readback has been done but we still to create the new collision component here
							//  so that it's part of the transaction and can be automatically deleted upon undo. UpdateCollisionData handles that :
							LandscapeComponent->UpdateCollisionData();
						}
					}
				}
			}

			// We need to invalidate the splines in case there's one that interacts with one of the newly added components : 
			Landscape->RequestSplineLayerUpdate();
			Landscape->RequestLayersInitialization();
			GEngine->BroadcastOnActorMoved(Landscape);
		}
	}
};

class FLandscapeToolAddComponent : public FLandscapeToolBase<FLandscapeToolStrokeAddComponent>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeAddComponent>;

public:
	FLandscapeToolAddComponent(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
		, bIsToolActionResolutionCompliant(true)
	{
	}
	virtual bool AffectsEditLayers() const override { return false; }

	virtual const TCHAR* GetToolName() const override { return TEXT("AddComponent"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_AddComponent", "Add New Landscape Component"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_AddComponent_Message", "Create new components for the current Landscape, one at a time.  The cursor shows a green wireframe where new components can be added."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	// Sphere traces can result in components being added at a good distance from any neighboring components, because it intersects against a virtual plane
	virtual bool UseSphereTrace() const override { return false; }

	virtual bool CanToolBeActivated() const override
	{ 
		return Super::CanToolBeActivated() && bIsToolActionResolutionCompliant;
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		if (EdMode != nullptr)
		{
			bIsToolActionResolutionCompliant = EdMode->IsLandscapeResolutionCompliant();
		}

		Super::Tick(ViewportClient, DeltaTime);
	}
	
	virtual void EnterTool() override
	{
		Super::EnterTool();
		AddCollision.Reset();
		if(ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			LandscapeInfo->UpdateAllAddCollisions(); // Todo - as this is only used by this tool, move it into this tool?
		}
	}

	virtual void ExitTool() override
	{
		Super::ExitTool();

		AddCollision.Reset();
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		if (AddCollision.IsSet())
		{
			const FColor LineColor = CanToolBeActivated() ? FColor(0, 255, 128) : FColor(255, 0, 64);

			PDI->DrawLine(AddCollision->Corners[3], AddCollision->Corners[1], LineColor, SDPG_Foreground);
			PDI->DrawLine(AddCollision->Corners[1], AddCollision->Corners[0], LineColor, SDPG_Foreground);

			PDI->DrawLine(AddCollision->Corners[0], AddCollision->Corners[2], LineColor, SDPG_Foreground);
			PDI->DrawLine(AddCollision->Corners[2], AddCollision->Corners[3], LineColor, SDPG_Foreground);
		}
	}

	virtual bool HitTrace(const FVector& TraceStart, const FVector& TraceEnd, FVector& OutHitLocation) override
	{
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (!LandscapeInfo)
		{
			return false;
		}

		ALandscapeProxy* Proxy = LandscapeInfo->GetLandscapeProxy();
		if (!Proxy)
		{
			return false;
		}

		AddCollision.Reset();

		FVector IntersectPoint;
		// Need to optimize collision for AddLandscapeComponent...?
		for (auto& XYToAddCollisionPair : LandscapeInfo->XYtoAddCollisionMap)
		{
			FLandscapeAddCollision& CurrentAddCollision = XYToAddCollisionPair.Value;
			// Triangle 1
			if(RayIntersectTriangle(TraceStart, TraceEnd, CurrentAddCollision.Corners[0], CurrentAddCollision.Corners[3], CurrentAddCollision.Corners[1], IntersectPoint))
			{
				AddCollision = CurrentAddCollision;
				OutHitLocation = Proxy->LandscapeActorToWorld().InverseTransformPosition(IntersectPoint);
				return true;
			}
			// Triangle 2
			if(RayIntersectTriangle(TraceStart, TraceEnd, CurrentAddCollision.Corners[0], CurrentAddCollision.Corners[2], CurrentAddCollision.Corners[3], IntersectPoint))
			{
				AddCollision = CurrentAddCollision;
				OutHitLocation = Proxy->LandscapeActorToWorld().InverseTransformPosition(IntersectPoint);
				return true;
			}
		}
		
		return false;
	}

	virtual int32 GetToolActionResolutionDelta() const override
	{
		int32 ResolutionDelta = 0;

		if (EdMode == nullptr)
		{
			return 0;
		}

		const FLandscapeToolTarget& ToolTarget = EdMode->CurrentToolTarget;
		FLandscapeBrush* CurrentBrush = EdMode->CurrentBrush;
		TOptional<FVector2D> LastMousePosition = CurrentBrush->GetLastMousePosition();
		FIntRect LandscapeIndices;

		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();

		if ( LandscapeInfo != nullptr && ToolTarget.LandscapeInfo.IsValid() && LastMousePosition.IsSet() && ToolTarget.LandscapeInfo->GetLandscapeXYComponentBounds(LandscapeIndices))
		{
			const int32 BrushSize = FMath::Max(EdMode->UISettings->BrushComponentSize, 0);
			const int32 ComponentSizeQuads = ToolTarget.LandscapeInfo->ComponentSizeQuads;
			const float BrushOriginX = static_cast<float>(LastMousePosition.GetValue().X / ComponentSizeQuads - (BrushSize - 1) / 2.0);
			const float BrushOriginY = static_cast<float>(LastMousePosition.GetValue().Y / ComponentSizeQuads - (BrushSize - 1) / 2.0);
			const int32 ComponentIndexX = FMath::FloorToInt(BrushOriginX);
			const int32 ComponentIndexY = FMath::FloorToInt(BrushOriginY);

			int32 NumNewComponents = 0;
			
			int32 HalfBrushSize = BrushSize / 2;
			FIntRect BrushSupport{ ComponentIndexX - HalfBrushSize, ComponentIndexY - HalfBrushSize, ComponentIndexX + HalfBrushSize, ComponentIndexY + HalfBrushSize };
			for (int32 Y = BrushSupport.Min.Y; Y <= BrushSupport.Max.Y; Y++)
			{
				for (int32 X = BrushSupport.Min.X; X <= BrushSupport.Max.X; X++)
				{
					NumNewComponents += LandscapeInfo->XYtoComponentMap.FindRef(FIntPoint(X, Y)) == nullptr ? 1 : 0;
				}
			}
			
			return NumNewComponents * ComponentSizeQuads * ComponentSizeQuads;
		}

		return 0;
	}

private:
	bool RayIntersectTriangle(const FVector& Start, const FVector& End, const FVector& A, const FVector& B, const FVector& C, FVector& IntersectPoint)
	{
		const FVector BA = A - B;
		const FVector CB = B - C;
		const FVector TriNormal = BA ^ CB;

		bool bCollide = FMath::SegmentPlaneIntersection(Start, End, FPlane(A, TriNormal), IntersectPoint);
		if (!bCollide)
		{
			return false;
		}

		FVector BaryCentric = FMath::ComputeBaryCentric2D(IntersectPoint, A, B, C);
		if (BaryCentric.X > 0.0f && BaryCentric.Y > 0.0f && BaryCentric.Z > 0.0f)
		{
			return true;
		}
		return false;
	}

	TOptional<FLandscapeAddCollision> AddCollision;
	bool bIsToolActionResolutionCompliant;
};

//
// FLandscapeToolDeleteComponent
//
class FLandscapeToolStrokeDeleteComponent : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeDeleteComponent(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeDeleteComponent_Apply);

		if (LandscapeInfo)
		{
			auto SelectedComponents = LandscapeInfo->GetSelectedComponents();
			if (SelectedComponents.Num() == 0)
			{
				// Get list of components to delete from brush
				// TODO - only retrieve bounds as we don't need the vert data
				FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
				if (!BrushInfo)
				{
					return;
				}

				int32 X1, Y1, X2, Y2;
				BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

				// Shrink bounds by 1,1 to avoid GetComponentsInRegion picking up extra components on all sides due to the overlap between components
				LandscapeInfo->GetComponentsInRegion(X1 + 1, Y1 + 1, X2 - 1, Y2 - 1, SelectedComponents);
			}

			// Delete the components
			EdMode->DeleteLandscapeComponents(LandscapeInfo, SelectedComponents);
		}
	}
};

class FLandscapeToolDeleteComponent : public FLandscapeToolBase<FLandscapeToolStrokeDeleteComponent>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeDeleteComponent>;

public:
	FLandscapeToolDeleteComponent(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual bool AffectsEditLayers() const override { return false; }

	virtual const TCHAR* GetToolName() const override { return TEXT("DeleteComponent"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_DeleteComponent", "Delete Landscape Components"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_DeleteComponent_Message", "Delete selected components . If no components are currently selected, deletes the component highlighted under the mouse cursor. "); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectComponent | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }
};

//
// FLandscapeToolCopy
//
template<class ToolTarget>
class FLandscapeToolStrokeCopy : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

protected:
	using CacheType = ToolTarget::CacheClass;
	using ValueType = CacheType::DataType;

public:
	FLandscapeToolStrokeCopy(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, Cache(InTarget.LandscapeInfo.Get(), nullptr) // TODO [jonathan.bard] : this won't work for weightmaps
		, HeightCache(InTarget.LandscapeInfo.Get())
		, WeightCache(InTarget.LandscapeInfo.Get())
	{
	}

	struct FGizmoPreData
	{
		float Ratio;
		float Data;
	};

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeCopy_Apply);

		//ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo;
		ALandscapeGizmoActiveActor* Gizmo = EdMode->UISettings->CurrentGizmoActor.Get();
		if (LandscapeInfo && Gizmo && Gizmo->GizmoTexture && Gizmo->GetRootComponent())
		{
			Gizmo->TargetLandscapeInfo = LandscapeInfo;

			// Get list of verts to update
			// TODO - only retrieve bounds as we don't need the data
			FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			//Gizmo->Modify(); // No transaction for Copied data as other tools...
			//Gizmo->SelectedData.Empty();
			Gizmo->ClearGizmoData();

			// Tablet pressure
			//float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

			bool bApplyToAll = EdMode->UISettings->bApplyToAllTargets;
			const int32 LayerNum = LandscapeInfo->Layers.Num();

			TArray<uint16> HeightData;
			TArray<uint8> WeightDatas; // Weight*Layers...
			TArray<ValueType> Data;

			TSet<ULandscapeLayerInfoObject*> LayerInfoSet;

			if (bApplyToAll)
			{
				HeightCache.CacheData(X1, Y1, X2, Y2);
				HeightCache.GetCachedData(X1, Y1, X2, Y2, HeightData);

				WeightCache.CacheData(X1, Y1, X2, Y2);
				WeightCache.GetCachedData(X1, Y1, X2, Y2, WeightDatas, LayerNum);
			}
			else
			{
				Cache.CacheData(X1, Y1, X2, Y2);
				Cache.GetCachedData(X1, Y1, X2, Y2, Data);
			}

			const float ScaleXY = static_cast<float>(LandscapeInfo->DrawScale.X);
			float Width = Gizmo->GetWidth();
			float Height = Gizmo->GetHeight();

			Gizmo->CachedWidth = Width;
			Gizmo->CachedHeight = Height;
			Gizmo->CachedScaleXY = ScaleXY;

			// Rasterize Gizmo regions
			int32 SizeX = FMath::CeilToInt(Width / ScaleXY);
			int32 SizeY = FMath::CeilToInt(Height / ScaleXY);

			const float W = (Width - ScaleXY) / (2 * ScaleXY);
			const float H = (Height - ScaleXY) / (2 * ScaleXY);

			FMatrix WToL = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().ToMatrixWithScale().InverseFast();
			//FMatrix LToW = Landscape->LocalToWorld();

			FVector BaseLocation = WToL.TransformPosition(Gizmo->GetActorLocation());
			FMatrix GizmoLocalToLandscape = FRotationTranslationMatrix(FRotator(0, Gizmo->GetActorRotation().Yaw, 0), FVector(BaseLocation.X, BaseLocation.Y, 0));

			const int32 NeighborNum = 4;
			bool bDidCopy = false;
			bool bFullCopy = !EdMode->UISettings->bUseSelectedRegion || !LandscapeInfo->SelectedRegion.Num();
			//bool bInverted = EdMode->UISettings->bUseSelectedRegion && EdMode->UISettings->bUseNegativeMask;

			// TODO: This is a mess and badly needs refactoring
			for (int32 Y = 0; Y < SizeY; ++Y)
			{
				for (int32 X = 0; X < SizeX; ++X)
				{
					FVector LandscapeLocal = GizmoLocalToLandscape.TransformPosition(FVector(-W + X, -H + Y, 0));
					const int32 LX = FMath::FloorToInt32(LandscapeLocal.X);
					const int32 LY = FMath::FloorToInt32(LandscapeLocal.Y);

					{
						for (int32 i = -1; (!bApplyToAll && i < 0) || i < LayerNum; ++i)
						{
							// Don't try to copy data for null layers
							if ((bApplyToAll && i >= 0 && !LandscapeInfo->Layers[i].LayerInfoObj) ||
								(!bApplyToAll && (EdMode->CurrentToolTarget.TargetType != ELandscapeToolTargetType::Heightmap) && !EdMode->CurrentToolTarget.LayerInfo.Get()))
							{
								continue;
							}

							FGizmoPreData GizmoPreData[NeighborNum];

							for (int32 LocalY = 0; LocalY < 2; ++LocalY)
							{
								for (int32 LocalX = 0; LocalX < 2; ++LocalX)
								{
									int32 x = FMath::Clamp(LX + LocalX, X1, X2);
									int32 y = FMath::Clamp(LY + LocalY, Y1, Y2);
									GizmoPreData[LocalX + LocalY * 2].Ratio = LandscapeInfo->SelectedRegion.FindRef(FIntPoint(x, y));
									int32 index = (x - X1) + (y - Y1)*(1 + X2 - X1);

									if (bApplyToAll)
									{
										if (i < 0)
										{
											GizmoPreData[LocalX + LocalY * 2].Data = Gizmo->GetNormalizedHeight(HeightData[index]);
										}
										else
										{
											GizmoPreData[LocalX + LocalY * 2].Data = WeightDatas[index*LayerNum + i];
										}
									}
									else
									{
										ValueType OriginalValue = Data[index];
										if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
										{
											GizmoPreData[LocalX + LocalY * 2].Data = Gizmo->GetNormalizedHeight(OriginalValue);
										}
										else
										{
											GizmoPreData[LocalX + LocalY * 2].Data = OriginalValue;
										}
									}
								}
							}

							FGizmoPreData LerpedData;
							const float FracX = static_cast<float>(LandscapeLocal.X - LX);
							const float FracY = static_cast<float>(LandscapeLocal.Y - LY);
							LerpedData.Ratio = bFullCopy ? 1.0f :
								FMath::Lerp(
								FMath::Lerp(GizmoPreData[0].Ratio, GizmoPreData[1].Ratio, FracX),
								FMath::Lerp(GizmoPreData[2].Ratio, GizmoPreData[3].Ratio, FracX),
								FracY
								);

							LerpedData.Data = FMath::Lerp(
								FMath::Lerp(GizmoPreData[0].Data, GizmoPreData[1].Data, FracX),
								FMath::Lerp(GizmoPreData[2].Data, GizmoPreData[3].Data, FracX),
								FracY
								);

							if (!bDidCopy && LerpedData.Ratio > 0.0f)
							{
								bDidCopy = true;
							}

							if (LerpedData.Ratio > 0.0f)
							{
								// Added for LayerNames
								if (bApplyToAll)
								{
									if (i >= 0)
									{
										LayerInfoSet.Add(LandscapeInfo->Layers[i].LayerInfoObj);
									}
								}
								else
								{
									if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
									{
										LayerInfoSet.Add(EdMode->CurrentToolTarget.LayerInfo.Get());
									}
								}

								FGizmoSelectData* GizmoSelectData = Gizmo->SelectedData.Find(FIntPoint(X, Y));
								if (GizmoSelectData)
								{
									if (bApplyToAll)
									{
										if (i < 0)
										{
											GizmoSelectData->HeightData = LerpedData.Data;
										}
										else
										{
											GizmoSelectData->WeightDataMap.Add(LandscapeInfo->Layers[i].LayerInfoObj, LerpedData.Data);
										}
									}
									else
									{
										if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
										{
											GizmoSelectData->HeightData = LerpedData.Data;
										}
										else
										{
											GizmoSelectData->WeightDataMap.Add(EdMode->CurrentToolTarget.LayerInfo.Get(), LerpedData.Data);
										}
									}
								}
								else
								{
									FGizmoSelectData NewData;
									NewData.Ratio = LerpedData.Ratio;
									if (bApplyToAll)
									{
										if (i < 0)
										{
											NewData.HeightData = LerpedData.Data;
										}
										else
										{
											NewData.WeightDataMap.Add(LandscapeInfo->Layers[i].LayerInfoObj, LerpedData.Data);
										}
									}
									else
									{
										if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
										{
											NewData.HeightData = LerpedData.Data;
										}
										else
										{
											NewData.WeightDataMap.Add(EdMode->CurrentToolTarget.LayerInfo.Get(), LerpedData.Data);
										}
									}
									Gizmo->SelectedData.Add(FIntPoint(X, Y), NewData);
								}
							}
						}
					}
				}
			}

			if (bDidCopy)
			{
				if (!bApplyToAll)
				{
					if (EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
					{
						Gizmo->DataType = ELandscapeGizmoType(Gizmo->DataType | LGT_Height);
					}
					else
					{
						Gizmo->DataType = ELandscapeGizmoType(Gizmo->DataType | LGT_Weight);
					}
				}
				else
				{
					if (LayerNum > 0)
					{
						Gizmo->DataType = ELandscapeGizmoType(Gizmo->DataType | LGT_Height);
						Gizmo->DataType = ELandscapeGizmoType(Gizmo->DataType | LGT_Weight);
					}
					else
					{
						Gizmo->DataType = ELandscapeGizmoType(Gizmo->DataType | LGT_Height);
					}
				}

				Gizmo->SampleData(SizeX, SizeY);

				// Update LayerInfos
				for (ULandscapeLayerInfoObject* LayerInfo : LayerInfoSet)
				{
					Gizmo->LayerInfos.Add(LayerInfo);
				}
			}

			//// Clean up Ratio 0 regions... (That was for sampling...)
			//for ( TMap<uint64, FGizmoSelectData>::TIterator It(Gizmo->SelectedData); It; ++It )
			//{
			//	FGizmoSelectData& Data = It.Value();
			//	if (Data.Ratio <= 0.0f)
			//	{
			//		Gizmo->SelectedData.Remove(It.Key());
			//	}
			//}

			Gizmo->ExportToClipboard();

			GEngine->BroadcastLevelActorListChanged();
		}
	}

protected:
	CacheType Cache;
	FLandscapeHeightCache HeightCache;
	FLandscapeFullWeightCache WeightCache;
};

template<class ToolTarget>
class FLandscapeToolCopy : public FLandscapeToolBase<FLandscapeToolStrokeCopy<ToolTarget>>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokeCopy<ToolTarget>>;

public:
	FLandscapeToolCopy(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
		, BackupCurrentBrush(nullptr)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Copy"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Copy", "Copy"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Copy_Message", "Copy and Paste allows you to copy terrain data from one area of your Landscape to another.  Use the select tool  in conjunction with the Copy gizmo to further refine your selection."); };


	virtual void SetEditRenderType() override
	{
		GLandscapeEditRenderMode = ELandscapeEditRenderMode::Gizmo | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask);
		GLandscapeEditRenderMode |= (this->EdMode && this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() && this->EdMode->CurrentToolTarget.LandscapeInfo->SelectedRegion.Num()) ? ELandscapeEditRenderMode::SelectRegion : ELandscapeEditRenderMode::SelectComponent;
	}

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ToolTarget::TargetType;
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& InTarget, const FVector& InHitLocation) override
	{
		this->EdMode->GizmoBrush->Tick(ViewportClient, 0.1f);

		// horrible hack
		// (but avoids duplicating the code from FLandscapeToolBase)
		BackupCurrentBrush = this->EdMode->CurrentBrush;
		this->EdMode->CurrentBrush = this->EdMode->GizmoBrush;

		return Super::BeginTool(ViewportClient, InTarget, InHitLocation);
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		Super::EndTool(ViewportClient);

		this->EdMode->CurrentBrush = BackupCurrentBrush;
	}

protected:
	FLandscapeBrush* BackupCurrentBrush;
};

//
// FLandscapeToolPaste
//
template<class ToolTarget>
class FLandscapeToolStrokePaste : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

protected:
	using CacheType = ToolTarget::CacheClass;
	using ValueType = CacheType::DataType;

public:
	FLandscapeToolStrokePaste(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, Cache(InTarget.LandscapeInfo.Get(), nullptr) // this won't work for weightmaps (we need to pass the targeted target layers), but this tool is currently not implemented for that case
		, HeightCache(InTarget.LandscapeInfo.Get())
		, WeightCache(InTarget.LandscapeInfo.Get())
	{
		static_assert(ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap, "This tool is currently only implemented for FHeightmapToolTarget as of now");
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokePaste_Apply);

		//ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo;
		ALandscapeGizmoActiveActor* Gizmo = EdMode->UISettings->CurrentGizmoActor.Get();
		// Cache and copy in Gizmo's region...
		if (LandscapeInfo && Gizmo && Gizmo->GetRootComponent())
		{
			if (Gizmo->SelectedData.Num() == 0)
			{
				return;
			}

			// Automatically fill in any placeholder layers
			// This gives a much better user experience when copying data to a newly created landscape
			for (ULandscapeLayerInfoObject* LayerInfo : Gizmo->LayerInfos)
			{
				int32 LayerInfoIndex = LandscapeInfo->GetLayerInfoIndex(LayerInfo);
				if (LayerInfoIndex == INDEX_NONE)
				{
					LayerInfoIndex = LandscapeInfo->GetLayerInfoIndex(LayerInfo->GetLayerName());
					if (LayerInfoIndex != INDEX_NONE)
					{
						FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[LayerInfoIndex];

						if (LayerSettings.LayerInfoObj == nullptr)
						{
							LayerSettings.Owner = LandscapeInfo->GetLandscapeProxy(); // this isn't strictly accurate, but close enough
							LayerSettings.LayerInfoObj = LayerInfo;
						}
					}
				}
			}

			Gizmo->TargetLandscapeInfo = LandscapeInfo;
			float ScaleXY = static_cast<float>(LandscapeInfo->DrawScale.X);

			//LandscapeInfo->Modify();

			// Get list of verts to update
			FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
			if (!BrushInfo)
			{
				return;
			}

			int32 X1, Y1, X2, Y2;
			BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

			// Tablet pressure
			float Pressure = (ViewportClient && ViewportClient->Viewport->IsPenActive()) ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

			// expand the area by one vertex in each direction to ensure normals are calculated correctly
			X1 -= 1;
			Y1 -= 1;
			X2 += 1;
			Y2 += 1;

			const bool bApplyToAll = EdMode->UISettings->bApplyToAllTargets;
			const int32 LayerNum = Gizmo->LayerInfos.Num() > 0 ? LandscapeInfo->Layers.Num() : 0;

			TArray<uint16> HeightData;
			TArray<uint8> WeightDatas; // Weight*Layers...
			TArray<ValueType> Data;

			if (bApplyToAll)
			{
				HeightCache.CacheData(X1, Y1, X2, Y2);
				HeightCache.GetCachedData(X1, Y1, X2, Y2, HeightData);

				if (LayerNum > 0)
				{
					WeightCache.CacheData(X1, Y1, X2, Y2);
					WeightCache.GetCachedData(X1, Y1, X2, Y2, WeightDatas, LayerNum);
				}
			}
			else
			{
				Cache.CacheData(X1, Y1, X2, Y2);
				Cache.GetCachedData(X1, Y1, X2, Y2, Data);
			}

			const float Width = Gizmo->GetWidth();
			const float Height = Gizmo->GetHeight();

			const float W = Gizmo->GetWidth() / (2 * ScaleXY);
			const float H = Gizmo->GetHeight() / (2 * ScaleXY);

			const FVector GizmoScale3D = Gizmo->GetRootComponent()->GetRelativeScale3D();
			const float SignX = GizmoScale3D.X > 0.0f ? 1.0f : -1.0f;
			const float SignY = GizmoScale3D.Y > 0.0f ? 1.0f : -1.0f;

			const float ScaleX = Gizmo->CachedWidth / Width * ScaleXY / Gizmo->CachedScaleXY;
			const float ScaleY = Gizmo->CachedHeight / Height * ScaleXY / Gizmo->CachedScaleXY;

			FMatrix WToL = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().ToMatrixWithScale().InverseFast();
			//FMatrix LToW = Landscape->LocalToWorld();
			FVector BaseLocation = WToL.TransformPosition(Gizmo->GetActorLocation());
			//FMatrix LandscapeLocalToGizmo = FRotationTranslationMatrix(FRotator(0, Gizmo->Rotation.Yaw, 0), FVector(BaseLocation.X - W + 0.5, BaseLocation.Y - H + 0.5, 0));
			FMatrix LandscapeToGizmoLocal =
				(FTranslationMatrix(FVector((-W + 0.5)*SignX, (-H + 0.5)*SignY, 0)) * FScaleRotationTranslationMatrix(FVector(SignX, SignY, 1.0f), FRotator(0, Gizmo->GetActorRotation().Yaw, 0), FVector(BaseLocation.X, BaseLocation.Y, 0))).InverseFast();

			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						// TODO: This is a mess and badly needs refactoring

						// Value before we apply our painting
						int32 index = (X - X1) + (Y - Y1)*(1 + X2 - X1);
						float PaintAmount = (Brush->GetBrushType() == ELandscapeBrushType::Gizmo) ? BrushValue : BrushValue * EdMode->UISettings->GetCurrentToolStrength() * Pressure;

						FVector GizmoLocal = LandscapeToGizmoLocal.TransformPosition(FVector(X, Y, 0));
						GizmoLocal.X *= ScaleX * SignX;
						GizmoLocal.Y *= ScaleY * SignY;

						const int32 LX = FMath::FloorToInt32(GizmoLocal.X);
						const int32 LY = FMath::FloorToInt32(GizmoLocal.Y);

						const float FracX = static_cast<float>(GizmoLocal.X - LX);
						const float FracY = static_cast<float>(GizmoLocal.Y - LY);

						FGizmoSelectData* Data00 = Gizmo->SelectedData.Find(FIntPoint(LX, LY));
						FGizmoSelectData* Data10 = Gizmo->SelectedData.Find(FIntPoint(LX + 1, LY));
						FGizmoSelectData* Data01 = Gizmo->SelectedData.Find(FIntPoint(LX, LY + 1));
						FGizmoSelectData* Data11 = Gizmo->SelectedData.Find(FIntPoint(LX + 1, LY + 1));

						for (int32 i = -1; (!bApplyToAll && i < 0) || i < LayerNum; ++i)
						{
							if ((bApplyToAll && (i < 0)) || (!bApplyToAll && EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap))
							{
								float OriginalValue;
								if (bApplyToAll)
								{
									OriginalValue = HeightData[index];
								}
								else
								{
									OriginalValue = Data[index];
								}

								float Value = LandscapeDataAccess::GetLocalHeight(static_cast<uint16>(OriginalValue));

								float DestValue = FLandscapeHeightCache::ClampValue(
									LandscapeDataAccess::GetTexHeight(
									FMath::Lerp(
									FMath::Lerp(Data00 ? FMath::Lerp(Value, Gizmo->GetLandscapeHeight(Data00->HeightData), Data00->Ratio) : Value,
									Data10 ? FMath::Lerp(Value, Gizmo->GetLandscapeHeight(Data10->HeightData), Data10->Ratio) : Value, FracX),
									FMath::Lerp(Data01 ? FMath::Lerp(Value, Gizmo->GetLandscapeHeight(Data01->HeightData), Data01->Ratio) : Value,
									Data11 ? FMath::Lerp(Value, Gizmo->GetLandscapeHeight(Data11->HeightData), Data11->Ratio) : Value, FracX),
									FracY
									))
									);

								switch (EdMode->UISettings->PasteMode)
								{
								case ELandscapeToolPasteMode::Raise:
									PaintAmount = OriginalValue < DestValue ? PaintAmount : 0.0f;
									break;
								case ELandscapeToolPasteMode::Lower:
									PaintAmount = OriginalValue > DestValue ? PaintAmount : 0.0f;
									break;
								default:
									break;
								}

								if (bApplyToAll)
								{
									HeightData[index] = static_cast<uint16>(FMath::Lerp(OriginalValue, DestValue, PaintAmount));
								}
								else
								{
									Data[index] = static_cast<uint16>(FMath::Lerp(OriginalValue, DestValue, PaintAmount));
								}
							}
							else
							{
								ULandscapeLayerInfoObject* LayerInfo;
								float OriginalValue;
								if (bApplyToAll)
								{
									LayerInfo = LandscapeInfo->Layers[i].LayerInfoObj;
									OriginalValue = WeightDatas[index*LayerNum + i];
								}
								else
								{
									LayerInfo = EdMode->CurrentToolTarget.LayerInfo.Get();
									OriginalValue = Data[index];
								}

								float DestValue = FLandscapeAlphaCache::ClampValue(static_cast<int32>(
									FMath::Lerp(
									FMath::Lerp(Data00 ? FMath::Lerp(OriginalValue, Data00->WeightDataMap.FindRef(LayerInfo), Data00->Ratio) : OriginalValue,
									Data10 ? FMath::Lerp(OriginalValue, Data10->WeightDataMap.FindRef(LayerInfo), Data10->Ratio) : OriginalValue, FracX),
									FMath::Lerp(Data01 ? FMath::Lerp(OriginalValue, Data01->WeightDataMap.FindRef(LayerInfo), Data01->Ratio) : OriginalValue,
									Data11 ? FMath::Lerp(OriginalValue, Data11->WeightDataMap.FindRef(LayerInfo), Data11->Ratio) : OriginalValue, FracX),
									FracY
									)));

								if (bApplyToAll)
								{
									WeightDatas[index*LayerNum + i] = static_cast<uint8>(FMath::Lerp(OriginalValue, DestValue, PaintAmount));
								}
								else
								{
									Data[index] = static_cast<ValueType>(FMath::Lerp(OriginalValue, DestValue, PaintAmount));
								}
							}
						}
					}
				}
			}

			for (ULandscapeLayerInfoObject* LayerInfo : Gizmo->LayerInfos)
			{
				if (LandscapeInfo->GetLayerInfoIndex(LayerInfo) != INDEX_NONE)
				{
					WeightCache.AddDirtyLayer(LayerInfo);
				}
			}

			if (bApplyToAll)
			{
				HeightCache.SetCachedData(X1, Y1, X2, Y2, HeightData);
				HeightCache.Flush();
				if (WeightDatas.Num())
				{
					// Set the layer data, bypassing painting restrictions because it doesn't work well when altering multiple layers
					WeightCache.SetCachedData(X1, Y1, X2, Y2, WeightDatas, LayerNum, ELandscapeLayerPaintingRestriction::None);
				}
				WeightCache.Flush();
			}
			else
			{
				Cache.SetCachedData(X1, Y1, X2, Y2, Data);
				Cache.Flush();
			}

			// Dirty any runtime virtual textures that our landscape components write to.
			LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);

			GEngine->BroadcastLevelActorListChanged();
		}
	}

protected:
	CacheType Cache;
	FLandscapeHeightCache HeightCache;
	FLandscapeFullWeightCache WeightCache;
};

template<class ToolTarget>
class FLandscapeToolPaste : public FLandscapeToolBase<FLandscapeToolStrokePaste<ToolTarget>>
{
	using Super = FLandscapeToolBase<FLandscapeToolStrokePaste<ToolTarget>>;

public:
	FLandscapeToolPaste(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
		, bUseGizmoRegion(false)
		, BackupCurrentBrush(nullptr)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Paste"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Region", "Region Copy/Paste"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Region_Message", "Copy and Paste allows you to copy terrain data from one area of your Landscape to another.  Use the select tool  in conjunction with the Copy gizmo to further refine your selection."); };

	virtual void SetEditRenderType() override
	{
		GLandscapeEditRenderMode = ELandscapeEditRenderMode::Gizmo | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask);
		GLandscapeEditRenderMode |= (this->EdMode && this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() && this->EdMode->CurrentToolTarget.LandscapeInfo->SelectedRegion.Num()) ? ELandscapeEditRenderMode::SelectRegion : ELandscapeEditRenderMode::SelectComponent;
	}

	void SetGizmoMode(bool InbUseGizmoRegion)
	{
		bUseGizmoRegion = InbUseGizmoRegion;
	}

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ToolTarget::TargetType;
	}

	virtual ELandscapeLayerUpdateMode GetBeginToolContentUpdateFlag() const override { return ELandscapeLayerUpdateMode::Update_All_Editing; }

	virtual ELandscapeLayerUpdateMode GetTickToolContentUpdateFlag() const override { return GetBeginToolContentUpdateFlag(); }
	
	virtual ELandscapeLayerUpdateMode GetEndToolContentUpdateFlag() const override { return ELandscapeLayerUpdateMode::Update_All; }
		
	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& InTarget, const FVector& InHitLocation) override
	{
		this->EdMode->GizmoBrush->Tick(ViewportClient, 0.1f);

		// horrible hack
		// (but avoids duplicating the code from FLandscapeToolBase)
		BackupCurrentBrush = this->EdMode->CurrentBrush;
		if (bUseGizmoRegion)
		{
			this->EdMode->CurrentBrush = this->EdMode->GizmoBrush;
		}

		return Super::BeginTool(ViewportClient, InTarget, InHitLocation);
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		Super::EndTool(ViewportClient);

		if (bUseGizmoRegion)
		{
			this->EdMode->CurrentBrush = BackupCurrentBrush;
		}
		check(this->EdMode->CurrentBrush == BackupCurrentBrush);
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		if (bUseGizmoRegion)
		{
			return true;
		}

		return FLandscapeToolBase<FLandscapeToolStrokePaste<ToolTarget>>::MouseMove(ViewportClient, Viewport, x, y);
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		ALandscapeGizmoActiveActor* Gizmo = this->EdMode->UISettings->CurrentGizmoActor.Get();
		// Cache and copy in Gizmo's region...
		if (this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() && Gizmo && Gizmo->GetRootComponent())
		{
			if (Gizmo->SelectedData.Num() == 0)
			{
				const bool bApplyToAll = this->EdMode->UISettings->bApplyToAllTargets;
				if (this->EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap)
				{
					return ELandscapeTargetLayerActionType::Normal;
				}
				else if (bApplyToAll)
				{
					// This tool will only change values on a layer if it is part of the copied data :
					return Gizmo->LayerInfos.Contains(InLayerInfo) ? ELandscapeTargetLayerActionType::Normal : ELandscapeTargetLayerActionType::None;
				}
				else
				{
					// This is not handled ATM (see comment in ctor) and will need to be addressed if the copy paste tool becomes available in paint mode 
					checkNoEntry();
				}
			}
		}

		return ELandscapeTargetLayerActionType::None;
	}

protected:
	bool bUseGizmoRegion;
	FLandscapeBrush* BackupCurrentBrush;
};

//
// FLandscapeToolCopyPaste
//
template<class ToolTarget>
class FLandscapeToolCopyPaste : public FLandscapeToolPaste<ToolTarget>
{
public:
	FLandscapeToolCopyPaste(FEdModeLandscape* InEdMode)
		: FLandscapeToolPaste<ToolTarget>(InEdMode)
		, CopyTool(InEdMode)
	{
	}

	// Just hybrid of Copy and Paste tool
	virtual const TCHAR* GetToolName() const override { return TEXT("CopyPaste"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Region", "Region Copy/Paste"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Region_Message", "Copy and Paste allows you to copy terrain data from one area of your Landscape to another.  Use the select tool  in conjunction with the Copy gizmo to further refine your selection."); };

	virtual void EnterTool() override
	{
		// Make sure gizmo actor is selected
		ALandscapeGizmoActiveActor* Gizmo = this->EdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo)
		{
			Gizmo->bFollowTerrainHeight = false;
			GEditor->SelectNone(false, true);
			GEditor->SelectActor(Gizmo, true, true, true);
		}
	}

	virtual void ExitTool() override
	{
		if (ALandscapeGizmoActiveActor* Gizmo = this->EdMode->UISettings->CurrentGizmoActor.Get())
		{
			Gizmo->bFollowTerrainHeight = true;
		}
	}

	// Copy tool doesn't use any view information, so just do it as one function
	void Copy()
	{
		CopyTool.BeginTool(nullptr, this->EdMode->CurrentToolTarget, FVector::ZeroVector);
		CopyTool.EndTool(nullptr);
	}

	void Paste()
	{
		this->SetGizmoMode(true);
		this->BeginTool(nullptr, this->EdMode->CurrentToolTarget, FVector::ZeroVector);
		this->EndTool(nullptr);
		this->SetGizmoMode(false);
	}

protected:
	FLandscapeToolCopy<ToolTarget> CopyTool;
};

void FEdModeLandscape::CopyDataToGizmo()
{
	// For Copy operation...
	if (CopyPasteTool /*&& CopyPasteTool == CurrentTool*/)
	{
		CopyPasteTool->Copy();
	}
	if (UISettings->CurrentGizmoActor.IsValid())
	{
		GEditor->SelectNone(false, true);
		GEditor->SelectActor(UISettings->CurrentGizmoActor.Get(), true, true, true);
	}
}

void FEdModeLandscape::PasteDataFromGizmo()
{
	// For Paste for Gizmo Region operation...
	if (CopyPasteTool /*&& CopyPasteTool == CurrentTool*/)
	{
		CopyPasteTool->Paste();
	}
	if (UISettings->CurrentGizmoActor.IsValid())
	{
		GEditor->SelectNone(false, true);
		GEditor->SelectActor(UISettings->CurrentGizmoActor.Get(), true, true, true);
	}
}

namespace ELandscapeEdge
{
	enum Type
	{
		None,

		// Edges
		X_Negative,
		X_Positive,
		Y_Negative,
		Y_Positive,

		// Corners
		X_Negative_Y_Negative,
		X_Positive_Y_Negative,
		X_Negative_Y_Positive,
		X_Positive_Y_Positive,
	};
}

struct HNewLandscapeGrabHandleProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	ELandscapeEdge::Type Edge;

	HNewLandscapeGrabHandleProxy(ELandscapeEdge::Type InEdge) :
		HHitProxy(HPP_Wireframe),
		Edge(InEdge)
	{
	}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		switch (Edge)
		{
		case ELandscapeEdge::X_Negative:
		case ELandscapeEdge::X_Positive:
			return EMouseCursor::ResizeLeftRight;
		case ELandscapeEdge::Y_Negative:
		case ELandscapeEdge::Y_Positive:
			return EMouseCursor::ResizeUpDown;
		case ELandscapeEdge::X_Negative_Y_Negative:
		case ELandscapeEdge::X_Positive_Y_Positive:
			return EMouseCursor::ResizeSouthEast;
		case ELandscapeEdge::X_Negative_Y_Positive:
		case ELandscapeEdge::X_Positive_Y_Negative:
			return EMouseCursor::ResizeSouthWest;
		}

		return EMouseCursor::SlashedCircle;
	}
};

IMPLEMENT_HIT_PROXY(HNewLandscapeGrabHandleProxy, HHitProxy)

//
// FLandscapeToolNewLandscape
//
class FLandscapeToolNewLandscape : public FLandscapeTool
{
public:
	FEdModeLandscape* EdMode;
	ENewLandscapePreviewMode NewLandscapePreviewMode;
	ELandscapeEdge::Type DraggingEdge;
	float DraggingEdge_Remainder;

	FLandscapeToolNewLandscape(FEdModeLandscape* InEdMode)
		: FLandscapeTool()
		, EdMode(InEdMode)
		, NewLandscapePreviewMode(ENewLandscapePreviewMode::NewLandscape)
		, DraggingEdge(ELandscapeEdge::None)
		, DraggingEdge_Remainder(0.0f)
	{
	}
	virtual bool AffectsEditLayers() const { return false; }
	virtual const TCHAR* GetToolName() const override { return TEXT("NewLandscape"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_NewLandscape", "New Landscape"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_NewLandscape_Message", "Create or import a new heightmap.  Assign a material and configure the components.  When you are ready to create your new Landscape, press the Create button in the lower-right corner of this panel. "); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual void EnterTool() override
	{
		DraggingEdge = ELandscapeEdge::None;
		DraggingEdge_Remainder = 0.0f;
		EdMode->NewLandscapePreviewMode = NewLandscapePreviewMode;
		EdMode->UISettings->RefreshImports();
	}

	virtual void ExitTool() override
	{
		NewLandscapePreviewMode = EdMode->NewLandscapePreviewMode;
		EdMode->NewLandscapePreviewMode = ENewLandscapePreviewMode::None;
		EdMode->UISettings->ClearImportLandscapeData();
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		// does nothing
		return false;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		// does nothing
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		// does nothing
		return false;
	}

	virtual bool InputKey(FEditorViewportClient* ViewportClient, FViewport* Viewport, FKey Key, EInputEvent Event) override 
	{ 
		if (Key == EKeys::LeftMouseButton)
		{
			// Press mouse button
			if (Event == IE_Pressed && !IsAltDown(Viewport))
			{
				// See if we clicked on a new landscape handle..
				int32 HitX = Viewport->GetMouseX();
				int32 HitY = Viewport->GetMouseY();
				HHitProxy* HitProxy = Viewport->GetHitProxy(HitX, HitY);
				if (HitProxy)
				{
					if (HitProxy->IsA(HNewLandscapeGrabHandleProxy::StaticGetType()))
					{
						HNewLandscapeGrabHandleProxy* EdgeProxy = (HNewLandscapeGrabHandleProxy*)HitProxy;
						DraggingEdge = EdgeProxy->Edge;
						DraggingEdge_Remainder = 0;

						return true;
					}
				}
			}
			else if (Event == IE_Released)
			{
				if (DraggingEdge)
				{
					DraggingEdge = ELandscapeEdge::None;
					DraggingEdge_Remainder = 0;

					return true;
				}
			}
		}
		
		return false; 
	}

	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override
	{
		ULandscapeEditorObject* UISettings = EdMode->UISettings;
		if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
		{
			FVector DeltaScale = InScale;
			DeltaScale.X = DeltaScale.Y = (FMath::Abs(InScale.X) > FMath::Abs(InScale.Y)) ? InScale.X : InScale.Y;

			UISettings->Modify();
			UISettings->NewLandscape_Location += InDrag;
			UISettings->NewLandscape_Rotation += InRot;
			UISettings->NewLandscape_Scale += DeltaScale;

			return true;
		}
		else if (DraggingEdge != ELandscapeEdge::None)
		{
			FVector HitLocation;
			EdMode->LandscapePlaneTrace(InViewportClient, FPlane(UISettings->NewLandscape_Location, FVector(0, 0, 1)), HitLocation);

			FTransform Transform(UISettings->NewLandscape_Rotation, UISettings->NewLandscape_Location, UISettings->NewLandscape_Scale * UISettings->NewLandscape_QuadsPerSection * UISettings->NewLandscape_SectionsPerComponent);
			HitLocation = Transform.InverseTransformPosition(HitLocation);

			UISettings->Modify();

			auto DragEdge = [&UISettings, &HitLocation, &Transform](const ELandscapeEdge::Type Edge)
			{
				int32& ComponentCount = Edge == ELandscapeEdge::X_Negative || Edge == ELandscapeEdge::X_Positive ? UISettings->NewLandscape_ComponentCount.X : UISettings->NewLandscape_ComponentCount.Y;
				const float Hit = static_cast<float>(Edge == ELandscapeEdge::X_Negative || Edge == ELandscapeEdge::X_Positive ? HitLocation.X : HitLocation.Y);
				const float PosOrNeg = Edge == ELandscapeEdge::X_Negative || Edge == ELandscapeEdge::Y_Negative ? -1.0f : 1.0f;
				const FVector XOrY = Edge == ELandscapeEdge::X_Negative || Edge == ELandscapeEdge::X_Positive ? FVector(1, 0, 0) : FVector(0, 1, 0);
				
				const int32 InitialComponentCount = ComponentCount;
				const int32 Delta = FMath::RoundToInt(Hit - PosOrNeg * static_cast<float>(InitialComponentCount) / 2.0f);
				ComponentCount = static_cast<int32>(InitialComponentCount + PosOrNeg * Delta);
				UISettings->NewLandscape_ClampSize();
				const float ActualDelta = static_cast<float>(ComponentCount - InitialComponentCount) / 2.0f;
				UISettings->NewLandscape_Location += PosOrNeg * XOrY * Transform.TransformVector(FVector(ActualDelta, ActualDelta, 0));
			};

			if (DraggingEdge == ELandscapeEdge::X_Negative ||
				DraggingEdge == ELandscapeEdge::X_Negative_Y_Negative ||
				DraggingEdge ==	ELandscapeEdge::X_Negative_Y_Positive)
			{
				DragEdge(ELandscapeEdge::X_Negative);
			}

			if (DraggingEdge == ELandscapeEdge::X_Positive ||
				DraggingEdge == ELandscapeEdge::X_Positive_Y_Negative ||
				DraggingEdge ==	ELandscapeEdge::X_Positive_Y_Positive)
			{
				DragEdge(ELandscapeEdge::X_Positive);
			}

			if (DraggingEdge == ELandscapeEdge::Y_Negative ||
				DraggingEdge == ELandscapeEdge::X_Negative_Y_Negative ||
				DraggingEdge == ELandscapeEdge::X_Positive_Y_Negative)
			{
				DragEdge(ELandscapeEdge::Y_Negative);
			}

			if (DraggingEdge == ELandscapeEdge::Y_Positive ||
				DraggingEdge == ELandscapeEdge::X_Negative_Y_Positive ||
				DraggingEdge == ELandscapeEdge::X_Positive_Y_Positive)
			{
				DragEdge(ELandscapeEdge::Y_Positive);
			}
						
			return true;
		}
		
		return false;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		if (EdMode->NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
		{
			static constexpr float        CornerSize = 0.33f;
			static constexpr FLinearColor CornerColor(1.0f, 0.5f, 0.0f);
			static constexpr FLinearColor EdgeColor(1.0f, 1.0f, 0.0f);
			static constexpr FLinearColor ComponentBorderColor(0.0f, 0.85f, 0.0f);
			static constexpr FLinearColor SectionBorderColor(0.0f, 0.4f, 0.0f);
			static constexpr FLinearColor InnerColor(0.0f, 0.25f, 0.0f);

			const ELevelViewportType ViewportType = ((FEditorViewportClient*)Viewport->GetClient())->ViewportType;

			const int32 ComponentCountX = EdMode->UISettings->NewLandscape_ComponentCount.X;
			const int32 ComponentCountY = EdMode->UISettings->NewLandscape_ComponentCount.Y;
			const int32 QuadsPerComponent = EdMode->UISettings->NewLandscape_SectionsPerComponent * EdMode->UISettings->NewLandscape_QuadsPerSection;
			const float ComponentSize = static_cast<float>(QuadsPerComponent);
			const int32 GridSize = EdMode->UISettings->WorldPartitionGridSize;
			const FVector Offset = EdMode->UISettings->NewLandscape_Location + FTransform(EdMode->UISettings->NewLandscape_Rotation, FVector::ZeroVector, EdMode->UISettings->NewLandscape_Scale).TransformVector(FVector(-ComponentCountX * ComponentSize / 2, -ComponentCountY * ComponentSize / 2, 0));
			const FTransform Transform = FTransform(EdMode->UISettings->NewLandscape_Rotation, Offset, EdMode->UISettings->NewLandscape_Scale);

			using LineCoords = TTuple<FVector, FVector>;

			auto DrawLine = [&PDI, &Transform](const LineCoords& AB, const FLinearColor& Color, const uint8 DepthPriorityGroup)
			{
				PDI->DrawLine(Transform.TransformPosition(AB.Get<0>()), Transform.TransformPosition(AB.Get<1>()), Color, DepthPriorityGroup);
			};

			auto DrawLineBorder = [&PDI, &DrawLine](const ELandscapeEdge::Type Edge, const LineCoords& AB, const FLinearColor& Color)
			{
				PDI->SetHitProxy(new HNewLandscapeGrabHandleProxy(Edge));
				DrawLine(AB, Color, SDPG_Foreground);
			};

			if (EdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
			{
				auto GetColor = [&](int32 ComponentIndex)
				{
					if (EdMode->IsGridBased() && (ComponentIndex % GridSize == 0))
					{
						return EdgeColor;
					}
					return ComponentBorderColor;
				};

				const TArray<uint16>& ImportHeights = EdMode->UISettings->GetImportLandscapeData();
				if (ImportHeights.Num() != 0)
				{
					const int32 NumLinesForeground = ComponentCountX * ComponentCountY * 2 + ComponentCountX + ComponentCountY + 8;
					PDI->AddReserveLines(SDPG_Foreground, NumLinesForeground);

					const int32 SizeX = ComponentCountX * QuadsPerComponent + 1;
					const int32 SizeY = ComponentCountY * QuadsPerComponent + 1;
					const int32 ImportSizeX = EdMode->UISettings->ImportLandscape_Width;
					const int32 ImportSizeY = EdMode->UISettings->ImportLandscape_Height;
					const int32 OffsetX = (SizeX - ImportSizeX) / 2;
					const int32 OffsetY = (SizeY - ImportSizeY) / 2;

					// Get coordinates for a line in X direction
					auto LineX = [QuadsPerComponent, ImportSizeX, ImportSizeY, &ImportHeights, OffsetX, OffsetY](int32 X, int32 Y)
					{
						X *= QuadsPerComponent;
						const int32 Y0 = Y * QuadsPerComponent;
						const int32 Y1 = (Y + 1) * QuadsPerComponent;
						const int32 ImportX = FMath::Clamp<int32>(X - OffsetX, 0, ImportSizeX - 1);
						const int32 ImportY0 = FMath::Clamp<int32>(Y0 - OffsetY, 0, ImportSizeY - 1);
						const int32 ImportY1 = FMath::Clamp<int32>(Y1 - OffsetY, 0, ImportSizeY - 1);
						const float Z0 = LandscapeDataAccess::GetLocalHeight(ImportHeights[ImportX + ImportY0 * ImportSizeX]);
						const float Z1 = LandscapeDataAccess::GetLocalHeight(ImportHeights[ImportX + ImportY1 * ImportSizeX]);
						return LineCoords{FVector(X, Y0, Z0), FVector(X, Y1, Z1)};
					};

					// Get coordinates for a line in Y direction
					auto LineY = [QuadsPerComponent, ImportSizeX, ImportSizeY, &ImportHeights, OffsetX, OffsetY](int32 X, int32 Y)
					{
						Y *= QuadsPerComponent;
						const int32 X0 = X * QuadsPerComponent;
						const int32 X1 = (X + 1) * QuadsPerComponent;
						const int32 ImportY = FMath::Clamp<int32>(Y - OffsetY, 0, ImportSizeY - 1);
						const int32 ImportX0 = FMath::Clamp<int32>(X0 - OffsetX, 0, ImportSizeX - 1);
						const int32 ImportX1 = FMath::Clamp<int32>(X1 - OffsetX, 0, ImportSizeX - 1);
						const float Z0 = LandscapeDataAccess::GetLocalHeight(ImportHeights[ImportX0 + ImportY * ImportSizeX]);
						const float Z1 = LandscapeDataAccess::GetLocalHeight(ImportHeights[ImportX1 + ImportY * ImportSizeX]);
						return LineCoords{FVector(X0, Y, Z0), FVector(X1, Y, Z1)};
					};

					// Draw a border in X direction
					auto DrawBorderX = [&PDI, ComponentCountY, &DrawLine, &DrawLineBorder, &LineX](
						const int32 X, ELandscapeEdge::Type FirstCornerEdge, ELandscapeEdge::Type BorderEdge, ELandscapeEdge::Type LastCornerEdge)
					{
						const LineCoords FirstComponent = LineX(X, 0);
						const LineCoords LastComponent = LineX(X, ComponentCountY - 1);
						const FVector FirstCornerEnd = FirstComponent.Get<0>() + CornerSize * (FirstComponent.Get<1>() - FirstComponent.Get<0>());
						const FVector LastCornerBegin = LastComponent.Get<1>() - CornerSize * (LastComponent.Get<1>() - LastComponent.Get<0>());

						DrawLineBorder(FirstCornerEdge, {FirstComponent.Get<0>(), FirstCornerEnd}, CornerColor);
						PDI->SetHitProxy(new HNewLandscapeGrabHandleProxy(BorderEdge));
						if (ComponentCountY == 1)
						{
							DrawLine({FirstCornerEnd, LastCornerBegin}, EdgeColor, SDPG_Foreground);
						}
						else
						{
							DrawLine({FirstCornerEnd, FirstComponent.Get<1>()}, EdgeColor, SDPG_Foreground);
							for (int32 Y = 1; Y < ComponentCountY - 1; ++Y)
							{
								DrawLine(LineX(X, Y), EdgeColor, SDPG_Foreground);
							}
							DrawLine({LastComponent.Get<0>(), LastCornerBegin}, EdgeColor, SDPG_Foreground);
						}
						DrawLineBorder(LastCornerEdge, {LastCornerBegin, LastComponent.Get<1>()}, CornerColor);
					};

					// Draw a border in Y direction
					auto DrawBorderY = [&PDI, ComponentCountX, &DrawLine, &DrawLineBorder, &LineY](
						const int32 Y, ELandscapeEdge::Type FirstCornerEdge, ELandscapeEdge::Type BorderEdge, ELandscapeEdge::Type LastCornerEdge)
					{
						const LineCoords FirstComponent = LineY(0, Y);
						const LineCoords LastComponent = LineY(ComponentCountX - 1, Y);
						const FVector FirstCornerEnd = FirstComponent.Get<0>() + CornerSize * (FirstComponent.Get<1>() - FirstComponent.Get<0>());
						const FVector LastCornerBegin = LastComponent.Get<1>() - CornerSize * (LastComponent.Get<1>() - LastComponent.Get<0>());

						DrawLineBorder(FirstCornerEdge, {FirstComponent.Get<0>(), FirstCornerEnd}, CornerColor);
						PDI->SetHitProxy(new HNewLandscapeGrabHandleProxy(BorderEdge));
						if (ComponentCountX == 1)
						{
							DrawLine({FirstCornerEnd, LastCornerBegin}, EdgeColor, SDPG_Foreground);
						}
						else
						{
							DrawLine({FirstCornerEnd, FirstComponent.Get<1>()}, EdgeColor, SDPG_Foreground);
							for (int32 X = 1; X < ComponentCountX - 1; ++X)
							{
								DrawLine(LineY(X, Y), EdgeColor, SDPG_Foreground);
							}
							DrawLine({LastComponent.Get<0>(), LastCornerBegin}, EdgeColor, SDPG_Foreground);
						}
						DrawLineBorder(LastCornerEdge, {LastCornerBegin, LastComponent.Get<1>()}, CornerColor);
					};
					
					// Left border
					DrawBorderX(0, ELandscapeEdge::X_Negative_Y_Negative, ELandscapeEdge::X_Negative, ELandscapeEdge::X_Negative_Y_Positive);

					// Right border
					DrawBorderX(ComponentCountX, ELandscapeEdge::X_Positive_Y_Negative, ELandscapeEdge::X_Positive, ELandscapeEdge::X_Positive_Y_Positive);

					// Bottom border
					DrawBorderY(0, ELandscapeEdge::X_Negative_Y_Negative, ELandscapeEdge::Y_Negative, ELandscapeEdge::X_Positive_Y_Negative);

					// Top border
					DrawBorderY(ComponentCountY, ELandscapeEdge::X_Negative_Y_Positive, ELandscapeEdge::Y_Positive, ELandscapeEdge::X_Positive_Y_Positive);

					// Reset mouse cursor after all border are drawn
					PDI->SetHitProxy(nullptr);

					// Left to right
					for (int32 X = 1; X < ComponentCountX; ++X)
					{
						const FLinearColor Color = GetColor(X);
						for (int32 Y = 0; Y < ComponentCountY; ++Y)
						{
							DrawLine(LineX(X, Y), Color, SDPG_Foreground);
						}
					}

					// Bottom to top
					for (int32 Y = 1; Y < ComponentCountY; ++Y)
					{
						const FLinearColor Color = GetColor(Y);
						for (int32 X = 0; X < ComponentCountX; ++X)
						{
							DrawLine(LineY(X, Y), Color, SDPG_Foreground);
						}
					}
				}
			}
			else // EdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::NewLandscape
			{
				auto GetColor = [&](int32 QuadIndex)
				{
					if (EdMode->IsGridBased() && (QuadIndex % (GridSize * QuadsPerComponent) == 0))
					{
						return EdgeColor;
					}
					if (QuadIndex % QuadsPerComponent == 0)
					{
						return ComponentBorderColor;
					}
					if (QuadIndex % EdMode->UISettings->NewLandscape_QuadsPerSection == 0)
					{
						return SectionBorderColor;
					}
					return InnerColor;
				};
								
				if (ViewportType == LVT_Perspective || ViewportType == LVT_OrthoXY || ViewportType == LVT_OrthoNegativeXY)
				{
					const int32 NumLines = ComponentCountX * QuadsPerComponent + 1 + ComponentCountY * QuadsPerComponent + 1;
					const int32 NumLinesForeground = ComponentCountX + 1 + ComponentCountY + 1;
					const int32 NumLinesWorld = NumLines - NumLinesForeground;
					constexpr int32 NumLinesForegroundCorners = 8;

					PDI->AddReserveLines(SDPG_Foreground, NumLinesForeground + NumLinesForegroundCorners);
					PDI->AddReserveLines(SDPG_World, NumLinesWorld);

					// Draw a border in X direction
					auto DrawBorderX = [ComponentSize, ComponentCountY, &DrawLineBorder](
						const int32 X, ELandscapeEdge::Type FirstCornerEdge, ELandscapeEdge::Type BorderEdge, ELandscapeEdge::Type LastCornerEdge)
					{
						DrawLineBorder(FirstCornerEdge, {FVector(X, 0, 0), FVector(X, CornerSize * ComponentSize, 0)}, CornerColor);
						DrawLineBorder(BorderEdge, {FVector(X, CornerSize * ComponentSize, 0), FVector(X, (ComponentCountY - CornerSize) * ComponentSize, 0)}, EdgeColor);
						DrawLineBorder(LastCornerEdge, {FVector(X, (ComponentCountY - CornerSize) * ComponentSize, 0), FVector(X, ComponentCountY * ComponentSize, 0)}, CornerColor);
					};

					// Draw a border in Y direction
					auto DrawBorderY = [ComponentSize, ComponentCountX, &DrawLineBorder](
						const int32 Y, ELandscapeEdge::Type FirstCornerEdge, ELandscapeEdge::Type BorderEdge, ELandscapeEdge::Type LastCornerEdge)
					{
						DrawLineBorder(FirstCornerEdge, {FVector(0, Y, 0), FVector(CornerSize * ComponentSize, Y, 0)}, CornerColor);
						DrawLineBorder(BorderEdge, {FVector(CornerSize * ComponentSize, Y, 0), FVector((ComponentCountX - CornerSize) * ComponentSize, Y, 0)}, EdgeColor);
						DrawLineBorder(LastCornerEdge, {FVector((ComponentCountX - CornerSize) * ComponentSize, Y, 0), FVector(ComponentCountX * ComponentSize, Y, 0)}, CornerColor);
					};

					// Left border
					DrawBorderX(0, ELandscapeEdge::X_Negative_Y_Negative, ELandscapeEdge::X_Negative, ELandscapeEdge::X_Negative_Y_Positive);

					// Right border
					DrawBorderX(ComponentCountX * QuadsPerComponent, ELandscapeEdge::X_Positive_Y_Negative, ELandscapeEdge::X_Positive, ELandscapeEdge::X_Positive_Y_Positive);

					// Bottom border
					DrawBorderY(0, ELandscapeEdge::X_Negative_Y_Negative, ELandscapeEdge::Y_Negative, ELandscapeEdge::X_Positive_Y_Negative);

					// Top border
					DrawBorderY(ComponentCountY * QuadsPerComponent, ELandscapeEdge::X_Negative_Y_Positive, ELandscapeEdge::Y_Positive, ELandscapeEdge::X_Positive_Y_Positive);

					// Reset mouse cursor after all border are drawn
					PDI->SetHitProxy(nullptr);

					// Left to right
					for (int32 X = 1; X < ComponentCountX * QuadsPerComponent; ++X)
					{
						const FLinearColor CurrentColor = GetColor(X);
						const uint8 DepthPriority = static_cast<uint8>(CurrentColor == InnerColor ? SDPG_World : SDPG_Foreground);
						DrawLine({FVector(X, 0, 0), FVector(X, ComponentCountY * ComponentSize, 0)}, CurrentColor, DepthPriority);
					}

					// Bottom to top
					for (int32 Y = 1; Y < ComponentCountY * QuadsPerComponent; ++Y)
					{
						const FLinearColor CurrentColor = GetColor(Y);
						const uint8 DepthPriority = static_cast<uint8>(CurrentColor == InnerColor ? SDPG_World : SDPG_Foreground);
						DrawLine({FVector(0, Y, 0), FVector(ComponentCountX * ComponentSize, Y, 0)}, CurrentColor, DepthPriority);
					}
				}
				else
				{
					// Don't allow dragging to resize in side-view, and there is no point drawing the inner lines as only the outer are visible.

					if (ViewportType == LVT_OrthoXZ || ViewportType == LVT_OrthoNegativeXZ)
					{
						DrawLine({FVector(0, 0, 0), FVector(ComponentCountX * ComponentSize, 0, 0)}, EdgeColor, SDPG_World);
						DrawLine({FVector(0, ComponentCountY * QuadsPerComponent, 0), FVector(ComponentCountX * ComponentSize, ComponentCountY * QuadsPerComponent, 0)}, EdgeColor, SDPG_World);
					}

					if (ViewportType == LVT_OrthoYZ || ViewportType == LVT_OrthoNegativeYZ)
					{
						DrawLine({FVector(0, 0, 0), FVector(0, ComponentCountY * ComponentSize, 0)}, EdgeColor, SDPG_World);
						DrawLine({FVector(ComponentCountX * QuadsPerComponent, 0, 0), FVector(ComponentCountX * QuadsPerComponent, ComponentCountY * ComponentSize, 0)}, EdgeColor, SDPG_World);
					}
				}
			}
		}
	}

	virtual int32 GetToolActionResolutionDelta() const override
	{
		if (EdMode != nullptr)
		{
			int32 NewLandscapeResolutionX = EdMode->GetNewLandscapeResolutionX();
			int32 NewLandscapeResolutionY = EdMode->GetNewLandscapeResolutionY();

			NewLandscapeResolutionX = (NewLandscapeResolutionX > 0) ? NewLandscapeResolutionX : 1;
			NewLandscapeResolutionY = (NewLandscapeResolutionY > 0) ? NewLandscapeResolutionY : 1;

			return NewLandscapeResolutionX * NewLandscapeResolutionY;
		}

		return 0;
	}
};


//
// FLandscapeToolResizeLandscape
//
class FLandscapeToolResizeLandscape : public FLandscapeTool
{
public:
	FEdModeLandscape* EdMode;

	FLandscapeToolResizeLandscape(FEdModeLandscape* InEdMode)
		: FLandscapeTool()
		, EdMode(InEdMode)
	{
	}

	virtual bool AffectsEditLayers() const { return false; }
	virtual const TCHAR* GetToolName() const override { return TEXT("ResizeLandscape"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("LandscapeMode_ResizeLandscape", "Change Landscape Component Size"); };
	virtual FText GetDisplayMessage() const override { return LOCTEXT("LandscapeMode_ResizeLandscape_Message", "Change Landscape Component Size"); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual void EnterTool() override
	{
		if (ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			const int32 ComponentSizeQuads = LandscapeInfo->ComponentSizeQuads;
			int32 MinX, MinY, MaxX, MaxY;
			if (EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
			{
				EdMode->UISettings->ResizeLandscape_Original_ComponentCount.X = (MaxX - MinX) / ComponentSizeQuads;
				EdMode->UISettings->ResizeLandscape_Original_ComponentCount.Y = (MaxY - MinY) / ComponentSizeQuads;
				EdMode->UISettings->ResizeLandscape_ComponentCount = EdMode->UISettings->ResizeLandscape_Original_ComponentCount;
			}
			else
			{
				EdMode->UISettings->ResizeLandscape_Original_ComponentCount = FIntPoint::ZeroValue;
				EdMode->UISettings->ResizeLandscape_ComponentCount = FIntPoint::ZeroValue;
			}
			EdMode->UISettings->ResizeLandscape_Original_QuadsPerSection = EdMode->CurrentToolTarget.LandscapeInfo->SubsectionSizeQuads;
			EdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent = EdMode->CurrentToolTarget.LandscapeInfo->ComponentNumSubsections;
			EdMode->UISettings->ResizeLandscape_QuadsPerSection = EdMode->UISettings->ResizeLandscape_Original_QuadsPerSection;
			EdMode->UISettings->ResizeLandscape_SectionsPerComponent = EdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent;
		}
	}

	virtual void ExitTool() override
	{
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		// does nothing
		return false;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		// does nothing
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		// does nothing
		return false;
	}
};

class FLandscapeToolImportExport : public FLandscapeTool
{
public:
	FEdModeLandscape* EdMode;
	FVector MinLocation;
	FIntRect LandscapeExtent;
	
	FLandscapeToolImportExport(FEdModeLandscape* InEdMode)
		: FLandscapeTool()
		, EdMode(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("ImportExport"); }
	virtual FText GetDisplayName() const override { return LOCTEXT("LandscapeMode_ImportExport", "Import Export"); };
	virtual FText GetDisplayMessage() const override { return LOCTEXT("LandscapeMode_ImportExport_Message", "Import/Export Landscape Data"); };

	virtual bool UsesTransformWidget() const { return true; }
	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::SelectComponent | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual FVector GetWidgetLocation() const override
	{
		
		if (EdMode->UISettings->ImportType != ELandscapeImportTransformType::Resample)
		{
			if (ALandscapeGizmoActiveActor* GizmoActor = EdMode->UISettings->CurrentGizmoActor.Get())
			{
				return EdMode->UISettings->CurrentGizmoActor->GetActorLocation();
			}	
		}
		return MinLocation;
	}

	virtual FMatrix GetWidgetRotation() const override
	{ 
		return FMatrix::Identity; 
	}

	virtual void EnterTool() override
	{
		if (EdMode)
		{
			EdMode->UISettings->RefreshImportLayersList(/* bRefreshFromTarget */true);
			EdMode->UISettings->RefreshImports();

			if (ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get())
			{
				MinLocation = FVector(0, 0, 0);
				if (LandscapeInfo->GetLandscapeExtent(LandscapeExtent))
				{
					MinLocation = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().TransformPosition(FVector(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, 0.0f));
				}
				else
				{
					LandscapeExtent = FIntRect(0, 0, 0, 0);
				}

				FVector LocalPosition(EdMode->UISettings->ImportLandscape_GizmoLocalPosition.X, EdMode->UISettings->ImportLandscape_GizmoLocalPosition.Y, 0.0f);
				FVector GizmoPosition = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().TransformPosition(LocalPosition);
				
				if (ALandscapeGizmoActiveActor* GizmoActor = EdMode->UISettings->CurrentGizmoActor.Get())
				{
					GizmoActor->SetActorLocation(GizmoPosition);
				}
			}
		}
	}

	virtual void ExitTool() override
	{
		
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		// does nothing
		return false;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		// does nothing
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		// does nothing
		return false;
	}
		
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
	{
		if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
		{
			// Resample Gizmo can't move
			if (EdMode && EdMode->UISettings->ImportType != ELandscapeImportTransformType::Resample)
			{
				if (ALandscapeGizmoActiveActor* GizmoActor = EdMode->UISettings->CurrentGizmoActor.Get())
				{
					GEditor->ApplyDeltaToActor(
						GizmoActor,
						true,
						&InDrag,
						nullptr,
						nullptr,
						InViewportClient->IsAltPressed(),
						InViewportClient->IsShiftPressed(),
						InViewportClient->IsCtrlPressed());

					return true;
				}
			}
		}
	
		return false;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (!LandscapeInfo)
		{
			return;
		}
			
		static const float        CornerSize = 0.33f;
		static const FLinearColor CornerColor(1.0f, 1.0f, 0.5f);
		static const FLinearColor EdgeColor(1.0f, 1.0f, 0.0f);
		static const FLinearColor ComponentBorderColor(0.0f, 0.85f, 0.0f);
		static const FLinearColor SectionBorderColor(0.0f, 0.4f, 0.0f);
		static const FLinearColor ComponentOutsideColor(0.0f, 0.0f, 0.85f);
		static const FLinearColor InnerColor(0.0f, 0.25f, 0.0f);

		const ELevelViewportType ViewportType = ((FEditorViewportClient*)Viewport->GetClient())->ViewportType;

		const int32 ComponentSizeInt = EdMode->UISettings->NewLandscape_SectionsPerComponent * EdMode->UISettings->NewLandscape_QuadsPerSection;
		const float ComponentSize = static_cast<float>(ComponentSizeInt);
		const FTransform GizmoTransform = FTransform(FRotator(0,0,0), GetWidgetLocation(), LandscapeInfo->DrawScale);
		const int32 Height = EdMode->UISettings->ImportType == ELandscapeImportTransformType::None ? EdMode->UISettings->ImportLandscape_Height : (LandscapeExtent.Height()+1);
		const int32 Width = EdMode->UISettings->ImportType == ELandscapeImportTransformType::None ? EdMode->UISettings->ImportLandscape_Width: (LandscapeExtent.Width()+1);
		const int32 ImportHeight = EdMode->UISettings->ImportType != ELandscapeImportTransformType::Resample ? EdMode->UISettings->ImportLandscape_Height : (LandscapeExtent.Height() + 1);
		const int32 ImportWidth = EdMode->UISettings->ImportType != ELandscapeImportTransformType::Resample ? EdMode->UISettings->ImportLandscape_Width : (LandscapeExtent.Width() + 1);

		const FTransform LandscapeTransfo = FTransform(FRotator(0, 0, 0), MinLocation, LandscapeInfo->DrawScale);

		if (EdMode->ImportExportMode == EImportExportMode::Import)
		{
			if (ViewportType == LVT_Perspective || ViewportType == LVT_OrthoXY || ViewportType == LVT_OrthoNegativeXY)
			{
				for (int32 x = 0; x < ImportWidth; x++)
				{
					if (x == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, 0, 0)), GizmoTransform.TransformPosition(FVector(x, CornerSize * ComponentSize, 0)), CornerColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, CornerSize * ComponentSize, 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight - (CornerSize * ComponentSize), 0)), EdgeColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, ImportHeight - (CornerSize * ComponentSize), 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight, 0)), CornerColor, SDPG_Foreground);
					}
					else if (x == (ImportWidth - 1))
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, 0, 0)), GizmoTransform.TransformPosition(FVector(x, CornerSize * ComponentSize, 0)), CornerColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, CornerSize * ComponentSize, 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight - (CornerSize * ComponentSize), 0)), EdgeColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, ImportHeight - (CornerSize * ComponentSize), 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight, 0)), CornerColor, SDPG_Foreground);
					}
					else if ((x % ComponentSizeInt) == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, 0, 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight, 0)), ComponentBorderColor, SDPG_Foreground);
					}
					else if ((x % EdMode->UISettings->NewLandscape_QuadsPerSection) == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(x, 0, 0)), GizmoTransform.TransformPosition(FVector(x, ImportHeight, 0)), SectionBorderColor, SDPG_Foreground);
					}
				}

				if (ImportWidth != Width)
				{
					for (int32 x = 0; x < Width; x++)
					{
						if ((x % ComponentSizeInt) == 0)
						{
							PDI->DrawLine(LandscapeTransfo.TransformPosition(FVector(x, 0, 0)), LandscapeTransfo.TransformPosition(FVector(x, Height, 0)), ComponentOutsideColor, SDPG_Foreground);
						}
					}
				}
			}
			else
			{
				PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, 0, 0)), GizmoTransform.TransformPosition(FVector(0, ImportHeight, 0)), EdgeColor, SDPG_Foreground);
				PDI->DrawLine(GizmoTransform.TransformPosition(FVector(ImportWidth, 0, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, ImportHeight, 0)), EdgeColor, SDPG_Foreground);
			}

			if (ViewportType == LVT_Perspective || ViewportType == LVT_OrthoXY || ViewportType == LVT_OrthoNegativeXY)
			{
				for (int32 y = 0; y < ImportHeight; y++)
				{
					if (y == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, y, 0)), GizmoTransform.TransformPosition(FVector(CornerSize * ComponentSize, y, 0)), CornerColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(CornerSize * ComponentSize, y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth - (CornerSize * ComponentSize), y, 0)), EdgeColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(ImportWidth - (CornerSize * ComponentSize), y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, y, 0)), CornerColor, SDPG_Foreground);
					}
					else if (y == (ImportHeight - 1))
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, y, 0)), GizmoTransform.TransformPosition(FVector(CornerSize * ComponentSize, y, 0)), CornerColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(CornerSize * ComponentSize, y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth - (CornerSize * ComponentSize), y, 0)), EdgeColor, SDPG_Foreground);
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(ImportWidth - (CornerSize * ComponentSize), y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, y, 0)), CornerColor, SDPG_Foreground);
					}
					else if ((y % ComponentSizeInt) == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, y, 0)), ComponentBorderColor, SDPG_Foreground);
					}
					else if ((y % EdMode->UISettings->NewLandscape_QuadsPerSection) == 0)
					{
						PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, y, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, y, 0)), SectionBorderColor, SDPG_Foreground);
					}
				}

				if (ImportHeight != Height)
				{
					for (int32 y = 0; y < Height; y++)
					{
						if ((y % ComponentSizeInt) == 0)
						{
							PDI->DrawLine(LandscapeTransfo.TransformPosition(FVector(0, y, 0)), LandscapeTransfo.TransformPosition(FVector(Width, y, 0)), ComponentOutsideColor, SDPG_Foreground);
						}
					}
				}
			}
			else
			{
				// and there's no point drawing the inner lines as only the outer is visible
				PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, 0, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, 0, 0)), EdgeColor, SDPG_Foreground);
				PDI->DrawLine(GizmoTransform.TransformPosition(FVector(0, ImportHeight, 0)), GizmoTransform.TransformPosition(FVector(ImportWidth, ImportHeight, 0)), EdgeColor, SDPG_Foreground);
			}
		}
	}
	
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const override
	{
		return EAxisList::XY;
	}
};

//////////////////////////////////////////////////////////////////////////

void FEdModeLandscape::InitializeTool_NewLandscape()
{
	auto Tool_NewLandscape = MakeUnique<FLandscapeToolNewLandscape>(this);
	Tool_NewLandscape->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_NewLandscape));
}

void FEdModeLandscape::InitializeTool_ResizeLandscape()
{
	auto Tool_ResizeLandscape = MakeUnique<FLandscapeToolResizeLandscape>(this);
	Tool_ResizeLandscape->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_ResizeLandscape));
}

void FEdModeLandscape::InitializeTool_ImportExport()
{
	auto Tool_ImportExportLandscape = MakeUnique<FLandscapeToolImportExport>(this);
	Tool_ImportExportLandscape->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_ImportExportLandscape));
}

void FEdModeLandscape::InitializeTool_Select()
{
	auto Tool_Select = MakeUnique<FLandscapeToolSelect>(this);
	Tool_Select->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_Select));
}

void FEdModeLandscape::InitializeTool_AddComponent()
{
	auto Tool_AddComponent = MakeUnique<FLandscapeToolAddComponent>(this);
	Tool_AddComponent->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_AddComponent));
}

void FEdModeLandscape::InitializeTool_DeleteComponent()
{
	auto Tool_DeleteComponent = MakeUnique<FLandscapeToolDeleteComponent>(this);
	Tool_DeleteComponent->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_DeleteComponent));
}

void FEdModeLandscape::InitializeTool_MoveToLevel()
{
	auto Tool_MoveToLevel = MakeUnique<FLandscapeToolMoveToLevel>(this);
	Tool_MoveToLevel->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_MoveToLevel));
}

void FEdModeLandscape::InitializeTool_Mask()
{
	auto Tool_Mask = MakeUnique<FLandscapeToolMask>(this);
	Tool_Mask->ValidBrushes.Add("BrushSet_Circle");
	Tool_Mask->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Mask->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Mask));
}

void FEdModeLandscape::InitializeTool_CopyPaste()
{
	auto Tool_CopyPaste_Heightmap = MakeUnique<FLandscapeToolCopyPaste<FHeightmapToolTarget>>(this);
	Tool_CopyPaste_Heightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_CopyPaste_Heightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_CopyPaste_Heightmap->ValidBrushes.Add("BrushSet_Pattern");
	Tool_CopyPaste_Heightmap->ValidBrushes.Add("BrushSet_Gizmo");
	CopyPasteTool = Tool_CopyPaste_Heightmap.Get();
	LandscapeTools.Add(MoveTemp(Tool_CopyPaste_Heightmap));

	//auto Tool_CopyPaste_Weightmap = MakeUnique<FLandscapeToolCopyPaste<FWeightmapToolTarget>>(this);
	//Tool_CopyPaste_Weightmap->ValidBrushes.Add("BrushSet_Circle");
	//Tool_CopyPaste_Weightmap->ValidBrushes.Add("BrushSet_Alpha");
	//Tool_CopyPaste_Weightmap->ValidBrushes.Add("BrushSet_Pattern");
	//Tool_CopyPaste_Weightmap->ValidBrushes.Add("BrushSet_Gizmo");
	//LandscapeTools.Add(MoveTemp(Tool_CopyPaste_Weightmap));
}

void FEdModeLandscape::InitializeTool_Visibility()
{
	auto Tool_Visibility = MakeUnique<FLandscapeToolVisibility>(this);
	Tool_Visibility->ValidBrushes.Add("BrushSet_Circle");
	Tool_Visibility->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Visibility->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Visibility));
}

#undef LOCTEXT_NAMESPACE
