// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdModeTools.h"
#include "Landscape.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/MapErrors.h"

#define LOCTEXT_NAMESPACE "LandscapeTools"
//
// FLandscapeToolErosionBase
//
class FLandscapeToolStrokeErosionBase : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

public:
	FLandscapeToolStrokeErosionBase(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, HeightCache(InTarget.LandscapeInfo.Get())
		, EditLayerHeightDataCache(InTarget.LandscapeInfo.Get(), this->HeightCache)
		, WeightCache(InTarget.LandscapeInfo.Get())
		, bWeightApplied(InTarget.TargetType != ELandscapeToolTargetType::Heightmap)
	{
	}

protected:
	FLandscapeHeightCache HeightCache;
	FLandscapeEditLayerStackDataCache<FHeightmapToolTarget> EditLayerHeightDataCache;
	FLandscapeFullWeightCache WeightCache;
	bool bWeightApplied;
};

template<class TStrokeClass>
class FLandscapeToolErosionBase : public FLandscapeToolBase<TStrokeClass>
{
	using Super = FLandscapeToolBase<TStrokeClass>;

public:
	FLandscapeToolErosionBase(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ELandscapeToolTargetType::Heightmap;
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		// Only heightmaps are supported ATM
		return (this->EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap) ? ELandscapeTargetLayerActionType::Normal : ELandscapeTargetLayerActionType::None;
	}
};

//
// FLandscapeToolErosion
//

class FLandscapeToolStrokeErosion : public FLandscapeToolStrokeErosionBase
{
	using Super = FLandscapeToolStrokeErosionBase;

public:
	FLandscapeToolStrokeErosion(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	virtual void SetEditLayer(const FGuid& EditLayerGUID) override
	{
		EditLayerHeightDataCache.SetCacheEditingLayer(EditLayerGUID);
		WeightCache.DataAccess.SetEditLayer(EditLayerGUID);
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeErosion_Apply);

		if (!this->LandscapeInfo)
		{
			return;
		}

		ALandscape* Landscape = this->LandscapeInfo->LandscapeActor.Get();
		bool bCombinedLayerOperation = UISettings->bCombinedLayersOperation && Landscape;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		X1 -= 1;
		Y1 -= 1;
		X2 += 1;
		Y2 += 1;

		const int32 NeighborNum = 4;
		const int32 Iteration = UISettings->ErodeIterationNum;
		const int32 Thickness = UISettings->ErodeSurfaceThickness;
		const int32 LayerNum = this->LandscapeInfo->Layers.Num();

		TArray<uint16> HeightData;
		EditLayerHeightDataCache.Initialize(bCombinedLayerOperation);
		EditLayerHeightDataCache.Read(X1, Y1, X2, Y2, HeightData);

		TArray<uint8> WeightDatas; // Weight*Layers...
		if (UISettings->bErosionUseLayerHardness || bWeightApplied)
		{
			WeightCache.CacheData(X1, Y1, X2, Y2);
			WeightCache.GetCachedData(X1, Y1, X2, Y2, WeightDatas, LayerNum);
		}

		// Apply the brush	
		uint16 Thresh = static_cast<uint16>(UISettings->ErodeThresh);
		int32 WeightMoveThresh = FMath::Min<int32>(FMath::Max<int32>(Thickness >> 2, Thresh), Thickness >> 1);

		TArray<float> CenterWeights;
		TArray<float> NeighborWeight;
		if (bWeightApplied)
		{
			CenterWeights.Empty(LayerNum);
			CenterWeights.AddUninitialized(LayerNum);
			NeighborWeight.Empty(NeighborNum*LayerNum);
			NeighborWeight.AddUninitialized(NeighborNum*LayerNum);
		}

		bool bHasChanged = false;
		for (int32 i = 0; i < Iteration; i++)
		{
			bHasChanged = false;
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						int32 Center = (X - X1) + (Y - Y1)*(1 + X2 - X1);
						int32 Neighbor[NeighborNum] = {
							(X - 1 - X1) + (Y - Y1)*(1 + X2 - X1),   // -X
							(X + 1 - X1) + (Y - Y1)*(1 + X2 - X1),   // +X
							(X - X1) + (Y - 1 - Y1)*(1 + X2 - X1),   // -Y
							(X - X1) + (Y + 1 - Y1)*(1 + X2 - X1) }; // +Y
						uint32 SlopeTotal = 0;
						uint16 SlopeMax = Thresh;

						for (int32 Idx = 0; Idx < NeighborNum; Idx++)
						{
							if (HeightData[Center] > HeightData[Neighbor[Idx]])
							{
								uint16 Slope = HeightData[Center] - HeightData[Neighbor[Idx]];
								if (Slope * BrushValue > Thresh)
								{
									SlopeTotal += Slope;
									if (SlopeMax < Slope)
									{
										SlopeMax = Slope;
									}
								}
							}
						}

						if (SlopeTotal > 0)
						{
							float Softness = 1.0f;
							if (UISettings->bErosionUseLayerHardness)
							{
								for (int32 Idx = 0; Idx < LayerNum; Idx++)
								{
									ULandscapeLayerInfoObject* LayerInfo = this->LandscapeInfo->Layers[Idx].LayerInfoObj;
									if (LayerInfo)
									{
										uint8 Weight = WeightDatas[Center*LayerNum + Idx];
										Softness -= (float)(Weight) / 255.0f * LayerInfo->GetHardness();
									}
								}
							}
							if (Softness > 0.0f)
							{
								//Softness = FMath::Clamp<float>(Softness, 0.0f, 1.0f);
								float TotalHeightDiff = 0;
								int32 WeightTransfer = FMath::Min<int32>(WeightMoveThresh, SlopeMax - Thresh);
								for (int32 Idx = 0; Idx < NeighborNum; Idx++)
								{
									float TotalWeight = 0.0f;
									if (HeightData[Center] > HeightData[Neighbor[Idx]])
									{
										uint16 Slope = HeightData[Center] - HeightData[Neighbor[Idx]];
										if (Slope > Thresh)
										{
											float WeightDiff = Softness * UISettings->GetCurrentToolStrength() * Pressure * ((float)Slope / SlopeTotal) * BrushValue;
											//uint16 HeightDiff = (uint16)((SlopeMax - Thresh) * WeightDiff);
											float HeightDiff = (SlopeMax - Thresh) * WeightDiff;
											HeightData[Neighbor[Idx]] += static_cast<uint16>(HeightDiff);
											TotalHeightDiff += HeightDiff;

											if (bWeightApplied)
											{
												for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
												{
													float CenterWeight = (float)(WeightDatas[Center*LayerNum + LayerIdx]) / 255.0f;
													float Weight = (float)(WeightDatas[Neighbor[Idx] * LayerNum + LayerIdx]) / 255.0f;
													NeighborWeight[Idx*LayerNum + LayerIdx] = Weight*(float)Thickness + CenterWeight*WeightDiff*WeightTransfer; // transferred + original...
													TotalWeight += NeighborWeight[Idx*LayerNum + LayerIdx];
												}
												// Need to normalize weight...
												for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
												{
													WeightDatas[Neighbor[Idx] * LayerNum + LayerIdx] = (TotalWeight > 0.0f) ? (uint8)(255.0f * NeighborWeight[Idx*LayerNum + LayerIdx] / TotalWeight) : 0;
												}
											}
										}
									}
								}

								HeightData[Center] -= static_cast<uint16>(TotalHeightDiff);

								if (bWeightApplied)
								{
									float TotalWeight = 0.0f;
									float WeightDiff = Softness * UISettings->GetCurrentToolStrength() * Pressure * BrushValue;

									for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
									{
										float Weight = (float)(WeightDatas[Center*LayerNum + LayerIdx]) / 255.0f;
										CenterWeights[LayerIdx] = Weight*Thickness - Weight*WeightDiff*WeightTransfer;
										TotalWeight += CenterWeights[LayerIdx];
									}
									// Need to normalize weight...
									for (int32 LayerIdx = 0; LayerIdx < LayerNum; LayerIdx++)
									{
										WeightDatas[Center*LayerNum + LayerIdx] = (TotalWeight > 0.0f) ? (uint8)(255.0f * CenterWeights[LayerIdx] / TotalWeight) : 0;
									}
								}

								bHasChanged = true;
							} // if Softness > 0.0f
						} // if SlopeTotal > 0
					}
				}
			}
			if (!bHasChanged)
			{
				break;
			}
		}

		float BrushSizeAdjust = 1.0f;
		if (UISettings->GetCurrentToolBrushRadius() < UISettings->MaximumValueRadius)
		{
			BrushSizeAdjust = UISettings->GetCurrentToolBrushRadius() / UISettings->MaximumValueRadius;
		}

		// Make some noise...
		for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
		{
			const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));

			for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
			{
				const float BrushValue = BrushScanline[X];

				if (BrushValue > 0.0f)
				{
					FNoiseParameter NoiseParam(0, UISettings->ErosionNoiseScale, BrushValue * Thresh * UISettings->GetCurrentToolStrength() * BrushSizeAdjust);
					float PaintAmount = NoiseModeConversion((ELandscapeToolNoiseMode)UISettings->ErosionNoiseMode, NoiseParam.NoiseAmount, NoiseParam.Sample(X, Y));
					HeightData[(X - X1) + (Y - Y1)*(1 + X2 - X1)] = FLandscapeHeightCache::ClampValue(static_cast<int32>(HeightData[(X - X1) + (Y - Y1)*(1 + X2 - X1)] + PaintAmount));
				}
			}
		}

		EditLayerHeightDataCache.Write(X1, Y1, X2, Y2, HeightData);
		if (bWeightApplied)
		{
			WeightCache.SetCachedData(X1, Y1, X2, Y2, WeightDatas, LayerNum, ELandscapeLayerPaintingRestriction::None);
		}
		WeightCache.Flush();

		// Dirty any runtime virtual textures that our landscape components write to.
		LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

class FLandscapeToolErosion : public FLandscapeToolErosionBase<FLandscapeToolStrokeErosion>
{
	using Super = FLandscapeToolErosionBase<FLandscapeToolStrokeErosion>;

public:
	FLandscapeToolErosion(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Erosion"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Erosion", "Erosion"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Erosion_Message", "The Erosion tool uses a thermal erosion simulation to adjust the height of the landscape. This simulates the transfer of soil from higher elevations to lower elevations. The larger the difference in elevation, the more erosion will occur. This tool also applies a noise effect on top of the erosion, if desired, to provide a more natural random appearance. "); };

};

//
// FLandscapeToolHydraErosion
//

class FLandscapeToolStrokeHydraErosion : public FLandscapeToolStrokeErosionBase
{
	using Super = FLandscapeToolStrokeErosionBase;

public:
	FLandscapeToolStrokeHydraErosion(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	virtual void SetEditLayer(const FGuid& EditLayerGUID) override
	{
		EditLayerHeightDataCache.SetCacheEditingLayer(EditLayerGUID);
		WeightCache.DataAccess.SetEditLayer(EditLayerGUID);
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeHydraErosion_Apply);

		if (!this->LandscapeInfo)
		{
			return;
		}

		ALandscape* Landscape = this->LandscapeInfo->LandscapeActor.Get();
		bool bCombinedLayerOperation = UISettings->bCombinedLayersOperation && Landscape;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		X1 -= 1;
		Y1 -= 1;
		X2 += 1;
		Y2 += 1;

		const int32 Iteration = UISettings->HErodeIterationNum;
		const uint16 RainAmount = static_cast<uint16>(UISettings->RainAmount);
		const float DissolvingRatio = 0.07f * UISettings->GetCurrentToolStrength() * Pressure;  //0.01;
		const float EvaporateRatio = 0.5f;
		const float SedimentCapacity = 0.10f * UISettings->SedimentCapacity; //DissolvingRatio; //0.01;

		TArray<uint16> HeightData;
		EditLayerHeightDataCache.Initialize(bCombinedLayerOperation);
		EditLayerHeightDataCache.Read(X1, Y1, X2, Y2, HeightData);

		// Apply the brush
		TArray<uint16> WaterData;
		WaterData.Empty((1 + X2 - X1)*(1 + Y2 - Y1));
		WaterData.AddZeroed((1 + X2 - X1)*(1 + Y2 - Y1));
		TArray<uint16> SedimentData;
		SedimentData.Empty((1 + X2 - X1)*(1 + Y2 - Y1));
		SedimentData.AddZeroed((1 + X2 - X1)*(1 + Y2 - Y1));

		// Only initial raining works better...
		FNoiseParameter NoiseParam(0, UISettings->RainDistScale, RainAmount);
		for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
		{
			const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
			auto* WaterDataScanline = WaterData.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

			for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
			{
				const float BrushValue = BrushScanline[X];

				if (BrushValue >= 1.0f)
				{
					float PaintAmount = NoiseModeConversion((ELandscapeToolNoiseMode)UISettings->RainDistMode, NoiseParam.NoiseAmount, NoiseParam.Sample(X, Y));
					if (PaintAmount > 0) // Raining only for positive region...
						WaterDataScanline[X] += static_cast<uint16>(PaintAmount);
				}
			}
		}

		for (int32 i = 0; i < Iteration; i++)
		{
			bool bWaterExist = false;
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						int32 Center = (X - X1) + (Y - Y1)*(1 + X2 - X1);

						const int32 NeighborNum = 8;
						int32 Neighbor[NeighborNum] = {
							(X - 1 - X1) + (Y - Y1)*(1 + X2 - X1),   // -X
							(X + 1 - X1) + (Y - Y1)*(1 + X2 - X1),   // +X
							(X - X1) + (Y - 1 - Y1)*(1 + X2 - X1),   // -Y
							(X - X1) + (Y + 1 - Y1)*(1 + X2 - X1),   // +Y
							(X - 1 - X1) + (Y - 1 - Y1)*(1 + X2 - X1),   // -X -Y
							(X + 1 - X1) + (Y + 1 - Y1)*(1 + X2 - X1),   // +X +Y
							(X + 1 - X1) + (Y - 1 - Y1)*(1 + X2 - X1),   // +X -Y
							(X - 1 - X1) + (Y + 1 - Y1)*(1 + X2 - X1) }; // -X +Y

						// Dissolving...
						float DissolvedAmount = DissolvingRatio * WaterData[Center] * BrushValue;
						if (DissolvedAmount > 0 && HeightData[Center] >= DissolvedAmount)
						{
							HeightData[Center] -= static_cast<uint16>(DissolvedAmount);
							SedimentData[Center] += static_cast<uint16>(DissolvedAmount);
						}

						uint32 TotalHeightDiff = 0;
						uint32 TotalAltitudeDiff = 0;
						uint32 AltitudeDiff[NeighborNum] = { 0 };
						uint32 TotalWaterDiff = 0;
						uint32 TotalSedimentDiff = 0;

						uint32 Altitude = HeightData[Center] + WaterData[Center];
						float AverageAltitude = 0;
						uint32 LowerNeighbor = 0;
						for (int32 Idx = 0; Idx < NeighborNum; Idx++)
						{
							uint32 NeighborAltitude = HeightData[Neighbor[Idx]] + WaterData[Neighbor[Idx]];
							if (Altitude > NeighborAltitude)
							{
								AltitudeDiff[Idx] = Altitude - NeighborAltitude;
								TotalAltitudeDiff += AltitudeDiff[Idx];
								LowerNeighbor++;
								AverageAltitude += NeighborAltitude;
								if (HeightData[Center] > HeightData[Neighbor[Idx]])
								{
									TotalHeightDiff += HeightData[Center] - HeightData[Neighbor[Idx]];
								}
							}
							else
							{
								AltitudeDiff[Idx] = 0;
							}
						}

						// Water Transfer
						if (LowerNeighbor > 0)
						{
							AverageAltitude /= (LowerNeighbor);
							// This is not mathematically correct, but makes good result
							if (TotalHeightDiff)
							{
								AverageAltitude *= static_cast<float>(1.0f - 0.1 * UISettings->GetCurrentToolStrength() * Pressure);
								//AverageAltitude -= 4000.0f * UISettings->ToolStrength;
							}

							uint32 WaterTransfer = static_cast<uint32>(FMath::Min<uint32>(WaterData[Center], Altitude - (uint32)AverageAltitude) * BrushValue);

							for (int32 Idx = 0; Idx < NeighborNum; Idx++)
							{
								if (AltitudeDiff[Idx] > 0)
								{
									uint32 WaterDiff = (uint32)(WaterTransfer * (float)AltitudeDiff[Idx] / TotalAltitudeDiff);
									WaterData[Neighbor[Idx]] += static_cast<uint16>(WaterDiff);
									TotalWaterDiff += WaterDiff;
									uint32 SedimentDiff = (WaterData[Center] > 0) ? (uint32)(SedimentData[Center] * (float)WaterDiff / WaterData[Center]) : 0;
									SedimentData[Neighbor[Idx]] += static_cast<uint16>(SedimentDiff);
									TotalSedimentDiff += SedimentDiff;
								}
							}

							WaterData[Center] -= static_cast<uint16>(TotalWaterDiff);
							SedimentData[Center] -= static_cast<uint16>(TotalSedimentDiff);
						}

						// evaporation
						if (WaterData[Center] > 0)
						{
							bWaterExist = true;
							WaterData[Center] = (uint16)(WaterData[Center] * (1.0f - EvaporateRatio));
							float SedimentCap = SedimentCapacity*WaterData[Center];
							float SedimentDiff = SedimentData[Center] - SedimentCap;
							if (SedimentDiff > 0)
							{
								SedimentData[Center] -= static_cast<uint16>(SedimentDiff);
								HeightData[Center] = FMath::Clamp<uint16>(static_cast<uint16>(HeightData[Center] + SedimentDiff), 0, LandscapeDataAccess::MaxValue);
							}
						}
					}
				}
			}

			if (!bWaterExist)
			{
				break;
			}
		}

		if (UISettings->bHErosionDetailSmooth)
		{
			//LowPassFilter<uint16>(X1, Y1, X2, Y2, BrushInfo, HeightData, UISettings->HErosionDetailScale, UISettings->ToolStrength * Pressure);
			LowPassFilter<uint16>(X1, Y1, X2, Y2, BrushInfo, HeightData, UISettings->HErosionDetailScale, 1.0f);
		}

		EditLayerHeightDataCache.Write(X1, Y1, X2, Y2, HeightData);

		// Dirty any runtime virtual textures that our landscape components write to.
		LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

class FLandscapeToolHydraErosion : public FLandscapeToolErosionBase<FLandscapeToolStrokeHydraErosion>
{
	using Super = FLandscapeToolErosionBase<FLandscapeToolStrokeHydraErosion>;

public:
	FLandscapeToolHydraErosion(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("HydraErosion"); } // formerly HydraulicErosion
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_HydraErosion", "Hydraulic Erosion"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_HydraErosion_Message", "The Hydro Erosion tool simulates erosion from water to adjust the height of the landscape. A noise filter is used to determine where the initial rain is distributed. Then the simulation is calculated to determine water flow from that initial rain as well as dissolving, water transfer, and evaporation. The result of that calculation provides the actual value used to lower the heightmap."); };

};

//
// Toolset initialization
//
void FEdModeLandscape::InitializeTool_Erosion()
{
	auto Tool_Erosion = MakeUnique<FLandscapeToolErosion>(this);
	Tool_Erosion->ValidBrushes.Add("BrushSet_Circle");
	Tool_Erosion->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Erosion->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Erosion));
}

void FEdModeLandscape::InitializeTool_HydraErosion()
{
	auto Tool_HydraErosion = MakeUnique<FLandscapeToolHydraErosion>(this);
	Tool_HydraErosion->ValidBrushes.Add("BrushSet_Circle");
	Tool_HydraErosion->ValidBrushes.Add("BrushSet_Alpha");
	Tool_HydraErosion->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_HydraErosion));
}

#undef LOCTEXT_NAMESPACE