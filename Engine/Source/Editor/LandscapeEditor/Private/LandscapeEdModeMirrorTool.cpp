// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Materials/MaterialInterface.h"
#include "AI/NavigationSystemBase.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UnrealWidgetFwd.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeEdMode.h"
#include "Containers/ArrayView.h"
#include "LandscapeEditorObject.h"
#include "ScopedTransaction.h"
#include "LandscapeEdit.h"
#include "LandscapeDataAccess.h"
#include "LandscapeRender.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "Landscape.h"
#include "Misc/MessageDialog.h"

#define LOCTEXT_NAMESPACE "Landscape"

class FLandscapeToolMirror : public FLandscapeTool
{
protected:
	FEdModeLandscape* EdMode;
	TObjectPtr<UMaterialInstanceDynamic> MirrorPlaneMaterial;

	ECoordSystem SavedCoordSystem;

public:
	FLandscapeToolMirror(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
	{
		UMaterialInterface* BaseMirrorPlaneMaterial = LoadObject<UMaterialInterface>(nullptr, TEXT("/Engine/EditorLandscapeResources/MirrorPlaneMaterial.MirrorPlaneMaterial"));
		MirrorPlaneMaterial = UMaterialInstanceDynamic::Create(BaseMirrorPlaneMaterial, GetTransientPackage());
		MirrorPlaneMaterial->SetScalarParameterValue(FName("LineThickness"), 2.0f);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(MirrorPlaneMaterial);
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Mirror"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeTool_Mirror", "Mirror Landscape"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeTool_Mirror_Message", "Copy one side of a landscape to the other side so that you can easily mirror or rotate the landscape geometry along the X or Y axis."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ELandscapeToolTargetType::Heightmap;
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		return ELandscapeTargetLayerActionType::Normal;
	}

	virtual void EnterTool() override
	{
		if (EdMode->UISettings->MirrorPoint == FVector2D::ZeroVector)
		{
			CenterMirrorPoint();
		}
		GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
		SavedCoordSystem = GLevelEditorModeTools().GetCoordSystem();
		GLevelEditorModeTools().SetCoordSystem(COORD_Local);
	}

	virtual void ExitTool() override
	{
		GLevelEditorModeTools().SetCoordSystem(SavedCoordSystem);
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		return true;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		return false;
	}

	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override
	{
		if (InKey == EKeys::Enter && InEvent == IE_Pressed)
		{
			ApplyMirror();
		}

		return false;
	}

	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override
	{
		const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
		if ((LandscapeProxy != nullptr) && (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None))
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			EdMode->UISettings->MirrorPoint += FVector2D(LandscapeToWorld.InverseTransformVector(InDrag));

			return true;
		}

		return false;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		// The editor can try to render the tool before the UpdateLandscapeEditorData command runs and the landscape editor realizes that the landscape has been hidden/deleted
		const ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo == nullptr)
		{
			return;
		}

		const ALandscapeProxy* const LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		if (LandscapeProxy)
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			int32 MinX, MinY, MaxX, MaxY;
			if (LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
			{
				FVector MirrorPoint3D = FVector((MaxX + MinX) / 2.0f, (MaxY + MinY) / 2.0f, 0);
				FVector MirrorPlaneScale = FVector(0, 1, 100);

				if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::MinusXToPlusX ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::PlusXToMinusX ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusXToPlusX ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusXToMinusX)
				{
					MirrorPoint3D.X = EdMode->UISettings->MirrorPoint.X;
					MirrorPlaneScale.Y = (MaxY - MinY) / 2.0f;
				}
				else
				{
					MirrorPoint3D.Y = EdMode->UISettings->MirrorPoint.Y;
					MirrorPlaneScale.Y = (MaxX - MinX) / 2.0f;
				}

				MirrorPoint3D.Z = GetLocalZAtPoint(LandscapeInfo, FMath::RoundToInt32(MirrorPoint3D.X), FMath::RoundToInt32(MirrorPoint3D.Y));
				MirrorPoint3D = LandscapeToWorld.TransformPosition(MirrorPoint3D);

				FMatrix Matrix;
				if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::MinusYToPlusY ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::PlusYToMinusY ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusYToPlusY ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusYToMinusY)
				{
					Matrix = FScaleRotationTranslationMatrix(MirrorPlaneScale, FRotator(0, 90, 0), FVector::ZeroVector);
				}
				else
				{
					Matrix = FScaleMatrix(MirrorPlaneScale);
				}

				Matrix *= LandscapeToWorld.ToMatrixWithScale();
				Matrix.SetOrigin(MirrorPoint3D);

				// Convert plane from horizontal to vertical
				Matrix = FMatrix(FVector(0, 1, 0), FVector(0, 0, 1), FVector(1, 0, 0), FVector(0, 0, 0)) * Matrix;

				const FBox Box = FBox(FVector(-1, -1, 0), FVector(+1, +1, 0));
				DrawWireBox(PDI, Matrix, Box, FLinearColor::Green, SDPG_World);

				const float LandscapeScaleRatio = static_cast<float>(LandscapeToWorld.GetScale3D().Z / LandscapeToWorld.GetScale3D().X);
				FVector2D UVScale = FVector2D(FMath::RoundToFloat(MirrorPlaneScale.Y / 10), FMath::RoundToFloat(MirrorPlaneScale.Z * LandscapeScaleRatio / 10 / 2) * 2);
				MirrorPlaneMaterial->SetVectorParameterValue(FName("GridSize"), FVector(UVScale, 0));
				DrawPlane10x10(PDI, Matrix, 1, FVector2D(0, 0), FVector2D(1, 1), MirrorPlaneMaterial->GetRenderProxy(), SDPG_World);
			}
		}
	}

	virtual bool OverrideSelection() const override
	{
		return true;
	}

	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override
	{
		// Only filter selection not deselection
		if (bInSelection)
		{
			return false;
		}

		return true;
	}

	virtual bool UsesTransformWidget() const override
	{
		// The editor can try to render the transform widget before the landscape editor ticks and realises that the landscape has been hidden/deleted
		const ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo == nullptr)
		{
			return false;
		}

		const ALandscapeProxy* const LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		if (LandscapeProxy)
		{
			return true;
		}

		return false;
	}

	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode CheckMode) const override
	{
		if (CheckMode == UE::Widget::WM_Translate)
		{
			switch (EdMode->UISettings->MirrorOp)
			{
			case ELandscapeMirrorOperation::MinusXToPlusX:
			case ELandscapeMirrorOperation::PlusXToMinusX:
			case ELandscapeMirrorOperation::RotateMinusXToPlusX:
			case ELandscapeMirrorOperation::RotatePlusXToMinusX:
				return EAxisList::X;
			case ELandscapeMirrorOperation::MinusYToPlusY:
			case ELandscapeMirrorOperation::PlusYToMinusY:
			case ELandscapeMirrorOperation::RotateMinusYToPlusY:
			case ELandscapeMirrorOperation::RotatePlusYToMinusY:
				return EAxisList::Y;
			default:
				check(0);
				return EAxisList::None;
			}
		}

		return EAxisList::None;
	}

	virtual FVector GetWidgetLocation() const override
	{
		const ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo == nullptr)
		{
			return FVector::ZeroVector;
		}

		const ALandscapeProxy* const LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		if (LandscapeProxy)
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			int32 MinX, MinY, MaxX, MaxY;
			if (!LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
			{
				MinX = MinY = 0;
				MaxX = MaxY = 0;
			}

			FVector MirrorPoint3D = FVector((MaxX + MinX) / 2.0f, (MaxY + MinY) / 2.0f, 0);
			if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::MinusXToPlusX ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::PlusXToMinusX ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusXToPlusX ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusXToMinusX)
			{
				MirrorPoint3D.X = EdMode->UISettings->MirrorPoint.X;
			}
			else
			{
				MirrorPoint3D.Y = EdMode->UISettings->MirrorPoint.Y;
			}
			MirrorPoint3D.Z = GetLocalZAtPoint(LandscapeInfo, FMath::RoundToInt32(MirrorPoint3D.X), FMath::RoundToInt32(MirrorPoint3D.Y));
			MirrorPoint3D = LandscapeToWorld.TransformPosition(MirrorPoint3D);
			MirrorPoint3D.Z += 1000.f; // place the widget a little off the ground for better visibility
			return MirrorPoint3D;
		}

		return FVector::ZeroVector;
	}

	virtual FMatrix GetWidgetRotation() const override
	{
		if (const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr)
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			FMatrix Result = FQuatRotationTranslationMatrix(LandscapeToWorld.GetRotation(), FVector::ZeroVector);
			if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::PlusXToMinusX ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::PlusYToMinusY ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusXToMinusX ||
				EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusYToMinusY)
			{
				Result = FRotationMatrix(FRotator(0, 180, 0)) * Result;
			}
			return Result;
		}

		return FMatrix::Identity;
	}

protected:
	float GetLocalZAtPoint(const ULandscapeInfo* LandscapeInfo, int32 x, int32 y) const
	{
		// try to find Z location
		TSet<ULandscapeComponent*> Components;
		LandscapeInfo->GetComponentsInRegion(x, y, x, y, Components);
		if (auto It = Components.CreateConstIterator(); It)
		{
			ULandscapeComponent* Component = *It;
			FLandscapeComponentDataInterface DataInterface(Component);
			return LandscapeDataAccess::GetLocalHeight(DataInterface.GetHeight(x - Component->SectionBaseX, y - Component->SectionBaseY));
		}
		return 0.0f;
	}

	/**
	 * @param SourceData  Data from the "source" side of the mirror op, including blend region
	 * @param DestData    Result of the mirror op, including blend region
	 * @param SourceSizeX Width of SourceData
	 * @param SourceSizeY Height of SourceData
	 * @param DestSizeX   Width of DestData
	 * @param DestSizeY   Height of DestData
	 * @param MirrorPos   Position of the mirror point in the source data (X or Y pos depending on MirrorOp)
	 * @param BlendWidth  Width of the blend region (in X or Y axis depending on MirrorOp)
	 */
	template<typename T>
	void ApplyMirrorInternal(TArrayView<const T> SourceData, TArrayView<T> DestData, int32 SourceSizeX, int32 SourceSizeY, int32 DestSizeX, int32 DestSizeY, int32 MirrorPos, int32 BlendWidth)
	{
		checkSlow(SourceData.Num() == SourceSizeX * SourceSizeY);
		checkSlow(DestData.Num()   == DestSizeX   * DestSizeY);

		switch (EdMode->UISettings->MirrorOp)
		{
		case ELandscapeMirrorOperation::MinusXToPlusX:
		case ELandscapeMirrorOperation::RotateMinusXToPlusX:
			{
				checkSlow(SourceSizeY == DestSizeY);
				checkSlow(MirrorPos + BlendWidth + 1 == SourceSizeX);
				const int32 BlendStart = (DestSizeX - MirrorPos - 1) - BlendWidth;
				const int32 BlendEnd   = BlendStart + 2 * BlendWidth + 1;
				const int32 Offset = 2 * MirrorPos - DestSizeX + 1;
				const bool bFlipY = (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusXToPlusX);
				for (int32 Y = 0; Y < DestSizeY; ++Y)
				{
					TArrayView<const T> SourceLine1 = SourceData.Slice(Y * SourceSizeX, SourceSizeX);
					TArrayView<const T> SourceLine2 = bFlipY ? SourceData.Slice((SourceSizeY - Y - 1) * SourceSizeX, SourceSizeX) : SourceLine1;
					TArrayView<T> DestLine = DestData.Slice(Y * DestSizeX, DestSizeX);

					// Pre-blend
					int32 DestX = 0;
					for (int32 SourceX = Offset; DestX < BlendStart; ++DestX, ++SourceX)
					{
						DestLine.GetData()[DestX] = SourceLine1.GetData()[SourceX];
					}

					// Blend
					for (int32 SourceX1 = BlendStart + Offset, SourceX2 = BlendEnd + Offset - 1;
						 DestX < BlendEnd; ++DestX, ++SourceX1, --SourceX2)
					{
						const float Frac = (float)(DestX - BlendStart + 1) / (BlendEnd - BlendStart + 1);
						const float Alpha = FMath::Cos(Frac * PI) * -0.5f + 0.5f;
						DestLine.GetData()[DestX] = FMath::Lerp(SourceLine1.GetData()[SourceX1], SourceLine2.GetData()[SourceX2], Alpha);
					}

					// Post-Blend
					for (int32 SourceX = BlendStart + Offset - 1; DestX < DestSizeX; ++DestX, --SourceX)
					{
						DestLine.GetData()[DestX] = SourceLine2.GetData()[SourceX];
					}
				}
			}
			break;
		case ELandscapeMirrorOperation::PlusXToMinusX:
		case ELandscapeMirrorOperation::RotatePlusXToMinusX:
			{
				checkSlow(SourceSizeY == DestSizeY);
				const int32 BlendStart = (SourceSizeX - MirrorPos - 1) - BlendWidth;
				const int32 BlendEnd   = BlendStart + 2 * BlendWidth + 1;
				const int32 Offset = 2 * MirrorPos - SourceSizeX + 1;
				const bool bFlipY = (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusXToMinusX);
				for (int32 Y = 0; Y < DestSizeY; ++Y)
				{
					TArrayView<const T> SourceLine1 = SourceData.Slice(Y * SourceSizeX, SourceSizeX);
					TArrayView<const T> SourceLine2 = bFlipY ? SourceData.Slice((SourceSizeY - Y - 1) * SourceSizeX, SourceSizeX) : SourceLine1;
					TArrayView<T> DestLine = DestData.Slice(Y * DestSizeX, DestSizeX);

					// Pre-blend
					int32 DestX = 0;
					for (int32 SourceX = SourceSizeX - 1; DestX < BlendStart; ++DestX, --SourceX)
					{
						DestLine.GetData()[DestX] = SourceLine2.GetData()[SourceX];
					}

					// Blend
					for (int32 SourceX1 = BlendStart + Offset, SourceX2 = BlendEnd + Offset - 1;
						 DestX < BlendEnd; ++DestX, ++SourceX1, --SourceX2)
					{
						const float Frac = (float)(DestX - BlendStart + 1) / (BlendEnd - BlendStart + 1);
						const float Alpha = FMath::Cos(Frac * PI) * -0.5f + 0.5f;
						DestLine.GetData()[DestX] = FMath::Lerp(SourceLine2.GetData()[SourceX2], SourceLine1.GetData()[SourceX1], Alpha);
					}

					// Post-Blend
					for (int32 SourceX = BlendEnd + Offset; DestX < DestSizeX; ++DestX, ++SourceX)
					{
						DestLine.GetData()[DestX] = SourceLine1.GetData()[SourceX];
					}
				}
			}
			break;
		case ELandscapeMirrorOperation::MinusYToPlusY:
		case ELandscapeMirrorOperation::RotateMinusYToPlusY:
			{
				checkSlow(SourceSizeX == DestSizeX);
				checkSlow(MirrorPos + BlendWidth + 1 == SourceSizeY);
				const int32 BlendStart = (DestSizeY - MirrorPos - 1) - BlendWidth;
				const int32 BlendEnd   = BlendStart + 2 * BlendWidth + 1;
				const int32 Offset = 2 * MirrorPos - DestSizeY + 1;
				const bool bFlipX = (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusYToPlusY);

				// Pre-blend
				int32 DestY = 0;
				for (int32 SourceY = Offset; DestY < BlendStart; ++DestY, ++SourceY)
				{
					TArrayView<const T> SourceLine = SourceData.Slice(SourceY * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					FMemory::Memcpy(DestLine.GetData(), SourceLine.GetData(), DestSizeX * sizeof(T));
				}

				// Blend
				for (int32 SourceY1 = BlendStart + Offset, SourceY2 = BlendEnd + Offset - 1;
					 DestY < BlendEnd; ++DestY, ++SourceY1, --SourceY2)
				{
					const float Frac = (float)(DestY - BlendStart + 1) / (BlendEnd - BlendStart + 1);
					const float Alpha = FMath::Cos(Frac * PI) * -0.5f + 0.5f;
					TArrayView<const T> SourceLine1 = SourceData.Slice(SourceY1 * SourceSizeX, SourceSizeX);
					TArrayView<const T> SourceLine2 = SourceData.Slice(SourceY2 * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					for (int32 DestX = 0; DestX < DestSizeX; ++DestX)
					{
						const int32 SourceX2 = bFlipX ? (SourceSizeX - DestX - 1) : DestX;
						DestLine.GetData()[DestX] = FMath::Lerp(SourceLine1.GetData()[DestX], SourceLine2.GetData()[SourceX2], Alpha);
					}
				}

				// Post-Blend
				for (int32 SourceY = BlendStart + Offset - 1; DestY < DestSizeY; ++DestY, --SourceY)
				{
					TArrayView<const T> SourceLine = SourceData.Slice(SourceY * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					FMemory::Memcpy(DestLine.GetData(), SourceLine.GetData(), DestSizeX * sizeof(T));
					if (bFlipX)
					{
						Algo::Reverse(DestLine);
					}
				}
			}
			break;
		case ELandscapeMirrorOperation::PlusYToMinusY:
		case ELandscapeMirrorOperation::RotatePlusYToMinusY:
			{
				checkSlow(SourceSizeX == DestSizeX);
				const int32 BlendStart = (SourceSizeY - MirrorPos - 1) - BlendWidth;
				const int32 BlendEnd   = BlendStart + 2 * BlendWidth + 1;
				const int32 Offset = 2 * MirrorPos - SourceSizeY + 1;
				const bool bFlipX = (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotatePlusYToMinusY);

				// Pre-blend
				int32 DestY = 0;
				for (int32 SourceY = SourceSizeY - 1; DestY < BlendStart; ++DestY, --SourceY)
				{
					TArrayView<const T> SourceLine = SourceData.Slice(SourceY * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					FMemory::Memcpy(DestLine.GetData(), SourceLine.GetData(), DestSizeX * sizeof(T));
					if (bFlipX)
					{
						Algo::Reverse(DestLine);
					}
				}

				// Blend
				for (int32 SourceY1 = BlendStart + Offset, SourceY2 = BlendEnd + Offset - 1;
					 DestY < BlendEnd; ++DestY, ++SourceY1, --SourceY2)
				{
					const float Frac = (float)(DestY - BlendStart + 1) / (BlendEnd - BlendStart + 1);
					const float Alpha = FMath::Cos(Frac * PI) * -0.5f + 0.5f;
					TArrayView<const T> SourceLine1 = SourceData.Slice(SourceY1 * SourceSizeX, SourceSizeX);
					TArrayView<const T> SourceLine2 = SourceData.Slice(SourceY2 * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					for (int32 DestX = 0; DestX < DestSizeX; ++DestX)
					{
						const int32 SourceX2 = bFlipX ? (SourceSizeX - DestX - 1) : DestX;
						DestLine.GetData()[DestX] = FMath::Lerp(SourceLine2.GetData()[SourceX2], SourceLine1.GetData()[DestX], Alpha);
					}
				}

				// Post-Blend
				for (int32 SourceY = BlendEnd + Offset; DestY < DestSizeY; ++DestY, ++SourceY)
				{
					TArrayView<const T> SourceLine = SourceData.Slice(SourceY * SourceSizeX, SourceSizeX);
					TArrayView<T> DestLine = DestData.Slice(DestY * DestSizeX, DestSizeX);
					FMemory::Memcpy(DestLine.GetData(), SourceLine.GetData(), DestSizeX * sizeof(T));
				}
			}
			break;
		default:
			check(0);
			return;
		}
	}

public:
	virtual void ApplyMirror()
	{
		FText Reason;
		if (!EdMode->CanEditLayer(&Reason))
		{
			FMessageDialog::Open(EAppMsgType::Ok, Reason);
			return;
		}
		FScopedTransaction Transaction(LOCTEXT("Mirror_Apply", "Landscape Editing: Mirror Landscape"));
		FScopedSetLandscapeEditingLayer Scope(EdMode->GetLandscape(), EdMode->GetCurrentLayerGuid(), [&] { EdMode->RequestLayersContentUpdateForceAll(); });

		const ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo == nullptr)
		{
			return;
		}

		const ALandscapeProxy* const LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		if (LandscapeProxy == nullptr)
		{
			return;
		}

		const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();
		const FVector2D MirrorPoint = EdMode->UISettings->MirrorPoint;
		int32 BlendWidth = FMath::Clamp(EdMode->UISettings->MirrorSmoothingWidth, 0, 32768);

		FLandscapeEditDataInterface LandscapeEdit(EdMode->CurrentToolTarget.LandscapeInfo.Get());

		int32 MinX, MinY, MaxX, MaxY;
		if (!LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
		{
			return;
		}

		const int32 SizeX = (1 + MaxX - MinX);
		const int32 SizeY = (1 + MaxY - MinY);

		int32 SourceMinX, SourceMinY;
		int32 SourceMaxX, SourceMaxY;
		int32 DestMinX, DestMinY;
		int32 DestMaxX, DestMaxY;
		int32 MirrorPos;

		switch (EdMode->UISettings->MirrorOp)
		{
		case ELandscapeMirrorOperation::MinusXToPlusX:
		case ELandscapeMirrorOperation::RotateMinusXToPlusX:
		case ELandscapeMirrorOperation::PlusXToMinusX:
		case ELandscapeMirrorOperation::RotatePlusXToMinusX:
			{
				MirrorPos = FMath::RoundToInt32(MirrorPoint.X);
				if (MirrorPos <= MinX || MirrorPos >= MaxX)
				{
					return;
				}
				const int32 MirrorSize = FMath::Max(MaxX - MirrorPos, MirrorPos - MinX); // not including the mirror column itself
				BlendWidth = FMath::Min(BlendWidth, MirrorSize);
				if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::MinusXToPlusX ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusXToPlusX)
				{
					SourceMinX = MirrorPos - MirrorSize;
					SourceMaxX = MirrorPos + BlendWidth;
					DestMinX = MirrorPos - BlendWidth - 1; // extra column to calc normals for mirror column
					DestMaxX = MirrorPos + MirrorSize;
				}
				else
				{
					SourceMinX = MirrorPos - BlendWidth;
					SourceMaxX = MirrorPos + MirrorSize;
					DestMinX = MirrorPos - MirrorSize;
					DestMaxX = MirrorPos + BlendWidth + 1; // extra column to calc normals for mirror column
				}
				SourceMinY = MinY;
				SourceMaxY = MaxY;
				DestMinY = MinY;
				DestMaxY = MaxY;
				MirrorPos -= SourceMinX;
			}
			break;
		case ELandscapeMirrorOperation::MinusYToPlusY:
		case ELandscapeMirrorOperation::RotateMinusYToPlusY:
		case ELandscapeMirrorOperation::PlusYToMinusY:
		case ELandscapeMirrorOperation::RotatePlusYToMinusY:
			{
				MirrorPos = FMath::RoundToInt32(MirrorPoint.Y);
				if (MirrorPos <= MinY || MirrorPos >= MaxY)
				{
					return;
				}
				const int32 MirrorSize = FMath::Max(MaxY - MirrorPos, MirrorPos - MinY); // not including the mirror row itself
				SourceMinX = MinX;
				SourceMaxX = MaxX;
				DestMinX = MinX;
				DestMaxX = MaxX;
				if (EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::MinusYToPlusY ||
					EdMode->UISettings->MirrorOp == ELandscapeMirrorOperation::RotateMinusYToPlusY)
				{
					SourceMinY = MirrorPos - MirrorSize;
					SourceMaxY = MirrorPos + BlendWidth;
					DestMinY = MirrorPos - BlendWidth - 1; // extra column to calc normals for mirror column
					DestMaxY = MirrorPos + MirrorSize;
				}
				else
				{
					SourceMinY = MirrorPos - BlendWidth;
					SourceMaxY = MirrorPos + MirrorSize;
					DestMinY = MirrorPos - MirrorSize;
					DestMaxY = MirrorPos + BlendWidth + 1; // extra column to calc normals for mirror column
				}
				MirrorPos -= SourceMinY;
			}
			break;
		default:
			check(0);
			return;
		}

		const int32 SourceSizeX = SourceMaxX - SourceMinX + 1;
		const int32 SourceSizeY = SourceMaxY - SourceMinY + 1;
		const int32 DestSizeX = DestMaxX - DestMinX + 1;
		const int32 DestSizeY = DestMaxY - DestMinY + 1;

		TArray<uint16> SourceHeightData, DestHeightData;
		SourceHeightData.AddUninitialized(SourceSizeX * SourceSizeY);
		DestHeightData.AddUninitialized(DestSizeX * DestSizeY);
		int32 TempMinX = SourceMinX; // GetHeightData overwrites its input min/max x/y
		int32 TempMaxX = SourceMaxX;
		int32 TempMinY = SourceMinY;
		int32 TempMaxY = SourceMaxY;
		LandscapeEdit.GetHeightData(TempMinX, TempMinY, TempMaxX, TempMaxY, &SourceHeightData[0], SourceSizeX);
		ApplyMirrorInternal<uint16>(SourceHeightData, DestHeightData, SourceSizeX, SourceSizeY, DestSizeX, DestSizeY, MirrorPos, BlendWidth);
		LandscapeEdit.SetHeightData(DestMinX, DestMinY, DestMaxX, DestMaxY, &DestHeightData[0], DestSizeX, true);

		TArray<uint8> SourceWeightData, DestWeightData;
		SourceWeightData.AddUninitialized(SourceSizeX * SourceSizeY);
		DestWeightData.AddUninitialized(DestSizeX * DestSizeY);
		for (const auto& LayerSettings : LandscapeInfo->Layers)
		{
			ULandscapeLayerInfoObject* LayerInfo = LayerSettings.LayerInfoObj;
			if (LayerInfo)
			{
				TempMinX = SourceMinX; // GetWeightData overwrites its input min/max x/y
				TempMaxX = SourceMaxX;
				TempMinY = SourceMinY;
				TempMaxY = SourceMaxY;
				LandscapeEdit.GetWeightData(LayerInfo, TempMinX, TempMinY, TempMaxX, TempMaxY, &SourceWeightData[0], SourceSizeX);
				ApplyMirrorInternal<uint8>(SourceWeightData, DestWeightData, SourceSizeX, SourceSizeY, DestSizeX, DestSizeY, MirrorPos, BlendWidth);
				LandscapeEdit.SetAlphaData(LayerInfo, DestMinX, DestMinY, DestMaxX, DestMaxY, &DestWeightData[0], DestSizeX, ELandscapeLayerPaintingRestriction::None);
				//LayerInfo->IsReferencedFromLoadedData = true;
			}
		}

		LandscapeEdit.Flush();

		TSet<ULandscapeComponent*> Components;
		if (LandscapeEdit.GetComponentsInRegion(DestMinX, DestMinY, DestMaxX, DestMaxY, &Components) && Components.Num() > 0)
		{
			EdMode->UpdateLayerUsageInformation();
		}
	}

	void CenterMirrorPoint()
	{
		ULandscapeInfo* const LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		int32 MinX, MinY, MaxX, MaxY;
		if (LandscapeInfo && LandscapeInfo->GetLandscapeExtent(MinX, MinY, MaxX, MaxY))
		{
			EdMode->UISettings->MirrorPoint = FVector2D((float)(MinX + MaxX) / 2.0f, (float)(MinY + MaxY) / 2.0f);
		}
		else
		{
			EdMode->UISettings->MirrorPoint = FVector2D(0, 0);
		}
	}
};

void FEdModeLandscape::ApplyMirrorTool()
{
	if (CurrentTool->GetToolName() == FName("Mirror"))
	{
		FLandscapeToolMirror* MirrorTool = (FLandscapeToolMirror*)CurrentTool;
		MirrorTool->ApplyMirror();
		GEditor->RedrawLevelEditingViewports();
	}
}

void FEdModeLandscape::CenterMirrorTool()
{
	if (CurrentTool->GetToolName() == FName("Mirror"))
	{
		FLandscapeToolMirror* MirrorTool = (FLandscapeToolMirror*)CurrentTool;
		MirrorTool->CenterMirrorPoint();
		GEditor->RedrawLevelEditingViewports();
	}
}

//
// Toolset initialization
//
void FEdModeLandscape::InitializeTool_Mirror()
{
	auto Tool_Mirror = MakeUnique<FLandscapeToolMirror>(this);
	Tool_Mirror->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_Mirror));
}

#undef LOCTEXT_NAMESPACE
