// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Algo/Find.h"
#include "Misc/App.h"
#include "InputCoreTypes.h"
#include "Engine/EngineTypes.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdit.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEdModeTools.h"
#include "LandscapeSettings.h"
#include "Landscape.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Logging/LogMacros.h"
#include "Misc/MapErrors.h"
#include "EngineModule.h"

#define LOCTEXT_NAMESPACE "LandscapeTools"

DEFINE_LOG_CATEGORY(LogLandscapeTools);

const int32 FNoiseParameter::Permutations[256] =
{
	151, 160, 137, 91, 90, 15,
	131, 13, 201, 95, 96, 53, 194, 233, 7, 225, 140, 36, 103, 30, 69, 142, 8, 99, 37, 240, 21, 10, 23,
	190, 6, 148, 247, 120, 234, 75, 0, 26, 197, 62, 94, 252, 219, 203, 117, 35, 11, 32, 57, 177, 33,
	88, 237, 149, 56, 87, 174, 20, 125, 136, 171, 168, 68, 175, 74, 165, 71, 134, 139, 48, 27, 166,
	77, 146, 158, 231, 83, 111, 229, 122, 60, 211, 133, 230, 220, 105, 92, 41, 55, 46, 245, 40, 244,
	102, 143, 54, 65, 25, 63, 161, 1, 216, 80, 73, 209, 76, 132, 187, 208, 89, 18, 169, 200, 196,
	135, 130, 116, 188, 159, 86, 164, 100, 109, 198, 173, 186, 3, 64, 52, 217, 226, 250, 124, 123,
	5, 202, 38, 147, 118, 126, 255, 82, 85, 212, 207, 206, 59, 227, 47, 16, 58, 17, 182, 189, 28, 42,
	223, 183, 170, 213, 119, 248, 152, 2, 44, 154, 163, 70, 221, 153, 101, 155, 167, 43, 172, 9,
	129, 22, 39, 253, 19, 98, 108, 110, 79, 113, 224, 232, 178, 185, 112, 104, 218, 246, 97, 228,
	251, 34, 242, 193, 238, 210, 144, 12, 191, 179, 162, 241, 81, 51, 145, 235, 249, 14, 239, 107,
	49, 192, 214, 31, 181, 199, 106, 157, 184, 84, 204, 176, 115, 121, 50, 45, 127, 4, 150, 254,
	138, 236, 205, 93, 222, 114, 67, 29, 24, 72, 243, 141, 128, 195, 78, 66, 215, 61, 156, 180
};

// 
// FLandscapeToolPaintBase
//
template<class TToolTarget, class TStrokeClass>
class FLandscapeToolPaintBase : public FLandscapeToolBase<TStrokeClass>
{
	using Super = FLandscapeToolBase<TStrokeClass>;

public:
	FLandscapeToolPaintBase(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return TToolTarget::TargetType;
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		// By default, consider that all tools deriving from FLandscapeToolPaintBase will affect this layer if it's in the list of targeted layers :
		return this->EdMode->CurrentToolTarget.GetLayerInfos().Contains(InLayerInfo) ? ELandscapeTargetLayerActionType::Normal : ELandscapeTargetLayerActionType::None;
	}
};


template<class ToolTarget>
class FLandscapeToolStrokePaintBase : public FLandscapeToolStrokeBase
{
	using Super = FLandscapeToolStrokeBase;

protected:
	using ValueType = typename ToolTarget::CacheClass::DataType;
	using CacheType = typename ToolTarget::CacheClass;

public:
	FLandscapeToolStrokePaintBase(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
		const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
		PaintStrengthGamma = Settings->PaintStrengthGamma;
		bDisablePaintingStartupSlowdown = Settings->bDisablePaintingStartupSlowdown;

		// Instantiate a cache object for every affected layer (note : null layer info object pointer is accepted for heightmaps!)
		TArray<ULandscapeLayerInfoObject*> AffectedLayerInfos = InTarget.GetLayerInfos();
		PerTargetLayerCache.Reserve(AffectedLayerInfos.Num());
		for (ULandscapeLayerInfoObject* LayerInfo : AffectedLayerInfos)
		{ 
			PerTargetLayerCache.Add(LayerInfo, CacheType(InTarget.LandscapeInfo.Get(), LayerInfo));
		}
	}

	float GetStrength(const ULandscapeEditorObject* UISettings) const
	{
		// Apply paint strength gamma for paint strokes only : 
		return (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap) 
			? UISettings->GetCurrentToolStrength() 
			: FMath::Pow(UISettings->GetCurrentToolStrength(), PaintStrengthGamma);
	}

	CacheType& GetCacheForTargetLayer(ULandscapeLayerInfoObject* InLayerInfo)
	{
		checkf((ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap) ^ (InLayerInfo != nullptr), TEXT("Heightmap type requires no target layer, weightmap type requires one"));
		CacheType* Cache = PerTargetLayerCache.Find(InLayerInfo);
		check(Cache != nullptr);
		return *Cache;
	}

protected:
	TMap<ULandscapeLayerInfoObject*, CacheType> PerTargetLayerCache;
	float PaintStrengthGamma = 1.0f;
	bool bDisablePaintingStartupSlowdown = false;
};

// 
// FLandscapeToolPaint
//
class FLandscapeToolStrokePaint : public FLandscapeToolStrokePaintBase<FWeightmapToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<FWeightmapToolTarget>;

public:

	/** Struct that holds the state of a given target layer wrt the paint stroke */
	struct FAffectedTargetLayer
	{
	public:
		FAffectedTargetLayer(ULandscapeLayerInfoObject* InLayer, bool bInInverted)
			: Layer(InLayer)
			, bInverted(bInInverted)
		{}

		ULandscapeLayerInfoObject* GetLayer() const
		{
			return Layer;
		}

		bool IsInverted() const
		{
			return bInverted;
		}

		ELandscapeTargetLayerActionType GetTargetLayerActionType(bool bInvertRequested, bool bInExclusivePaintingRequested) const
		{
			// Only let inverted layers be painted when allowed
			if (!bInExclusivePaintingRequested && bInverted)
			{
				return ELandscapeTargetLayerActionType::None;
			}

			// Don't affect inverted layers when invert-painting : simply erase the painted layers in that case :
			if (bInvertRequested && bInverted)
			{
				return ELandscapeTargetLayerActionType::None;
			}

			// Apply invert on non-inverted layers when we're not invert-painting already :
			const bool bLayerInvert = bInvertRequested || bInverted;
			return bLayerInvert ? ELandscapeTargetLayerActionType::Inverted : ELandscapeTargetLayerActionType::Normal;
		}

	private:
		/** The target layer being affected */
		ULandscapeLayerInfoObject* Layer = nullptr;
		/** Indicates that the effect of this paint stroke on this layer should be inverted (when exclusive-painting) */
		bool bInverted = false;
	};

	FLandscapeToolStrokePaint(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
		, bIsAllowListMode(EdMode->UISettings->PaintingRestriction == ELandscapeLayerPaintingRestriction::UseComponentAllowList &&
		                   (InViewportClient->Viewport->KeyState(EKeys::Equals) || InViewportClient->Viewport->KeyState(EKeys::Hyphen)))
		, bAddToAllowList(bIsAllowListMode && InViewportClient->Viewport->KeyState(EKeys::Equals))
		, AllAffectedTargetLayers(GetAffectedTargetLayersForTarget(InTarget))
	{

		// Add a cache entry in the alternate cache for each affected target layer that is inverted (and verify that others have an entry in the normal cache already)
		for (const FAffectedTargetLayer& AffectedTargetLayer : AllAffectedTargetLayers)
		{
			if (AffectedTargetLayer.IsInverted())
			{
				check(!this->PerTargetLayerCache.Contains(AffectedTargetLayer.GetLayer())); // if it's inverted, it should not be in PerTargetLayerCache
				AlternatePerTargetLayerCache.Add(AffectedTargetLayer.GetLayer(), CacheType(InTarget.LandscapeInfo.Get(), AffectedTargetLayer.GetLayer()));
			}
			else
			{
				check(this->PerTargetLayerCache.Contains(AffectedTargetLayer.GetLayer())); // if it's not inverted, it should be in PerTargetLayerCache already
			}
		}
	}

	static TArray<FAffectedTargetLayer> GetAffectedTargetLayersForTarget(const FLandscapeToolTarget& InTarget)
	{
		TArray<FAffectedTargetLayer> AffectedTargetLayers;
		TSet<ULandscapeLayerInfoObject*> TargetedLayerInfos(InTarget.GetLayerInfos());
		Algo::Transform(TargetedLayerInfos, AffectedTargetLayers, [](ULandscapeLayerInfoObject* InLayer) { return FAffectedTargetLayer(InLayer, /*bInInverted = */false); });

		TArray<FTargetLayerBlendGroup> BlendGroups = InTarget.LandscapeInfo->GetTargetLayerBlendGroups();

		for (ULandscapeLayerInfoObject* TargetLayerInfo : TargetedLayerInfos)
		{
			TOptional<FTargetLayerBlendGroup> BlendGroup = InTarget.LandscapeInfo->GetBlendGroupForTargetLayer(TargetLayerInfo);
			if (BlendGroup.IsSet())
			{
				// Now create a cache entry for each of the target layers that are in the blend group and not in the list of affected layers. These will be inversely affected
				for (TObjectPtr<ULandscapeLayerInfoObject> LayerInfo : BlendGroup->GetTargetLayers())
				{
					if (!TargetedLayerInfos.Contains(LayerInfo))
					{
						check(!AffectedTargetLayers.ContainsByPredicate([LayerInfo](const FAffectedTargetLayer& InLayer){ return InLayer.GetLayer() == LayerInfo; }));
						AffectedTargetLayers.Emplace(LayerInfo, /*bInInverted = */true);
					}
				}
			}
		}

		return AffectedTargetLayers;
	}

	static bool IsExclusivePaintingRequested(const ULandscapeEditorObject* UISettings, bool bInAlternateModifierPressed)
	{
		switch (UISettings->WeightBlendedTargetLayerPaintMode)
		{ 
		case ELandscapeWeightBlendPaintMode::ExclusiveRequiresCtrl:
		{
			return bInAlternateModifierPressed;
		}
		case ELandscapeWeightBlendPaintMode::ExclusiveRequiresNoCtrl:
		{
			return !bInAlternateModifierPressed;
		}
		case ELandscapeWeightBlendPaintMode::ForceExclusive:
		{
			return true;
		}
		case ELandscapeWeightBlendPaintMode::ForceNonExclusive:
		{
			return false;
		}
		default:
			checkNoEntry();
		}

		return false;
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokePaint_Apply);

		// Invert when holding Shift
		const bool bInvert = InteractorPositions.Last().bModifierPressed;
		const bool bExclusivePaintingRequested = IsExclusivePaintingRequested(UISettings, InteractorPositions.Last().bAlternateModifierPressed);
		UE_LOGF(LogLandscapeTools, VeryVerbose, "bInvert = %d, bExclusivePaintingRequested = %d", bInvert, bExclusivePaintingRequested);

		// TODO [jonathan.bard] : this should move one level up, as this should also be doable when using FLandscapeBrushComponent : 
		if (bIsAllowListMode)
		{
			// Get list of components overlapped by the brush : 
			FLandscapeBrush::FOverlapInfo OverlapInfo = Brush->GetOverlapInfo(InteractorPositions);
			if (!OverlapInfo)
			{
				return;
			}

			TSet<ULandscapeLayerInfoObject*> SelectedLayers;
			PerTargetLayerCache.GetKeys(SelectedLayers);

			TSet<ULandscapeComponent*> SelectedComponents = OverlapInfo.OverlappedLoadedComponents;

			for (ULandscapeComponent* Component : SelectedComponents)
			{
				for (ULandscapeLayerInfoObject* LayerInfo : SelectedLayers)
				{
					if (bAddToAllowList && !Component->LayerAllowList.Contains(LayerInfo))
					{
						Component->Modify();
						Component->LayerAllowList.Add(LayerInfo);
					}
					else if (!bAddToAllowList && Component->LayerAllowList.Contains(LayerInfo))
					{
						Component->Modify();
						FLandscapeEditDataInterface LandscapeEdit(LandscapeInfo);
						Component->LayerAllowList.RemoveSingle(LayerInfo);
						Component->DeleteLayer(LayerInfo, LandscapeEdit);
					}
				}
			}

			return;
		}

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		const bool bUseWeightTargetValue = UISettings->bUseWeightTargetValue;
		const bool bCacheOriginalData = !bUseWeightTargetValue;

		for (const FAffectedTargetLayer& AffectedLayer : AllAffectedTargetLayers)
		{
			const ELandscapeTargetLayerActionType LayerActionType = AffectedLayer.GetTargetLayerActionType(bInvert, bExclusivePaintingRequested);
			if (LayerActionType == ELandscapeTargetLayerActionType::None)
			{
				continue;
			}

			CacheType* Cache = AffectedLayer.IsInverted() ? AlternatePerTargetLayerCache.Find(AffectedLayer.GetLayer()) : this->PerTargetLayerCache.Find(AffectedLayer.GetLayer());
			check(Cache != nullptr);

			TMap<FIntPoint, float>& TotalInfluenceMap = PerTargetTotalInfluenceMap.FindOrAdd(AffectedLayer.GetLayer());

			Cache->CacheData(X1, Y1, X2, Y2, bCacheOriginalData);

			// The data we'll be writing to
			TArray<ValueType> Data;
			Cache->GetCachedData(X1, Y1, X2, Y2, Data);

			// The source data we use for editing. 
			TArray<ValueType>* SourceDataArrayPtr = &Data;
			TArray<ValueType> OriginalData;

			if (!bUseWeightTargetValue)
			{
				// When painting weights (and not using target value mode), we use a source value that tends more
				// to the current value as we paint over the same region multiple times.
				// TODO: Make this frame-rate independent
				Cache->GetOriginalData(X1, Y1, X2, Y2, OriginalData);
				SourceDataArrayPtr = &OriginalData;

				for (int32 Y = Y1; Y <= Y2; Y++)
				{
					ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
					ValueType* OriginalDataScanline = OriginalData.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
					for (int32 X = X1; X <= X2; X++)
					{
						ValueType& SourceValue = OriginalDataScanline[X];
						ValueType& CurrentValue = DataScanline[X];
						if (bDisablePaintingStartupSlowdown)
						{
							SourceValue = CurrentValue;
						}
						else
						{
							float VertexInfluence = TotalInfluenceMap.FindRef(FIntPoint(X, Y));
							SourceValue = FMath::Lerp(SourceValue, CurrentValue, FMath::Min<float>(VertexInfluence * 0.05f, 1.0f));
						}
					}
				}
			}

			// Adjust strength based on brush size and drawscale, so strength 1 = one hemisphere
			const float AdjustedStrength = ToolTarget::StrengthMultiplier(this->LandscapeInfo, UISettings->GetCurrentToolBrushRadius());
			float PaintStrength = this->GetStrength(UISettings) * Pressure * AdjustedStrength;
		
			ValueType DestValue = FWeightmapToolTarget::CacheClass::ClampValue(static_cast<int32>(255.0f * UISettings->WeightTargetValue));

			// TODO: make paint tool framerate independent like the sculpt tool
			// const float DeltaTime = FMath::Min<float>(FApp::GetDeltaTime(), 0.1f); // Under 10 fps slow down paint speed
			// SculptStrength *= DeltaTime * 3.0f; // * 3.0f to partially compensate for impact of DeltaTime on slowing the tools down compared to the old framerate-dependent version

			if (PaintStrength <= 0.0f)
			{
				continue;
			}

			// Erasing cancels target value mode, it feels weird otherwise : we cannot "invent" a new target value and we also cannot *not* respect the inverted action contract, 
			//  otherwise, it feels like the tool is ineffective :
			const bool bUseWeightTargetValueForLayer = (LayerActionType == ELandscapeTargetLayerActionType::Inverted) ? false : bUseWeightTargetValue;
			if (!bUseWeightTargetValueForLayer)
			{
				PaintStrength = FMath::Max(PaintStrength, 1.0f);
			}

			// Apply the brush
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				ValueType* SourceDataScanline = SourceDataArrayPtr->GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const FIntPoint Key = FIntPoint(X, Y);
					const float BrushValue = BrushScanline[X];

					// Update influence map
					float VertexInfluence = TotalInfluenceMap.FindRef(Key);
					TotalInfluenceMap.Add(Key, VertexInfluence + BrushValue);

					float PaintAmount = BrushValue * PaintStrength;
					ValueType& CurrentValue = DataScanline[X];
					const ValueType& SourceValue = SourceDataScanline[X];

					if (bUseWeightTargetValueForLayer)
					{
						CurrentValue = FMath::Lerp(CurrentValue, DestValue, PaintAmount / AdjustedStrength);
					}
					else
					{
						const int32 IntPaintAmount = FMath::RoundToInt(PaintAmount);
						if (LayerActionType == ELandscapeTargetLayerActionType::Inverted)
						{
							CurrentValue = CacheType::ClampValue(FMath::Min<int32>(SourceValue - IntPaintAmount, CurrentValue));
						}
						else
						{
							CurrentValue = CacheType::ClampValue(FMath::Max<int32>(SourceValue + IntPaintAmount, CurrentValue));
						}
					}
				}
			}

			Cache->SetCachedData(X1, Y1, X2, Y2, Data, UISettings->PaintingRestriction);
			Cache->Flush();
		}
		
		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}

protected:
	using ToolTarget = FWeightmapToolTarget;
	using ValueType = Super::ValueType;
	using CacheType = Super::CacheType;

	// TODO [jonathan.bard] : remove this, it's only used for bDisablePaintingStartupSlowdown == false, which likely no one uses
	TMap<ULandscapeLayerInfoObject*, TMap<FIntPoint, float>> PerTargetTotalInfluenceMap; // amount of time and weight the brush has spent on each vertex.

	bool bIsAllowListMode;
	bool bAddToAllowList;

	/** Contains info about all the layers being affected by this paint stroke */
	TArray<FAffectedTargetLayer> AllAffectedTargetLayers;

	/** Cache for each FAffectedTargetLayer that were not cached by the base class's PerTargetLayerCache (inverted layers) */
	TMap<ULandscapeLayerInfoObject*, CacheType> AlternatePerTargetLayerCache;
};

class FLandscapeToolPaint : public FLandscapeToolPaintBase<FWeightmapToolTarget, FLandscapeToolStrokePaint>
{
	using Super = FLandscapeToolPaintBase<FWeightmapToolTarget, FLandscapeToolStrokePaint>;

public:
	FLandscapeToolPaint(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Paint"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Paint", "Paint"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Paint_Message", "The Paint tool increases or decreases the weight of the Material layer being applied to the Landscape."); };

	virtual void EnterTool()
	{
		if (EdMode->UISettings->PaintingRestriction == ELandscapeLayerPaintingRestriction::UseComponentAllowList)
		{
			EdMode->UISettings->UpdateComponentLayerAllowList();
		}

		Super::EnterTool();
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		const bool bInvertRequested = bInIsModifierPressed;
		const bool bExclusivePaintingRequested = FLandscapeToolStrokePaint::IsExclusivePaintingRequested(this->EdMode->UISettings, bInIsAlternateModifierPressed);

		const TArray<FLandscapeToolStrokePaint::FAffectedTargetLayer> AffectedTargetLayers = FLandscapeToolStrokePaint::GetAffectedTargetLayersForTarget(this->EdMode->CurrentToolTarget);
		const FLandscapeToolStrokePaint::FAffectedTargetLayer* AffectedLayer = AffectedTargetLayers.FindByPredicate([InLayerInfo](const FLandscapeToolStrokePaint::FAffectedTargetLayer& InAffectedLayer)
			{
				return InAffectedLayer.GetLayer() == InLayerInfo;
			});
		return AffectedLayer
			? AffectedLayer->GetTargetLayerActionType(bInvertRequested, bExclusivePaintingRequested)
			: ELandscapeTargetLayerActionType::None;
	}
};

class FLandscapeToolStrokeErase : public FLandscapeToolStrokePaintBase<FHeightmapToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<FHeightmapToolTarget>;

protected:
	using ToolTarget = FHeightmapToolTarget;
	using ValueType = Super::ValueType;
	using CacheType = Super::CacheType;

public:

	FLandscapeToolStrokeErase(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeErase_Apply);

		if (!this->LandscapeInfo) return;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		X1 -= 1;
		Y1 -= 1;
		X2 += 1;
		Y2 += 1;
		
		// Erase is currently heightmap-only so we can assume a single target layer : 
		CacheType& Cache = GetCacheForTargetLayer(/*InLayerInfo = */nullptr);
		Cache.CacheData(X1, Y1, X2, Y2);

		TArray<ValueType> Data;
		Cache.GetCachedData(X1, Y1, X2, Y2, Data);

		static const ValueType FlattenHeight = LandscapeDataAccess::GetTexHeight(0.f);

		// Apply the brush
		for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
		{
			const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
			ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

			for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
			{
				const float BrushValue = BrushScanline[X];

				if (BrushValue > 0.0f)
				{
					float Strength = FMath::Clamp<float>(BrushValue * UISettings->GetCurrentToolStrength() * Pressure, 0.0f, 1.0f);

					int32 Delta = DataScanline[X] - FlattenHeight;
					if (Delta > 0)
					{
						DataScanline[X] = static_cast<ValueType>(FMath::FloorToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenHeight, Strength)));
					}
					else
					{
						DataScanline[X] = static_cast<ValueType>(FMath::CeilToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenHeight, Strength)));
					}
				}
			}
		}

		Cache.SetCachedData(X1, Y1, X2, Y2, Data, UISettings->PaintingRestriction);
		Cache.Flush();

		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

//
class FLandscapeToolStrokeSculpt : public FLandscapeToolStrokePaintBase<FHeightmapToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<FHeightmapToolTarget>;

protected:
	using ToolTarget = FHeightmapToolTarget;
	using CacheType = Super::CacheType;
	using ValueType = Super::ValueType;

public:

	FLandscapeToolStrokeSculpt(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{}

	virtual void SetEditLayer(const FGuid& EditLayerGUID) override
	{
		// Sculpt is currently heightmap-only so we can assume a single target layer : 
		CacheType& Cache = GetCacheForTargetLayer(/*InLayerInfo = */nullptr);
		Cache.DataAccess.SetEditLayer(EditLayerGUID);
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeSculpt_Apply);

		// Invert when holding Shift
		const bool bInvert = InteractorPositions.Last().bModifierPressed;
		UE_LOGF(LogLandscapeTools, VeryVerbose, "bInvert = %d", bInvert);

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		X1 -= 1;
		Y1 -= 1;
		X2 += 1;
		Y2 += 1;

		// Sculpt is heightmap-only so we can assume a single target layer : 
		CacheType& Cache = GetCacheForTargetLayer(/*InLayerInfo = */nullptr);
		Cache.CacheData(X1, Y1, X2, Y2);

		bool bUseClayBrush = UISettings->bUseClayBrush;

		// The data we'll be writing to
		TArray<ValueType> Data;
		Cache.GetCachedData(X1, Y1, X2, Y2, Data);

		// The source data we use for editing. 
		TArray<ValueType>* SourceDataArrayPtr = &Data;

		FMatrix ToWorld = ToolTarget::ToWorldMatrix(this->LandscapeInfo);
		FMatrix FromWorld = ToolTarget::FromWorldMatrix(this->LandscapeInfo);

		// Adjust strength based on brush size and drawscale, so strength 1 = one hemisphere
		const float AdjustedStrength = ToolTarget::StrengthMultiplier(this->LandscapeInfo, UISettings->GetCurrentToolBrushRadius());

		float SculptStrength = UISettings->GetCurrentToolStrength() * Pressure * AdjustedStrength;
		const float DeltaTime = FMath::Min<float>(static_cast<float>(FApp::GetDeltaTime()), 0.1f); // Under 10 fps slow down paint speed
		SculptStrength *= DeltaTime * 3.0f; // * 3.0f to partially compensate for impact of DeltaTime on slowing the tools down compared to the old framerate-dependent version

		if (SculptStrength <= 0.0f)
		{
			return;
		}

		if (!bUseClayBrush)
		{
			SculptStrength = FMath::Max(SculptStrength, 1.0f);
		}

		FPlane BrushPlane(ForceInit);
		TArray<FVector> Normals;

		if (bUseClayBrush)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ComputeClayBrush);

			// Calculate normals for brush verts in data space
			Normals.Empty(SourceDataArrayPtr->Num());
			Normals.AddZeroed(SourceDataArrayPtr->Num());

			for (int32 Y = Y1; Y < Y2; Y++)
			{
				ValueType* SourceDataScanline_0 = SourceDataArrayPtr->GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				ValueType* SourceDataScanline_1 = SourceDataArrayPtr->GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				FVector* NormalsScanline_0 = Normals.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				FVector* NormalsScanline_1 = Normals.GetData() + (Y + 1 - Y1) * (X2 - X1 + 1) + (0 - X1);
				for (int32 X = X1; X < X2; X++)
				{
					FVector Vert00 = ToWorld.TransformPosition(FVector((float)X + 0.0f, (float)Y + 0.0f, SourceDataScanline_0[X + 0]));
					FVector Vert01 = ToWorld.TransformPosition(FVector((float)X + 0.0f, (float)Y + 1.0f, SourceDataScanline_1[X + 0]));
					FVector Vert10 = ToWorld.TransformPosition(FVector((float)X + 1.0f, (float)Y + 0.0f, SourceDataScanline_0[X + 1]));
					FVector Vert11 = ToWorld.TransformPosition(FVector((float)X + 1.0f, (float)Y + 1.0f, SourceDataScanline_1[X + 1]));

					FVector FaceNormal1 = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
					FVector FaceNormal2 = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();

					// contribute to the vertex normals.
					NormalsScanline_0[X + 1] += FaceNormal1;
					NormalsScanline_1[X + 0] += FaceNormal2;
					NormalsScanline_0[X + 0] += FaceNormal1 + FaceNormal2;
					NormalsScanline_1[X + 1] += FaceNormal1 + FaceNormal2;
				}
			}
			for (int32 Y = Y1; Y <= Y2; Y++)
			{
				FVector* NormalsScanline = Normals.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				for (int32 X = X1; X <= X2; X++)
				{
					NormalsScanline[X] = NormalsScanline[X].GetSafeNormal();
				}
			}

			// Find brush centroid location
			FVector AveragePoint(0.0f, 0.0f, 0.0f);
			FVector AverageNormal(0.0f, 0.0f, 0.0f);
			float TotalWeight = 0.0f;
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				ValueType* SourceDataScanline = SourceDataArrayPtr->GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				FVector* NormalsScanline = Normals.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						AveragePoint += FVector((float)X * BrushValue, (float)Y * BrushValue, (float)SourceDataScanline[X] * BrushValue);

						FVector SampleNormal = NormalsScanline[X];
						AverageNormal += SampleNormal * BrushValue;

						TotalWeight += BrushValue;
					}
				}
			}

			if (TotalWeight > 0.0f)
			{
				AveragePoint /= TotalWeight;
				AverageNormal = AverageNormal.GetSafeNormal();
			}

			// Convert to world space
			FVector AverageLocation = ToWorld.TransformPosition(AveragePoint);
			FVector StrengthVector = ToWorld.TransformVector(FVector(0, 0, SculptStrength));

			// Brush pushes out in the normal direction
			FVector OffsetVector = AverageNormal * StrengthVector.Z;
			if (bInvert)
			{
				OffsetVector *= -1;
			}

			// World space brush plane
			BrushPlane = FPlane(AverageLocation + OffsetVector, AverageNormal);
		}

		// Apply the brush
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ApplyBrush);

			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
				ValueType* SourceDataScanline = SourceDataArrayPtr->GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const FIntPoint Key = FIntPoint(X, Y);
					const float BrushValue = BrushScanline[X];

					float SculptAmount = BrushValue * SculptStrength;
					ValueType& CurrentValue = DataScanline[X];
					const ValueType& SourceValue = SourceDataScanline[X];

					if (bUseClayBrush)
					{
						// Brush application starts from original world location at start of stroke
						FVector WorldLoc = ToWorld.TransformPosition(FVector(X, Y, SourceValue));

						// Calculate new location on the brush plane
						WorldLoc.Z = (BrushPlane.W - BrushPlane.X*WorldLoc.X - BrushPlane.Y*WorldLoc.Y) / BrushPlane.Z;

						// Painted amount lerps based on brush falloff.
						float PaintValue = FMath::Lerp<float>(SourceValue, static_cast<float>(FromWorld.TransformPosition(WorldLoc).Z), BrushValue);

						if (bInvert)
						{
							CurrentValue = ToolTarget::CacheClass::ClampValue(FMath::Min<int32>(FMath::RoundToInt(PaintValue), CurrentValue));
						}
						else
						{
							CurrentValue = ToolTarget::CacheClass::ClampValue(FMath::Max<int32>(FMath::RoundToInt(PaintValue), CurrentValue));
						}
					}
					else
					{
						if (bInvert)
						{
							CurrentValue = ToolTarget::CacheClass::ClampValue(FMath::Min<int32>(SourceValue - FMath::RoundToInt(SculptAmount), CurrentValue));
						}
						else
						{
							CurrentValue = ToolTarget::CacheClass::ClampValue(FMath::Max<int32>(SourceValue + FMath::RoundToInt(SculptAmount), CurrentValue));
						}
					}
				}
			}
		}

		Cache.SetCachedData(X1, Y1, X2, Y2, Data);
		Cache.Flush();

		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

class FLandscapeToolSculpt : public FLandscapeToolPaintBase<FHeightmapToolTarget, FLandscapeToolStrokeSculpt>
{
	using Super = FLandscapeToolPaintBase<FHeightmapToolTarget, FLandscapeToolStrokeSculpt>;

public:
	FLandscapeToolSculpt(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Sculpt"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Sculpt", "Sculpt"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Sculpt_Message", "Raise or lower the Landscape using the selected brush shape and falloff."); };

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const
	{
		const bool bInvert = bInIsModifierPressed;
		return bInvert ? ELandscapeTargetLayerActionType::Inverted : ELandscapeTargetLayerActionType::Normal;
	}
};

class FLandscapeToolErase : public FLandscapeToolPaintBase<FHeightmapToolTarget, FLandscapeToolStrokeErase>
{
	using Super = FLandscapeToolPaintBase<FHeightmapToolTarget, FLandscapeToolStrokeErase>;

public:
	FLandscapeToolErase(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Erase"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Erase", "Erase"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Erase_Message", "Erase the Landscape using the selected brush shape and falloff."); };

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		return ELandscapeTargetLayerActionType::Normal;
	}
};

// 
// FLandscapeToolSmooth
//

template<class ToolTarget>
class FLandscapeToolStrokeSmooth : public FLandscapeToolStrokePaintBase<ToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<ToolTarget>;

protected:
	using CacheType = Super::CacheType;
	using ValueType = Super::ValueType;

	TMap<ULandscapeLayerInfoObject*, FLandscapeEditLayerStackDataCache<ToolTarget>> PerTargetLayerEditLayerDataCache;

public:
	FLandscapeToolStrokeSmooth(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{

		for (auto& It : this->PerTargetLayerCache)
		{
			ULandscapeLayerInfoObject* LayerInfo = It.Key;
			CacheType& Cache = It.Value;
			PerTargetLayerEditLayerDataCache.Add(LayerInfo, FLandscapeEditLayerStackDataCache<ToolTarget>(InTarget.LandscapeInfo.Get(), Cache));
		}
	}

	virtual void SetEditLayer(const FGuid& EditLayerGUID) override
	{
		for (auto& It : PerTargetLayerEditLayerDataCache)
		{
			It.Value.SetCacheEditingLayer(EditLayerGUID);;
		}
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeSmooth_Apply);

		if (!this->LandscapeInfo) return;

		ALandscape* Landscape = this->LandscapeInfo->LandscapeActor.Get();
		const bool bTargetIsHeightmap = (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap);
		const bool bCombinedLayerOperation = bTargetIsHeightmap && UISettings->bCombinedLayersOperation && Landscape;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		CA_SUPPRESS(6326);
		if (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap)
		{
			// Note:  this is an inclusive bounds that may extend outside the available landscape coords on any side.
			X1 -= 1;
			Y1 -= 1;
			X2 += 1;
			Y2 += 1;
		}

		for (auto& It : this->PerTargetLayerEditLayerDataCache)
		{
			TArray<ValueType> Data;
			FLandscapeEditLayerStackDataCache<ToolTarget>& EditLayerDataCache = It.Value;
			EditLayerDataCache.Initialize(bCombinedLayerOperation);
			EditLayerDataCache.Read(X1, Y1, X2, Y2, Data);
			const TArray<ValueType> ReadData { Data };
		
			const float ToolStrength = FMath::Clamp<float>(this->GetStrength(UISettings) * Pressure, 0.0f, 1.0f);

			// Apply the brush
			if (UISettings->bDetailSmooth)
			{
				LowPassFilter<ValueType>(X1, Y1, X2, Y2, BrushInfo, Data, UISettings->DetailScale, ToolStrength);
			}
			else
			{
				const int32 FilterRadius = UISettings->SmoothFilterKernelSize;

				for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
				{
					const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
					ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

					for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
					{
						const float BrushValue =  BrushScanline[X] * ToolStrength;

						if (BrushValue > 0.0f)
						{
							// needs to be ~12 bits larger than ToolTarget::CacheClass::DataType (for max FilterRadius (31))
							// the editor is 64-bit native so just go the whole hog :)
							int64 FilterValue = 0;
							int32 FilterSamplingNumber = 0;

							const int32 XRadius = FMath::Min3<int32>(FilterRadius, X - BrushInfo.GetBounds().Min.X, BrushInfo.GetBounds().Max.X - X - 1);
							const int32 YRadius = FMath::Min3<int32>(FilterRadius, Y - BrushInfo.GetBounds().Min.Y, BrushInfo.GetBounds().Max.Y - Y - 1);

							const int32 SampleX1 = X - XRadius; checkSlow(SampleX1 >= BrushInfo.GetBounds().Min.X);
							const int32 SampleY1 = Y - YRadius; checkSlow(SampleY1 >= BrushInfo.GetBounds().Min.Y);
							const int32 SampleX2 = X + XRadius; checkSlow(SampleX2 < BrushInfo.GetBounds().Max.X);
							const int32 SampleY2 = Y + YRadius; checkSlow(SampleY2 < BrushInfo.GetBounds().Max.Y);
							for (int32 SampleY = SampleY1; SampleY <= SampleY2; SampleY++)
							{
								const float* SampleBrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, SampleY));
								const float* SampleBrushScanline2 = BrushInfo.GetDataPtr(FIntPoint(0, Y + (Y - SampleY)));
								const ValueType* SampleDataScanline = ReadData.GetData() + (SampleY - Y1) * (X2 - X1 + 1) + (0 - X1);

								for (int32 SampleX = SampleX1; SampleX <= SampleX2; SampleX++)
								{
									// constrain sample to within the brush, symmetrically to prevent flattening bug
									const float SampleBrushValue =
										FMath::Min(
											FMath::Min<float>(SampleBrushScanline[SampleX], SampleBrushScanline[X + (X - SampleX)]),
											FMath::Min<float>(SampleBrushScanline2[SampleX], SampleBrushScanline2[X + (X - SampleX)])
										);
									if (SampleBrushValue > 0.0f)
									{
										FilterValue += SampleDataScanline[SampleX];
										FilterSamplingNumber++;
									}
								}
							}

							if (FilterSamplingNumber > 0)
							{
								FilterValue /= FilterSamplingNumber;
							}

							DataScanline[X] = FMath::Lerp(DataScanline[X], static_cast<ValueType>(FilterValue), BrushValue);
						}
					}
				}
			}

			EditLayerDataCache.Write(X1, Y1, X2, Y2, Data, UISettings->PaintingRestriction);
		}

		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

template<class ToolTarget>
class FLandscapeToolSmooth : public FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeSmooth<ToolTarget>>
{
	using Super = FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeSmooth<ToolTarget>>;

public:
	FLandscapeToolSmooth(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Smooth"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Smooth", "Smooth"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Smooth_Message", "Smooth the Landscape within the brushes influence by averaging the Z position of the Landscape vertices."); };
};

//
// FLandscapeToolFlatten
//
template<class ToolTarget>
class FLandscapeToolStrokeFlatten : public FLandscapeToolStrokePaintBase<ToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<ToolTarget>;

protected:
	using CacheType = Super::CacheType;
	using ValueType = Super::ValueType;

	TMap<ULandscapeLayerInfoObject*, FLandscapeEditLayerStackDataCache<ToolTarget>> PerTargetLayerEditLayerDataCache;

	struct FFlattenValues
	{
		ValueType FlattenValue = static_cast<ValueType>(0);
		FVector FlattenNormal = FVector::Zero();
		float FlattenPlaneDist = 0.0f;
	};
	TMap<ULandscapeLayerInfoObject*, FFlattenValues> PerTargetLayerFlattenValues;

public:
	FLandscapeToolStrokeFlatten(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: FLandscapeToolStrokePaintBase<ToolTarget>(InEdMode, InViewportClient, InTarget)
	{
		if (InEdMode->UISettings->bUseFlattenTarget && (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap))
		{
			FTransform LocalToWorld = InTarget.LandscapeInfo->GetLandscapeProxy()->ActorToWorld();
			float Height = static_cast<float>((InEdMode->UISettings->GetFlattenTarget(/*bInReturnPreviewValueIfActive = */false) - LocalToWorld.GetTranslation().Z) / LocalToWorld.GetScale3D().Z);
			FFlattenValues& FlattenValues = PerTargetLayerFlattenValues.Add(/*(ULandscapeLayerInfoObject)*/nullptr);
			FlattenValues.FlattenValue = static_cast<ValueType>(LandscapeDataAccess::GetTexHeight(Height));
		}

		for (auto& It : this->PerTargetLayerCache)
		{
			ULandscapeLayerInfoObject* LayerInfo = It.Key;
			CacheType& Cache = It.Value;
			PerTargetLayerEditLayerDataCache.Add(LayerInfo, FLandscapeEditLayerStackDataCache<ToolTarget>(InTarget.LandscapeInfo.Get(), Cache));
		}
	}

	virtual void SetEditLayer(const FGuid& EditLayerGUID) override
	{
		for (auto& It : PerTargetLayerEditLayerDataCache)
		{
			It.Value.SetCacheEditingLayer(EditLayerGUID);;
		}
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
 	{
		if (!this->LandscapeInfo) return;

		ALandscape* Landscape = this->LandscapeInfo->LandscapeActor.Get();
		const bool bTargetIsHeightmap = (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap);
		const bool bCombinedLayerOperation = bTargetIsHeightmap && UISettings->bCombinedLayersOperation && Landscape;

		// Can't use slope if use Flatten Target because no normal is provided
		bool bUseSlopeFlatten = UISettings->bUseSlopeFlatten && !UISettings->bUseFlattenTarget;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		CA_SUPPRESS(6326);
		if (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap)
		{
			X1 -= 1;
			Y1 -= 1;
			X2 += 1;
			Y2 += 1;
		}

		for (auto& It : this->PerTargetLayerEditLayerDataCache)
		{
			ULandscapeLayerInfoObject* LayerInfo = It.Key;
			FLandscapeEditLayerStackDataCache<ToolTarget>& EditLayerDataCache = It.Value;
			CacheType& Cache = this->GetCacheForTargetLayer(LayerInfo);

			FFlattenValues* FlattenValues = PerTargetLayerFlattenValues.Find(LayerInfo);
			const bool bInitializedFlattenValue = (FlattenValues != nullptr);
			if (!bInitializedFlattenValue || (UISettings->bPickValuePerApply && bTargetIsHeightmap))
			{
				FlattenValues = &PerTargetLayerFlattenValues.FindOrAdd(LayerInfo);

				float FlattenX = static_cast<float>(InteractorPositions[0].Position.X);
				float FlattenY = static_cast<float>(InteractorPositions[0].Position.Y);
				int32 FlattenHeightX = FMath::FloorToInt(FlattenX);
				int32 FlattenHeightY = FMath::FloorToInt(FlattenY);

				float InterpolatedValue = 0.f;
				float V00 = 0.f;
				float V10 = 0.f;
				float V01 = 0.f;
				float V11 = 0.f;
				if (bCombinedLayerOperation)
				{
					// create a new accessor and point it read from the final runtime data
					typename ToolTarget::CacheClass::AccessorClass RuntimeDataAccessor(this->LandscapeInfo, LayerInfo);
					RuntimeDataAccessor.SetEditLayer(FGuid());

					ValueType P00, P10, P01, P11;
					// GetData will handle interpolation/extrapolation as needed if this samples outside valid landscape coords.
					FIntPoint SampleCoords(FlattenHeightX, FlattenHeightY);
					RuntimeDataAccessor.GetData(SampleCoords.X, SampleCoords.Y, SampleCoords.X, SampleCoords.Y, &P00);
					SampleCoords = { FlattenHeightX + 1, FlattenHeightY };
					RuntimeDataAccessor.GetData(SampleCoords.X, SampleCoords.Y, SampleCoords.X, SampleCoords.Y, &P10);
					SampleCoords = { FlattenHeightX, FlattenHeightY + 1 };
					RuntimeDataAccessor.GetData(SampleCoords.X, SampleCoords.Y, SampleCoords.X, SampleCoords.Y, &P01);
					SampleCoords = { FlattenHeightX + 1, FlattenHeightY + 1 };
					RuntimeDataAccessor.GetData(SampleCoords.X, SampleCoords.Y, SampleCoords.X, SampleCoords.Y, &P11);
					// Release Texture Mips that will be Locked by the next SynchronousUpdateComponentVisibilityForHeight (inside the EditLayerDataCache.Read call)
					RuntimeDataAccessor.Flush();
					V00 = P00;
					V10 = P10;
					V01 = P01;
					V11 = P11;
					InterpolatedValue = FMath::Lerp(FMath::Lerp(V00, V10, FlattenX - FlattenHeightX), FMath::Lerp(V01, V11, FlattenX - FlattenHeightX), FlattenY - FlattenHeightY);
				}
				else
				{
					Cache.CacheData(FlattenHeightX, FlattenHeightY, FlattenHeightX + 1, FlattenHeightY + 1);
					InterpolatedValue = Cache.GetValue(FlattenX, FlattenY);
				}
				FlattenValues->FlattenValue = static_cast<ValueType>(InterpolatedValue);

				if (bUseSlopeFlatten && bTargetIsHeightmap)
				{
					if (bCombinedLayerOperation)
					{
						// Can't rely on cache in this mode
						FVector Vert00 = FVector(0.0f, 0.0f, V00);
						FVector Vert01 = FVector(0.0f, 1.0f, V01);
						FVector Vert10 = FVector(1.0f, 0.0f, V10);
						FVector Vert11 = FVector(1.0f, 1.0f, V11);
						FVector FaceNormal1 = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
						FVector FaceNormal2 = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();
						FlattenValues->FlattenNormal = (FaceNormal1 + FaceNormal2).GetSafeNormal();
					}
					else
					{
						FlattenValues->FlattenNormal = Cache.GetNormal(FlattenHeightX, FlattenHeightY);
					}
					FlattenValues->FlattenPlaneDist = static_cast<float>(-(FlattenValues->FlattenNormal | FVector(FlattenX, FlattenY, InterpolatedValue)));
				}
			}

			TArray<ValueType> Data;
			EditLayerDataCache.Initialize(bCombinedLayerOperation);
			EditLayerDataCache.Read(X1, Y1, X2, Y2, Data);

			const float PaintStrength = this->GetStrength(UISettings);

			// Apply the brush
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						float Strength = FMath::Clamp<float>(BrushValue * PaintStrength * Pressure, 0.0f, 1.0f);

						if (!(bUseSlopeFlatten && bTargetIsHeightmap))
						{
							int32 Delta = DataScanline[X] - FlattenValues->FlattenValue;
							switch (UISettings->FlattenMode)
							{
							case ELandscapeToolFlattenMode::Terrace:
								if (bTargetIsHeightmap)
								{
									const FTransform& LocalToWorld = this->LandscapeInfo->GetLandscapeProxy()->ActorToWorld();
									float ScaleZ = static_cast<float>(LocalToWorld.GetScale3D().Z);
									float TranslateZ = static_cast<float>(LocalToWorld.GetTranslation().Z);
									float TerraceInterval = FMath::Clamp(UISettings->TerraceInterval, 1.0f, 32768.f);
									float Smoothness = UISettings->TerraceSmooth;
									float WorldHeight = LandscapeDataAccess::GetLocalHeight(DataScanline[X]);

									//move into world space
									WorldHeight = (WorldHeight * ScaleZ) + TranslateZ;
									float CurrentHeight = WorldHeight;

									//smoothing part
									float CurrentLevel = WorldHeight / TerraceInterval;
									Smoothness = 1.0f / FMath::Max(Smoothness, 0.0001f);
									float CurrentPhase = FMath::Frac(CurrentLevel);
									float Halfmask = FMath::Clamp(FMath::CeilToFloat(CurrentPhase - 0.5f), 0.0f, 1.0f);
									CurrentLevel = FMath::FloorToFloat(WorldHeight / TerraceInterval);
									float SCurve = FMath::Lerp(CurrentPhase, (1.0f - CurrentPhase), Halfmask) * 2.0f;
									SCurve = FMath::Pow(SCurve, Smoothness) * 0.5f;
									SCurve = FMath::Lerp(SCurve, 1.0f - SCurve, Halfmask) * TerraceInterval;
									WorldHeight = (CurrentLevel * TerraceInterval) + SCurve;
									//end of smoothing part

									float FinalHeight = FMath::Lerp(CurrentHeight, WorldHeight, Strength);
									FinalHeight = (FinalHeight - TranslateZ) / ScaleZ;
									DataScanline[X] = static_cast<ValueType>(LandscapeDataAccess::GetTexHeight(FinalHeight));
								}
								break;
							case ELandscapeToolFlattenMode::Interval:
								if (bTargetIsHeightmap)
								{
									const FTransform& LocalToWorld = this->LandscapeInfo->GetLandscapeProxy()->ActorToWorld();
									float ScaleZ = static_cast<float>(LocalToWorld.GetScale3D().Z);
									float TranslateZ = static_cast<float>(LocalToWorld.GetTranslation().Z);
									float TerraceInterval = UISettings->TerraceInterval;
									float TargetHeight = LandscapeDataAccess::GetLocalHeight(FlattenValues->FlattenValue);
									float CurrentHeight = LandscapeDataAccess::GetLocalHeight(DataScanline[X]);
														
									//move into world space
									TargetHeight = (TargetHeight * ScaleZ) + TranslateZ;
									CurrentHeight = (CurrentHeight * ScaleZ) + TranslateZ;

									TargetHeight = (FMath::RoundToFloat(TargetHeight / TerraceInterval)) * TerraceInterval;
									TargetHeight = FMath::Lerp(CurrentHeight, TargetHeight, BrushValue);
							
									//back to local space of landscape object
									TargetHeight = (TargetHeight - TranslateZ) / ScaleZ;
									DataScanline[X] = static_cast<ValueType>(LandscapeDataAccess::GetTexHeight(TargetHeight));

								}
								break;
							case ELandscapeToolFlattenMode::Raise:
								if (Delta < 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::CeilToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenValues->FlattenValue, Strength)));
								}
								break;
							case ELandscapeToolFlattenMode::Lower:
								if (Delta > 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::FloorToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenValues->FlattenValue, Strength)));
								}
								break;
							default:
							case ELandscapeToolFlattenMode::Both:
								if (Delta > 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::FloorToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenValues->FlattenValue, Strength)));
								}
								else
								{
									DataScanline[X] = static_cast<ValueType>(FMath::CeilToInt(FMath::Lerp((float)DataScanline[X], (float)FlattenValues->FlattenValue, Strength)));
								}
								break;
							}
						}
						else
						{
							ValueType DestValue = static_cast<ValueType>(-(FlattenValues->FlattenNormal.X * X + FlattenValues->FlattenNormal.Y * Y + FlattenValues->FlattenPlaneDist) / FlattenValues->FlattenNormal.Z);
							//float PlaneDist = FlattenValues->FlattenNormal | FVector(X, Y, HeightData(HeightDataIndex)) + FlattenValues->FlattenPlaneDist;
							float PlaneDist = static_cast<float>(DataScanline[X] - DestValue);
							DestValue = static_cast<ValueType>(DataScanline[X] - PlaneDist * Strength);
							switch (UISettings->FlattenMode)
							{
							case ELandscapeToolFlattenMode::Raise:
								if (PlaneDist < 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::CeilToInt(FMath::Lerp((float)DataScanline[X], (float)DestValue, Strength)));
								}
								break;
							case ELandscapeToolFlattenMode::Lower:
								if (PlaneDist > 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::FloorToInt(FMath::Lerp((float)DataScanline[X], (float)DestValue, Strength)));
								}
								break;
							default:
							case ELandscapeToolFlattenMode::Both:
								if (PlaneDist > 0)
								{
									DataScanline[X] = static_cast<ValueType>(FMath::FloorToInt(FMath::Lerp((float)DataScanline[X], (float)DestValue, Strength)));
								}
								else
								{
									DataScanline[X] = static_cast<ValueType>(FMath::CeilToInt(FMath::Lerp((float)DataScanline[X], (float)DestValue, Strength)));
								}
								break;
							}
						}
					}
				}
			}

			EditLayerDataCache.Write(X1, Y1, X2, Y2, Data, UISettings->PaintingRestriction);
		}

		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

template<class ToolTarget>
class FLandscapeToolFlatten : public FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeFlatten<ToolTarget>>
{
	using Super = FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeFlatten<ToolTarget>>;

protected:
	TObjectPtr<UStaticMesh> HeightmapFlattenPlaneMesh;
	TObjectPtr<UStaticMeshComponent> HeightmapFlattenPreviewComponent;
	bool bCanToolBeActivatedNextTick = false;
	bool bCanToolBeActivatedValue = false;

public:
	FLandscapeToolFlatten(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
		, HeightmapFlattenPlaneMesh(LoadObject<UStaticMesh>(nullptr, TEXT("/Engine/EditorLandscapeResources/FlattenPlaneMesh.FlattenPlaneMesh")))
	{
		check(HeightmapFlattenPlaneMesh);
	}

	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const override
	{ 
		if (this->EdMode->UISettings->bFlattenEyeDropperModeActivated)
		{
			OutCursor = EMouseCursor::EyeDropper;
			return true;
		}
		
		return false; 
	}

	virtual void SetCanToolBeActivated(bool Value) override
	{ 
		bCanToolBeActivatedNextTick = true;
		bCanToolBeActivatedValue = Value;
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(HeightmapFlattenPlaneMesh);
		Collector.AddReferencedObject(HeightmapFlattenPreviewComponent);
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Flatten"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Flatten", "Flatten"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Flatten_Message", "Raise and lower the Landscape to be the same Z height as the location from which you started using the tool."); };

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		if (bCanToolBeActivatedNextTick)
		{
			this->bCanToolBeActivated = bCanToolBeActivatedValue;
			bCanToolBeActivatedNextTick = false;
		}

		Super::Tick(ViewportClient, DeltaTime);

		const ALandscapeProxy* LandscapeProxy = this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? this->EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
		if ((LandscapeProxy != nullptr) && (HeightmapFlattenPreviewComponent != nullptr))
		{
			bool bShowGrid = this->EdMode->UISettings->bUseFlattenTarget && this->EdMode->UISettings->bShowFlattenTargetPreview;
			HeightmapFlattenPreviewComponent->SetVisibility(bShowGrid);

			// Always adjust the preview component's Z so that we react to the FlattenTarget changes in the details panel, which greatly helps eyeballing the adjustment : 
			const FTransform LocalToWorld = LandscapeProxy->ActorToWorld();
			FVector Origin = HeightmapFlattenPreviewComponent->GetRelativeLocation();
			Origin.Z = ConvertFlattenTargetToLocalZ(LandscapeProxy, /*bInReturnPreviewValueIfActive = */true);
			HeightmapFlattenPreviewComponent->SetRelativeLocation(Origin, /*bSweep = */false);
		}
	}

	virtual bool MouseEnter(FEditorViewportClient* InViewportClient, FViewport* Viewport, int32 InMouseX, int32 InMouseY) override
	{
		if (InViewportClient->IsLevelEditorClient() && this->EdMode->UISettings->bFlattenEyeDropperModeActivated)
		{
			this->EdMode->UISettings->bFlattenEyeDropperModeMousingOverViewport = true;
		}
		return true;
	}

	virtual bool MouseLeave(FEditorViewportClient* InViewportClient, FViewport* Viewport) override
	{
		if (InViewportClient->IsLevelEditorClient() && this->EdMode->UISettings->bFlattenEyeDropperModeActivated)
		{
			this->EdMode->UISettings->bFlattenEyeDropperModeMousingOverViewport = false;
		}
		return true;
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		bool bResult = Super::MouseMove(ViewportClient, Viewport, x, y);

		if (ViewportClient->IsLevelEditorClient())
		{
			const ALandscapeProxy* LandscapeProxy = this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? this->EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
			if ((LandscapeProxy != nullptr) && (HeightmapFlattenPreviewComponent != nullptr))
			{
				FVector MousePosition;
				if (this->EdMode->LandscapeMouseTrace((FEditorViewportClient*)ViewportClient, x, y, MousePosition))
				{
					const FTransform LocalToWorld = this->EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy()->ActorToWorld();
					FVector Origin;
					Origin.X = FMath::RoundToDouble(MousePosition.X);
					Origin.Y = FMath::RoundToDouble(MousePosition.Y);
					Origin.Z = ConvertFlattenTargetToLocalZ(LandscapeProxy, /*bInReturnPreviewValueIfActive = */true);
					HeightmapFlattenPreviewComponent->SetRelativeLocation(Origin, false);

					// Clamp the value to the height map
					uint16 TexHeight = LandscapeDataAccess::GetTexHeight(static_cast<float>(MousePosition.Z));
					float Height = LandscapeDataAccess::GetLocalHeight(TexHeight);

					// Convert the height back to world space
					this->EdMode->UISettings->FlattenEyeDropperModeDesiredTarget = static_cast<float>(Height * LocalToWorld.GetScale3D().Z + LocalToWorld.GetTranslation().Z);
				}
			}
		}

		return bResult;
	}

	virtual void EnterTool() override
	{
		Super::EnterTool();
		if (!this->EdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			return;
		}

		this->EdMode->UISettings->bFlattenEyeDropperModeMousingOverViewport = false;

		ALandscapeProxy* LandscapeProxy = this->EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? this->EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
		if ((LandscapeProxy != nullptr) && (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap))
		{
			HeightmapFlattenPreviewComponent = NewObject<UStaticMeshComponent>(LandscapeProxy, NAME_None, RF_Transient);
			HeightmapFlattenPreviewComponent->SetStaticMesh(HeightmapFlattenPlaneMesh);
			HeightmapFlattenPreviewComponent->SetCanEverAffectNavigation(false);
			HeightmapFlattenPreviewComponent->AttachToComponent(LandscapeProxy->GetRootComponent(), FAttachmentTransformRules::KeepRelativeTransform);
			HeightmapFlattenPreviewComponent->RegisterComponent();

			bool bShowGrid = this->EdMode->UISettings->bUseFlattenTarget && this->EdMode->UISettings->bShowFlattenTargetPreview;
			HeightmapFlattenPreviewComponent->SetVisibility(bShowGrid);

			// Try to set a sane initial location for the preview grid
			const FTransform LocalToWorld = LandscapeProxy->GetRootComponent()->GetComponentToWorld();
			FVector Origin = FVector::ZeroVector;
			Origin.Z = ConvertFlattenTargetToLocalZ(LandscapeProxy, /*bInReturnPreviewValueIfActive = */false);
			HeightmapFlattenPreviewComponent->SetRelativeLocation(Origin, false);
		}
	}

	virtual void ExitTool() override
	{
		Super::ExitTool();

		if (HeightmapFlattenPreviewComponent != nullptr)
		{
			HeightmapFlattenPreviewComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
			HeightmapFlattenPreviewComponent->DestroyComponent();
			HeightmapFlattenPreviewComponent = nullptr;
		}

		// Make sure the eye dropper tool is reset upon leaving the flatten tool :
		this->EdMode->UISettings->bFlattenEyeDropperModeActivated = false;
		this->EdMode->UISettings->bFlattenEyeDropperModeMousingOverViewport = false;
	}

private:
	double ConvertFlattenTargetToLocalZ(const ALandscapeProxy* InLandscapeProxy, bool bInReturnPreviewValueIfActive) const
	{
		const FTransform LocalToWorld = InLandscapeProxy->ActorToWorld();
		return (FMath::RoundToDouble((this->EdMode->UISettings->GetFlattenTarget(bInReturnPreviewValueIfActive) - LocalToWorld.GetTranslation().Z) / LocalToWorld.GetScale3D().Z * LANDSCAPE_INV_ZSCALE) - 0.1f) * LANDSCAPE_ZSCALE;
	}
};

// 
// FLandscapeToolNoise
//
template<class ToolTarget>
class FLandscapeToolStrokeNoise : public FLandscapeToolStrokePaintBase<ToolTarget>
{
	using Super = FLandscapeToolStrokePaintBase<ToolTarget>;

protected:
	using CacheType = Super::CacheType;
	using ValueType = Super::ValueType;

public:
	FLandscapeToolStrokeNoise(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: Super(InEdMode, InViewportClient, InTarget)
	{
	}

	void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolInteractorPosition>& InteractorPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeToolStrokeNoise_Apply);

		if (!this->LandscapeInfo) return;

		// Get list of verts to update
		FLandscapeBrushData BrushInfo = Brush->ApplyBrush(InteractorPositions);
		if (!BrushInfo)
		{
			return;
		}

		int32 X1, Y1, X2, Y2;
		BrushInfo.GetInclusiveBounds(X1, Y1, X2, Y2);

		// Tablet pressure
		float Pressure = ViewportClient->Viewport->IsPenActive() ? ViewportClient->Viewport->GetTabletPressure() : 1.0f;

		// expand the area by one vertex in each direction to ensure normals are calculated correctly
		CA_SUPPRESS(6326);
		if (ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap)
		{
			X1 -= 1;
			Y1 -= 1;
			X2 += 1;
			Y2 += 1;
		}

		for (auto& It : this->PerTargetLayerCache)
		{
			CacheType& Cache = It.Value;
			Cache.CacheData(X1, Y1, X2, Y2);
			TArray<ValueType> Data;
			Cache.GetCachedData(X1, Y1, X2, Y2, Data);

			float BrushSizeAdjust = 1.0f;
			CA_SUPPRESS(6326);
			if (ToolTarget::TargetType != ELandscapeToolTargetType::Weightmap && UISettings->GetCurrentToolBrushRadius() < UISettings->MaximumValueRadius)
			{
				BrushSizeAdjust = UISettings->GetCurrentToolBrushRadius() / UISettings->MaximumValueRadius;
			}

			CA_SUPPRESS(6326);
			bool bUseWeightTargetValue = UISettings->bUseWeightTargetValue && ToolTarget::TargetType == ELandscapeToolTargetType::Weightmap;

			const float PaintStrength = this->GetStrength(UISettings);

			// Apply the brush
			for (int32 Y = BrushInfo.GetBounds().Min.Y; Y < BrushInfo.GetBounds().Max.Y; Y++)
			{
				const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
				ValueType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);

				for (int32 X = BrushInfo.GetBounds().Min.X; X < BrushInfo.GetBounds().Max.X; X++)
				{
					const float BrushValue = BrushScanline[X];

					if (BrushValue > 0.0f)
					{
						float OriginalValue = DataScanline[X];
						if (bUseWeightTargetValue)
						{
							FNoiseParameter NoiseParam(0, UISettings->NoiseScale, 255.0f / 2.0f);
							float DestValue = NoiseModeConversion(ELandscapeToolNoiseMode::Add, NoiseParam.NoiseAmount, NoiseParam.Sample(X, Y)) * UISettings->WeightTargetValue;
							switch (UISettings->NoiseMode)
							{
							case ELandscapeToolNoiseMode::Add:
								if (OriginalValue >= DestValue)
								{
									continue;
								}
								break;
							case ELandscapeToolNoiseMode::Sub:
								DestValue += (1.0f - UISettings->WeightTargetValue) * NoiseParam.NoiseAmount;
								if (OriginalValue <= DestValue)
								{
									continue;
								}
								break;
							}
							DataScanline[X] = ToolTarget::CacheClass::ClampValue(FMath::RoundToInt(FMath::Lerp(OriginalValue, DestValue, BrushValue * UISettings->GetCurrentToolStrength() * Pressure)));
						}
						else
						{
							float TotalStrength = BrushValue * PaintStrength * Pressure * ToolTarget::StrengthMultiplier(this->LandscapeInfo, UISettings->GetCurrentToolBrushRadius());
							FNoiseParameter NoiseParam(0, UISettings->NoiseScale, TotalStrength * BrushSizeAdjust);
							float PaintAmount = NoiseModeConversion(UISettings->NoiseMode, NoiseParam.NoiseAmount, NoiseParam.Sample(X, Y));
							DataScanline[X] = static_cast<ValueType>(ToolTarget::CacheClass::ClampValue(static_cast<int32>(OriginalValue + PaintAmount)));
						}
					}
				}
			}

			Cache.SetCachedData(X1, Y1, X2, Y2, Data, UISettings->PaintingRestriction);
			Cache.Flush();
		}

		// Dirty any runtime virtual textures that our landscape components write to.
		this->LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(X1, Y1, X2, Y2);
	}
};

template<class ToolTarget>
class FLandscapeToolNoise : public FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeNoise<ToolTarget>>
{
	using Super = FLandscapeToolPaintBase<ToolTarget, FLandscapeToolStrokeNoise<ToolTarget>>;

public:
	FLandscapeToolNoise(FEdModeLandscape* InEdMode)
		: Super(InEdMode)
	{
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Noise"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Noise", "Noise"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Noise_Message", "The Noise tool applies a noise filter to the heightmap or layer weight. The strength determines the amount of noise."); };
};


//
// Toolset initialization
//
void FEdModeLandscape::InitializeTool_Paint()
{
	TUniquePtr<FLandscapeToolSculpt> Tool_Sculpt = MakeUnique<FLandscapeToolSculpt>(this);
	Tool_Sculpt->ValidBrushes.Add("BrushSet_Circle");
	Tool_Sculpt->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Sculpt->ValidBrushes.Add("BrushSet_Pattern");
	Tool_Sculpt->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_Sculpt));

	TUniquePtr<FLandscapeToolErase> Tool_Erase = MakeUnique<FLandscapeToolErase>(this);
	Tool_Erase->ValidBrushes.Add("BrushSet_Circle");
	LandscapeTools.Add(MoveTemp(Tool_Erase));

	TUniquePtr<FLandscapeToolPaint> Tool_Paint = MakeUnique<FLandscapeToolPaint>(this);
	Tool_Paint->ValidBrushes.Add("BrushSet_Circle");
	Tool_Paint->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Paint->ValidBrushes.Add("BrushSet_Pattern");
	Tool_Paint->ValidBrushes.Add("BrushSet_Component");
	LandscapeTools.Add(MoveTemp(Tool_Paint));
}

void FEdModeLandscape::InitializeTool_Smooth()
{
	TUniquePtr<FLandscapeToolSmooth<FHeightmapToolTarget>> Tool_Smooth_Heightmap = MakeUnique<FLandscapeToolSmooth<FHeightmapToolTarget>>(this);
	Tool_Smooth_Heightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Smooth_Heightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Smooth_Heightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Smooth_Heightmap));

	TUniquePtr<FLandscapeToolSmooth<FWeightmapToolTarget>> Tool_Smooth_Weightmap = MakeUnique<FLandscapeToolSmooth<FWeightmapToolTarget>>(this);
	Tool_Smooth_Weightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Smooth_Weightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Smooth_Weightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Smooth_Weightmap));
}

void FEdModeLandscape::InitializeTool_Flatten()
{
	TUniquePtr<FLandscapeToolFlatten<FHeightmapToolTarget>> Tool_Flatten_Heightmap = MakeUnique<FLandscapeToolFlatten<FHeightmapToolTarget>>(this);
	Tool_Flatten_Heightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Flatten_Heightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Flatten_Heightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Flatten_Heightmap));

	TUniquePtr<FLandscapeToolFlatten<FWeightmapToolTarget>> Tool_Flatten_Weightmap = MakeUnique<FLandscapeToolFlatten<FWeightmapToolTarget>>(this);
	Tool_Flatten_Weightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Flatten_Weightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Flatten_Weightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Flatten_Weightmap));
}

void FEdModeLandscape::InitializeTool_Noise()
{
	TUniquePtr<FLandscapeToolNoise<FHeightmapToolTarget>> Tool_Noise_Heightmap = MakeUnique<FLandscapeToolNoise<FHeightmapToolTarget>>(this);
	Tool_Noise_Heightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Noise_Heightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Noise_Heightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Noise_Heightmap));

	TUniquePtr<FLandscapeToolNoise<FWeightmapToolTarget>> Tool_Noise_Weightmap = MakeUnique<FLandscapeToolNoise<FWeightmapToolTarget>>(this);
	Tool_Noise_Weightmap->ValidBrushes.Add("BrushSet_Circle");
	Tool_Noise_Weightmap->ValidBrushes.Add("BrushSet_Alpha");
	Tool_Noise_Weightmap->ValidBrushes.Add("BrushSet_Pattern");
	LandscapeTools.Add(MoveTemp(Tool_Noise_Weightmap));
}

#undef LOCTEXT_NAMESPACE