// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "HitProxies.h"
#include "AI/NavigationSystemBase.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/Texture2D.h"
#include "UnrealWidgetFwd.h"
#include "EditorModeManager.h"
#include "UnrealEdGlobals.h"
#include "EditorViewportClient.h"
#include "LandscapeToolInterface.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "ScopedTransaction.h"
#include "LandscapeEdit.h"
#include "LandscapeRender.h"
#include "LandscapeDataAccess.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "Raster.h"
#include "Landscape.h"
#include "Misc/MessageDialog.h"
#include "LandscapeEdModeTools.h"
#include "TextureResource.h"

#define LOCTEXT_NAMESPACE "Landscape"

class FLandscapeRampToolHeightRasterPolicy
{
public:
	// X = Side Falloff Alpha, Y = Height
	typedef FVector2D InterpolantType;

	/** Initialization constructor. */
	FLandscapeRampToolHeightRasterPolicy(TArray<uint16>& InData, int32 InMinX, int32 InMinY, int32 InMaxX, int32 InMaxY, bool InbRaiseTerrain, bool InbLowerTerrain) :
		Data(InData),
		MinX(InMinX),
		MinY(InMinY),
		MaxX(InMaxX),
		MaxY(InMaxY),
		bRaiseTerrain(InbRaiseTerrain),
		bLowerTerrain(InbLowerTerrain)
	{
	}

protected:

	// FTriangleRasterizer policy interface.

	int32 GetMinX() const { return MinX; }
	int32 GetMaxX() const { return MaxX; }
	int32 GetMinY() const { return MinY; }
	int32 GetMaxY() const { return MaxY; }

	void ProcessPixel(int32 X, int32 Y, const InterpolantType& Interpolant, bool BackFacing)
	{
		const float CosInterpX = static_cast<float>(Interpolant.X >= 1 ? 1 : 0.5 - 0.5 * FMath::Cos(Interpolant.X * PI));
		const float Alpha = CosInterpX;
		uint16& Dest = Data[(Y - MinY)*(1 + MaxX - MinX) + X - MinX];
		float Value = FMath::Lerp((float)Dest, (float)Interpolant.Y, Alpha);
		uint16 DValue = static_cast<uint16>(FMath::Clamp<float>(Value, 0, LandscapeDataAccess::MaxValue));
		if ((bRaiseTerrain && DValue > Dest) ||
			(bLowerTerrain && DValue < Dest))
		{
			Dest = DValue;
		}
	}

private:
	TArray<uint16>& Data;
	int32 MinX, MinY, MaxX, MaxY;
	uint32 bRaiseTerrain : 1, bLowerTerrain : 1;
};

class HLandscapeRampToolPointHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY();

	int8 Point;

	HLandscapeRampToolPointHitProxy(int8 InPoint) :
		HHitProxy(HPP_Foreground),
		Point(InPoint)
	{
	}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}
};

IMPLEMENT_HIT_PROXY(HLandscapeRampToolPointHitProxy, HHitProxy)

class FLandscapeToolRamp : public FLandscapeTool
{
protected:
	FEdModeLandscape* EdMode;
	TObjectPtr<UTexture2D> SpriteTexture;
	FVector Points[2];
	int8 NumPoints;
	int8 SelectedPoint;
	bool bMovingPoint;

public:
	FLandscapeToolRamp(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
		, SpriteTexture(LoadObject<UTexture2D>(NULL, TEXT("/Engine/EditorResources/S_Terrain.S_Terrain")))
		, NumPoints(0)
		, SelectedPoint(INDEX_NONE)
		, bMovingPoint(false)
	{
		check(SpriteTexture);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(SpriteTexture);
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Ramp"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Ramp", "Ramp"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Ramp_Message", "Create a ramp between two specified points, adding falloffs according to the settings. Note that this tool cannot use any brushes."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	virtual ELandscapeToolTargetType GetSupportedTargetType() const override
	{
		return ELandscapeToolTargetType::Heightmap;
	}

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		return ELandscapeTargetLayerActionType::Normal;
	}

	virtual void EnterTool() override
	{
		NumPoints = 0;
		SelectedPoint = INDEX_NONE;
		GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) override
	{
		if (NumPoints < 2)
		{
			Points[NumPoints] = InHitLocation;
			SelectedPoint = NumPoints;
			NumPoints++;
			bMovingPoint = true;
			GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
		}
		else
		{
			if (SelectedPoint != INDEX_NONE)
			{
				Points[SelectedPoint] = InHitLocation;
				bMovingPoint = true;
				GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
			}
		}

		GUnrealEd->RedrawLevelEditingViewports();

		return true;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		bMovingPoint = false;
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		if (bMovingPoint)
		{
			if (!Viewport->KeyState(EKeys::LeftMouseButton))
			{
				bMovingPoint = false;
				return false;
			}

			FVector HitLocation;
			if (EdMode->LandscapeMouseTrace(ViewportClient, x, y, HitLocation))
			{
				if (NumPoints == 1)
				{
					SelectedPoint = NumPoints;
					NumPoints++;
				}

				Points[SelectedPoint] = HitLocation;

				GUnrealEd->RedrawLevelEditingViewports();
			}

			return true;
		}

		return false;
	}

	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override
	{
		if (InKey == EKeys::Enter && InEvent == IE_Pressed)
		{
			if (CanApplyRamp())
			{
				ApplyRamp();
			}
		}

		if (InKey == EKeys::Escape && InEvent == IE_Pressed)
		{
			ResetRamp();
		}

		// Handle clicking on points to select them and drag them around
		if (InKey == EKeys::LeftMouseButton)
		{
			if (InEvent == IE_Pressed)
			{
				if (!InViewport->KeyState(EKeys::MiddleMouseButton) && !InViewport->KeyState(EKeys::RightMouseButton) && !IsAltDown(InViewport) && InViewportClient->GetCurrentWidgetAxis() == EAxisList::None)
				{
					HHitProxy* HitProxy = InViewport->GetHitProxy(InViewport->GetMouseX(), InViewport->GetMouseY());
					if (HitProxy && HitProxy->IsA(HLandscapeRampToolPointHitProxy::StaticGetType()))
					{
						HLandscapeRampToolPointHitProxy* PointHitProxy = (HLandscapeRampToolPointHitProxy*)HitProxy;
						SelectedPoint = PointHitProxy->Point;
						GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Translate);
						GUnrealEd->RedrawLevelEditingViewports();

						bMovingPoint = true;
						return true;
					}
				}
				return false;
			}
			else if (InEvent == IE_Released)
			{
				bMovingPoint = false;
				return false;
			}
		}

		if (InKey == EKeys::End && InEvent == IE_Pressed)
		{
			if (SelectedPoint != INDEX_NONE)
			{
				const int32 MinX = FMath::FloorToInt32(Points[SelectedPoint].X);
				const int32 MinY = FMath::FloorToInt32(Points[SelectedPoint].Y);
				const int32 MaxX = MinX + 1;
				const int32 MaxY = MinY + 1;

				FLandscapeEditDataInterface LandscapeEdit(EdMode->CurrentToolTarget.LandscapeInfo.Get());

				TArray<uint16> Data;
				Data.AddZeroed(4);

				int32 ValidMinX = MinX;
				int32 ValidMinY = MinY;
				int32 ValidMaxX = MaxX;
				int32 ValidMaxY = MaxY;
				LandscapeEdit.GetHeightData(ValidMinX, ValidMinY, ValidMaxX, ValidMaxY, Data.GetData(), 0);

				if (ValidMaxX - ValidMinX != 1 && ValidMaxY - ValidMinY != 1)
				{
					// If we didn't read 4 values then we're partly off the edge of the landscape
					return true;
				}

				checkSlow(ValidMinX == MinX);
				checkSlow(ValidMinY == MinY);
				checkSlow(ValidMaxX == MaxX);
				checkSlow(ValidMaxY == MaxY);

				Points[SelectedPoint].Z = (FMath::BiLerp<float>(Data[0], Data[1], Data[2], Data[3], FMath::Frac(Points[SelectedPoint].X), FMath::Frac(Points[SelectedPoint].Y)) - LandscapeDataAccess::MidValue) * LANDSCAPE_ZSCALE;

				return true;
			}
		}

		// Change Ramp Width
		if ((InEvent == IE_Pressed || InEvent == IE_Repeat) && (InKey == EKeys::LeftBracket || InKey == EKeys::RightBracket))
		{
			const float OldValue = EdMode->UISettings->RampWidth;
			const float SliderMin = 0.0f;
			const float SliderMax = 8192.0f;
			const float Diff = 0.05f;

			float NewValue;
			if (InKey == EKeys::LeftBracket)
			{
				NewValue = OldValue - OldValue * Diff;
				NewValue = FMath::Min(NewValue, OldValue - 1.0f);
			}
			else
			{
				NewValue = OldValue + OldValue * Diff;
				NewValue = FMath::Max(NewValue, OldValue + 1.0f);
			}

			NewValue = FMath::RoundToFloat(FMath::Clamp(NewValue, SliderMin, SliderMax));

			EdMode->UISettings->RampWidth = NewValue;

			return true;
		}

		return false;
	}

	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override
	{
		if (SelectedPoint != INDEX_NONE && InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
		{
			if (const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr)
			{
				const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

				Points[SelectedPoint] += LandscapeToWorld.InverseTransformVector(InDrag);
			}

			return true;
		}

		return false;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		// The editor can try to render the tool before the UpdateLandscapeEditorData command runs and the landscape editor realizes that the landscape has been hidden/deleted
		const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
		if ((LandscapeProxy != nullptr) && (NumPoints > 0))
		{
			const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

			const FLinearColor SelectedSpriteColor = FLinearColor::White + (GEngine->GetSelectedMaterialColor() * GEngine->SelectionHighlightIntensity * 10);

			FVector WorldPoints[2];
			for (int32 i = 0; i < NumPoints; i++)
			{
				WorldPoints[i] = LandscapeToWorld.TransformPosition(Points[i]);
			}

			float SpriteScale = EdMode->UISettings->RampWidth / 4;
			if (NumPoints > 1)
			{
				SpriteScale = FMath::Min(SpriteScale, static_cast<float>((WorldPoints[1] - WorldPoints[0]).Size() / 2));
			}
			SpriteScale = FMath::Clamp<float>(SpriteScale, 10, 500);

			for (int8 i = 0; i < NumPoints; i++)
			{
				const FLinearColor SpriteColor = (i == SelectedPoint) ? SelectedSpriteColor : FLinearColor::White;

				PDI->SetHitProxy(new HLandscapeRampToolPointHitProxy(i));
				PDI->DrawSprite(WorldPoints[i],
					SpriteScale,
					SpriteScale,
					SpriteTexture->GetResource(),
					SpriteColor,
					SDPG_Foreground,
					/*U = */0.f, /*UL = */0.f, 
					/*V = */0.f, /*VL = */0.f,
					SE_BLEND_Masked);
			}
			PDI->SetHitProxy(NULL);

			if (NumPoints == 2)
			{
				const FVector Side = FVector::CrossProduct(Points[1] - Points[0], FVector(0, 0, 1)).GetSafeNormal2D();
				FVector InnerSide = Side * (EdMode->UISettings->RampWidth * 0.5f * (1 - EdMode->UISettings->RampSideFalloff));
				FVector OuterSide = Side * (EdMode->UISettings->RampWidth * 0.5f);
				InnerSide = LandscapeToWorld.TransformVectorNoScale(InnerSide);
				OuterSide = LandscapeToWorld.TransformVectorNoScale(OuterSide);

				FVector InnerVerts[2][2];
				InnerVerts[0][0] = WorldPoints[0] - InnerSide;
				InnerVerts[0][1] = WorldPoints[0] + InnerSide;
				InnerVerts[1][0] = WorldPoints[1] - InnerSide;
				InnerVerts[1][1] = WorldPoints[1] + InnerSide;

				FVector OuterVerts[2][2];
				OuterVerts[0][0] = WorldPoints[0] - OuterSide;
				OuterVerts[0][1] = WorldPoints[0] + OuterSide;
				OuterVerts[1][0] = WorldPoints[1] - OuterSide;
				OuterVerts[1][1] = WorldPoints[1] + OuterSide;

				// Left
				DrawDashedLine(PDI, OuterVerts[0][0], OuterVerts[1][0], FColor::White, 50, SDPG_Foreground);

				// Center
				DrawDashedLine(PDI, InnerVerts[0][0], InnerVerts[0][1], FColor::White, 50, SDPG_Foreground);
				PDI->DrawLine(InnerVerts[0][0], InnerVerts[0][1], FLinearColor::White, SDPG_World);
				DrawDashedLine(PDI, InnerVerts[0][0], InnerVerts[1][0], FColor::White, 50, SDPG_Foreground);
				PDI->DrawLine(InnerVerts[0][0], InnerVerts[1][0], FLinearColor::White, SDPG_World);
				DrawDashedLine(PDI, InnerVerts[0][1], InnerVerts[1][1], FColor::White, 50, SDPG_Foreground);
				PDI->DrawLine(InnerVerts[0][1], InnerVerts[1][1], FLinearColor::White, SDPG_World);
				DrawDashedLine(PDI, InnerVerts[1][0], InnerVerts[1][1], FColor::White, 50, SDPG_Foreground);
				PDI->DrawLine(InnerVerts[1][0], InnerVerts[1][1], FLinearColor::White, SDPG_World);

				// Right
				DrawDashedLine(PDI, OuterVerts[0][1], OuterVerts[1][1], FColor::White, 50, SDPG_Foreground);
			}
		}
	}

	virtual bool OverrideSelection() const override
	{
		return true;
	}

	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override
	{
		// Only filter selection not deselection
		if (bInSelection)
		{
			return false;
		}

		return true;
	}

	virtual bool UsesTransformWidget() const override
	{
		if (SelectedPoint != INDEX_NONE)
		{
			// The editor can try to render the transform widget before the landscape editor ticks and realizes that the landscape has been hidden/deleted
			return EdMode->CurrentToolTarget.LandscapeInfo.IsValid() && (EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() != nullptr);
		}

		return false;
	}

	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode CheckMode) const override
	{
		if (SelectedPoint != INDEX_NONE)
		{
			if (CheckMode == UE::Widget::WM_Translate)
			{
				return EAxisList::XYZ;
			}
			else
			{
				return EAxisList::None;
			}
		}

		return EAxisList::None;
	}

	virtual FVector GetWidgetLocation() const override
	{
		if (SelectedPoint != INDEX_NONE)
		{
			const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy();
			if (LandscapeProxy)
			{
				const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();
				return LandscapeToWorld.TransformPosition(Points[SelectedPoint]);
			}
		}

		return FVector::ZeroVector;
	}

	virtual FMatrix GetWidgetRotation() const override
	{
		if (SelectedPoint != INDEX_NONE)
		{
			const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy();
			if (LandscapeProxy)
			{
				const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();
				return FQuatRotationTranslationMatrix(LandscapeToWorld.GetRotation(), FVector::ZeroVector);
			}
		}

		return FMatrix::Identity;
	}

	virtual void ApplyRamp()
	{
		FText Reason;
		if (!EdMode->CanEditLayer(&Reason))
		{
			FMessageDialog::Open(EAppMsgType::Ok, Reason);
			return;
		}

		if (!CanApplyRamp())
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("RampApplyError", "Cannot apply ramp, make sure 2 control points are used."));
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("Ramp_Apply", "Landscape Editing: Add ramp"));
		ALandscape* Landscape = EdMode->GetLandscape();
		FGuid EditLayerGUID = EdMode->GetCurrentLayerGuid();

		const ULandscapeInfo* LandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get();
		const ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		const FTransform LandscapeToWorld = LandscapeProxy->LandscapeActorToWorld();

		const FVector2D Side = FVector2D(FVector::CrossProduct(Points[1] - Points[0], FVector(0,0,1))).GetSafeNormal();
		const FVector2D InnerSide = Side * (EdMode->UISettings->RampWidth * 0.5f * (1 - EdMode->UISettings->RampSideFalloff)) / LandscapeToWorld.GetScale3D().X;
		const FVector2D OuterSide = Side * (EdMode->UISettings->RampWidth * 0.5f) / LandscapeToWorld.GetScale3D().X;

		FVector2D InnerVerts[2][2];
		InnerVerts[0][0] = FVector2D(Points[0]) - InnerSide;
		InnerVerts[0][1] = FVector2D(Points[0]) + InnerSide;
		InnerVerts[1][0] = FVector2D(Points[1]) - InnerSide;
		InnerVerts[1][1] = FVector2D(Points[1]) + InnerSide;

		FVector2D OuterVerts[2][2];
		OuterVerts[0][0] = FVector2D(Points[0]) - OuterSide;
		OuterVerts[0][1] = FVector2D(Points[0]) + OuterSide;
		OuterVerts[1][0] = FVector2D(Points[1]) - OuterSide;
		OuterVerts[1][1] = FVector2D(Points[1]) + OuterSide;

		const double Heights[2] = {
			Points[0].Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue,
			Points[1].Z * LANDSCAPE_INV_ZSCALE + LandscapeDataAccess::MidValue
		};

		int32 MinX = FMath::CeilToInt32(FMath::Min(FMath::Min(OuterVerts[0][0].X, OuterVerts[0][1].X), FMath::Min(OuterVerts[1][0].X, OuterVerts[1][1].X))) - 1; // +/- 1 to make sure we have enough data for calculating correct normals
		int32 MinY = FMath::CeilToInt32(FMath::Min(FMath::Min(OuterVerts[0][0].Y, OuterVerts[0][1].Y), FMath::Min(OuterVerts[1][0].Y, OuterVerts[1][1].Y))) - 1;
		int32 MaxX = FMath::FloorToInt32(FMath::Max(FMath::Max(OuterVerts[0][0].X, OuterVerts[0][1].X), FMath::Max(OuterVerts[1][0].X, OuterVerts[1][1].X))) + 1;
		int32 MaxY = FMath::FloorToInt32(FMath::Max(FMath::Max(OuterVerts[0][0].Y, OuterVerts[0][1].Y), FMath::Max(OuterVerts[1][0].Y, OuterVerts[1][1].Y))) + 1;

		// I'd dearly love to use FIntRect in this code, but Landscape works with "Inclusive Max" and FIntRect is "Exclusive Max"
		int32 LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY;
		if (!LandscapeInfo->GetLandscapeExtent(LandscapeMinX, LandscapeMinY, LandscapeMaxX, LandscapeMaxY))
		{
			return;
		}

		MinX = FMath::Max(MinX, LandscapeMinX);
		MinY = FMath::Max(MinY, LandscapeMinY);
		MaxX = FMath::Min(MaxX, LandscapeMaxX);
		MaxY = FMath::Min(MaxY, LandscapeMaxY);

		if (MinX > MaxX || MinY > MaxY)
		{
			// The bounds don't intersect any data, so we skip applying the ramp entirely
			return;
		}

		// construct the caches, and set them to work in the EditLayer
		FLandscapeEditDataInterface LandscapeEdit(EdMode->CurrentToolTarget.LandscapeInfo.Get(), EditLayerGUID);
		FLandscapeHeightCache HeightCache(EdMode->CurrentToolTarget.LandscapeInfo.Get());
		FLandscapeEditLayerStackDataCache<FHeightmapToolTarget> EditLayerHeightDataCache(EdMode->CurrentToolTarget.LandscapeInfo.Get(), HeightCache);
		EditLayerHeightDataCache.SetCacheEditingLayer(EditLayerGUID);

		const bool bCombinedLayerOperation = EdMode->UISettings->bCombinedLayersOperation && Landscape;
		EditLayerHeightDataCache.Initialize(bCombinedLayerOperation);

		// Heights raster
		bool bRaiseTerrain = true; //EdMode->UISettings->Ramp_bRaiseTerrain;
		bool bLowerTerrain = true; //EdMode->UISettings->Ramp_bLowerTerrain;
		if (bRaiseTerrain || bLowerTerrain)
		{
			TArray<uint16> Data;
			Data.AddZeroed((1 + MaxY - MinY) * (1 + MaxX - MinX));

			int32 ValidMinX = MinX;
			int32 ValidMinY = MinY;
			int32 ValidMaxX = MaxX;
			int32 ValidMaxY = MaxY;
			EditLayerHeightDataCache.Read(ValidMinX, ValidMinY, ValidMaxX, ValidMaxY, Data);

			if (ValidMinX > ValidMaxX || ValidMinY > ValidMaxY)
			{
				// The bounds don't intersect any data, so we skip applying the ramp entirely
				return;
			}

			FLandscapeEditDataInterface::ShrinkData(Data, MinX, MinY, MaxX, MaxY, ValidMinX, ValidMinY, ValidMaxX, ValidMaxY);

			MinX = ValidMinX;
			MinY = ValidMinY;
			MaxX = ValidMaxX;
			MaxY = ValidMaxY;

			FTriangleRasterizer<FLandscapeRampToolHeightRasterPolicy> Rasterizer(
				FLandscapeRampToolHeightRasterPolicy(Data, MinX, MinY, MaxX, MaxY, bRaiseTerrain, bLowerTerrain));

			// Left
			Rasterizer.DrawTriangle(FVector2D(0, Heights[0]), FVector2D(1, Heights[0]), FVector2D(0, Heights[1]), OuterVerts[0][0], InnerVerts[0][0], OuterVerts[1][0], false);
			Rasterizer.DrawTriangle(FVector2D(1, Heights[0]), FVector2D(0, Heights[1]), FVector2D(1, Heights[1]), InnerVerts[0][0], OuterVerts[1][0], InnerVerts[1][0], false);

			// Center
			Rasterizer.DrawTriangle(FVector2D(1, Heights[0]), FVector2D(1, Heights[0]), FVector2D(1, Heights[1]), InnerVerts[0][0], InnerVerts[0][1], InnerVerts[1][0], false);
			Rasterizer.DrawTriangle(FVector2D(1, Heights[0]), FVector2D(1, Heights[1]), FVector2D(1, Heights[1]), InnerVerts[0][1], InnerVerts[1][0], InnerVerts[1][1], false);

			// Right
			Rasterizer.DrawTriangle(FVector2D(1, Heights[0]), FVector2D(0, Heights[0]), FVector2D(1, Heights[1]), InnerVerts[0][1], OuterVerts[0][1], InnerVerts[1][1], false);
			Rasterizer.DrawTriangle(FVector2D(0, Heights[0]), FVector2D(1, Heights[1]), FVector2D(0, Heights[1]), OuterVerts[0][1], InnerVerts[1][1], OuterVerts[1][1], false);

			EditLayerHeightDataCache.Write(MinX, MinY, MaxX, MaxY, Data);

			// Dirty any runtime virtual textures that our landscape components write to.
			LandscapeInfo->DirtyRuntimeVirtualTextureForLandscapeArea(MinX, MinY, MaxX, MaxY);
		}
 		if (Landscape)
 		{
 			Landscape->RequestLayersContentUpdate(ELandscapeLayerUpdateMode::Update_Heightmap_All);
 		}
	}

	bool CanApplyRamp()
	{
		return EdMode->CanEditLayer() && (NumPoints == 2);
	}

	void ResetRamp()
	{
		NumPoints = 0;
		SelectedPoint = INDEX_NONE;
	}
};

void FEdModeLandscape::ApplyRampTool()
{
	if (CurrentTool->GetToolName() == FName("Ramp"))
	{
		FLandscapeToolRamp* RampTool = (FLandscapeToolRamp*)CurrentTool;
		RampTool->ApplyRamp();
		GEditor->RedrawLevelEditingViewports();
	}
}

bool FEdModeLandscape::CanApplyRampTool()
{
	if (CurrentTool->GetToolName() == FName("Ramp"))
	{
		FLandscapeToolRamp* RampTool = (FLandscapeToolRamp*)CurrentTool;

		return RampTool->CanApplyRamp();
	}
	return false;
}

void FEdModeLandscape::ResetRampTool()
{
	if (CurrentTool->GetToolName() == FName("Ramp"))
	{
		FLandscapeToolRamp* RampTool = (FLandscapeToolRamp*)CurrentTool;
		RampTool->ResetRamp();
		GEditor->RedrawLevelEditingViewports();
	}
}

//
// Toolset initialization
//
void FEdModeLandscape::InitializeTool_Ramp()
{
	auto Tool_Ramp = MakeUnique<FLandscapeToolRamp>(this);
	Tool_Ramp->ValidBrushes.Add("BrushSet_Dummy");
	LandscapeTools.Add(MoveTemp(Tool_Ramp));
}

#undef LOCTEXT_NAMESPACE
