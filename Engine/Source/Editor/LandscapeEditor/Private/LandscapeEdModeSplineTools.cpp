// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "EngineDefines.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "InputCoreTypes.h"
#include "Engine/EngineTypes.h"
#include "GameFramework/Actor.h"
#include "CollisionQueryParams.h"
#include "Components/MeshComponent.h"
#include "Exporters/Exporter.h"
#include "Editor/UnrealEdEngine.h"
#include "UObject/PropertyPortFlags.h"
#include "EngineUtils.h"
#include "EditorUndoClient.h"
#include "UnrealWidgetFwd.h"
#include "EditorModeManager.h"
#include "UnrealEdGlobals.h"
#include "EditorViewportClient.h"
#include "Framework/Application/SlateApplication.h"
#include "LandscapeToolInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeSplineActor.h"
#include "ILandscapeSplineInterface.h"
#include "LandscapeEdMode.h"
#include "ScopedTransaction.h"
#include "LandscapeRender.h"
#include "LandscapeSplineProxies.h"
#include "PropertyEditorModule.h"
#include "LandscapeSplineImportExport.h"
#include "LandscapeSplinesComponent.h"
#include "LandscapeSplineSegment.h"
#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplineSelection.h"
#include "ControlPointMeshComponent.h"
#include "Algo/AnyOf.h"
#include "Algo/Copy.h"
#include "GizmoEdModeInterface.h"
#include "HAL/PlatformApplicationMisc.h"
#include "UnrealExporter.h"
#include "UnrealWidget.h"


#define LOCTEXT_NAMESPACE "Landscape"

//
// FLandscapeToolSplines
//
class FLandscapeToolSplines : public FLandscapeTool, public FEditorUndoClient
{
public:
	FLandscapeToolSplines(FEdModeLandscape* InEdMode)
		: EdMode(InEdMode)
		, LandscapeInfo(nullptr)
		, SplineSelection(nullptr)
		, DraggingTangent_Segment(nullptr)
		, DraggingTangent_Length(0.0f)
		, DraggingTangent_CacheCoordSpace(ECoordSystem::COORD_None)
		, DraggingTangent_End(false)
		, bMovingControlPoint(false)
		, bAutoRotateOnJoin(true)
		, bAlwaysRotateForward(false)
		, bAutoChangeConnectionsOnMove(true)
		, bDeleteLooseEnds(false)
		, bCopyMeshToNewControlPoint(false)
		, bAllowDuplication(true)
		, bDuplicatingControlPoint(false)
		, bUpdatingAddSegment(false)
		, DuplicateDelay(0)
		, DuplicateDelayAccumulatedDrag(FVector::ZeroVector)
		, DuplicateCachedRotation(FRotator::ZeroRotator)
		, DuplicateCacheSplitSegmentParam(0.0f)
		, DuplicateCacheSplitSegmentTangentLenStart(0.0f)
		, DuplicateCacheSplitSegmentTangentLenEnd(0.0f)
		, DuplicateCacheSplitSegmentTangentLen(0.0f)
	{
		// Register to update when an undo/redo operation has been called to update our list of actors
		GEditor->RegisterForUndo(this);
	}

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(LandscapeInfo);
		Collector.AddReferencedObject(SplineSelection);
		Collector.AddReferencedObject(DraggingTangent_Segment);
	}

	~FLandscapeToolSplines()
	{
		// GEditor is invalid at shutdown as the object system is unloaded before the landscape module.
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			// Remove undo delegate
			GEditor->UnregisterForUndo(this);
		}
	}

	virtual const TCHAR* GetToolName() const override { return TEXT("Splines"); }
	virtual FText GetDisplayName() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Splines", "Splines"); };
	virtual FText GetDisplayMessage() const override { return NSLOCTEXT("UnrealEd", "LandscapeMode_Splines_Message", "Create a Landscape Spline to carve your landscape, modify blendmasks and deform meshes into roads and other linear features.  Spline mesh settings can be found in the details panel when you have  segments selected."); };

	virtual void SetEditRenderType() override { GLandscapeEditRenderMode = ELandscapeEditRenderMode::None | (GLandscapeEditRenderMode & ELandscapeEditRenderMode::BitMaskForMask); }
	virtual bool SupportsMask() const override { return false; }

	void AddSegment(ULandscapeSplineControlPoint* Start, ULandscapeSplineControlPoint* End, bool bAutoRotateStart, bool bAutoRotateEnd)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AddSegment", "Add Landscape Spline Segment"));

		if (Start == End)
		{
			//UE_LOG( TEXT("Can't join spline control point to itself.") );
			return;
		}

		if (Start->GetOuterULandscapeSplinesComponent() != End->GetOuterULandscapeSplinesComponent())
		{
			//UE_LOG( TEXT("Can't join spline control points across different terrains.") );
			return;
		}

		for (const FLandscapeSplineConnection& Connection : Start->ConnectedSegments)
		{
			// if the *other* end on the connected segment connects to the "end" control point...
			if (Connection.GetFarConnection().ControlPoint == End)
			{
				//UE_LOG( TEXT("Spline control points already joined connected!") );
				return;
			}
		}

		ULandscapeSplinesComponent* SplinesComponent = Start->GetOuterULandscapeSplinesComponent();
		SplinesComponent->Modify();
		Start->Modify();
		End->Modify();

		ULandscapeSplineSegment* NewSegment = NewObject<ULandscapeSplineSegment>(SplinesComponent, NAME_None, RF_Transactional);
		SplinesComponent->Segments.Add(NewSegment);

		NewSegment->Connections[0].ControlPoint = Start;
		NewSegment->Connections[1].ControlPoint = End;

		NewSegment->Connections[0].SocketName = Start->GetBestConnectionTo(End->Location);
		NewSegment->Connections[1].SocketName = End->GetBestConnectionTo(Start->Location);

		FVector StartLocation; FRotator StartRotation;
		Start->GetConnectionLocationAndRotation(NewSegment->Connections[0].SocketName, StartLocation, StartRotation);
		FVector EndLocation; FRotator EndRotation;
		End->GetConnectionLocationAndRotation(NewSegment->Connections[1].SocketName, EndLocation, EndRotation);

		// Set up tangent lengths
		NewSegment->Connections[0].TangentLen = static_cast<float>((EndLocation - StartLocation).Size());
		NewSegment->Connections[1].TangentLen = NewSegment->Connections[0].TangentLen;

		NewSegment->AutoFlipTangents();

		// set up other segment options
		ULandscapeSplineSegment* CopyFromSegment = nullptr;
		if (Start->ConnectedSegments.Num() > 0)
		{
			CopyFromSegment = Start->ConnectedSegments[0].Segment;
		}
		else if (End->ConnectedSegments.Num() > 0)
		{
			CopyFromSegment = End->ConnectedSegments[0].Segment;
		}
		else
		{
			// Use defaults
		}

		if (CopyFromSegment != nullptr)
		{
			NewSegment->LayerName = CopyFromSegment->LayerName;
			NewSegment->SplineMeshes = CopyFromSegment->SplineMeshes;
			NewSegment->LDMaxDrawDistance = CopyFromSegment->LDMaxDrawDistance;
			NewSegment->bRaiseTerrain = CopyFromSegment->bRaiseTerrain;
			NewSegment->bLowerTerrain = CopyFromSegment->bLowerTerrain;
			NewSegment->bPlaceSplineMeshesInStreamingLevels = CopyFromSegment->bPlaceSplineMeshesInStreamingLevels;
			NewSegment->BodyInstance = CopyFromSegment->BodyInstance;
			NewSegment->bCastShadow = CopyFromSegment->bCastShadow;
			NewSegment->TranslucencySortPriority = CopyFromSegment->TranslucencySortPriority;
			NewSegment->RuntimeVirtualTextures = CopyFromSegment->RuntimeVirtualTextures;
			NewSegment->VirtualTextureLodBias = CopyFromSegment->VirtualTextureLodBias;
			NewSegment->VirtualTextureCullMips = CopyFromSegment->VirtualTextureCullMips;
			NewSegment->VirtualTextureRenderPassType = CopyFromSegment->VirtualTextureRenderPassType;
			NewSegment->bRenderCustomDepth = CopyFromSegment->bRenderCustomDepth;
			NewSegment->CustomDepthStencilWriteMask = CopyFromSegment->CustomDepthStencilWriteMask;
			NewSegment->CustomDepthStencilValue = CopyFromSegment->CustomDepthStencilValue;
		}

		Start->ConnectedSegments.Add(FLandscapeSplineConnection(NewSegment, 0));
		End->ConnectedSegments.Add(FLandscapeSplineConnection(NewSegment, 1));

		bool bUpdatedStart = false;
		bool bUpdatedEnd = false;
		if (bAutoRotateStart)
		{
			Start->AutoCalcRotation(bAlwaysRotateForward);
			Start->UpdateSplinePoints();
			bUpdatedStart = true;
		}
		if (bAutoRotateEnd)
		{
			End->AutoCalcRotation(bAlwaysRotateForward);
			End->UpdateSplinePoints();
			bUpdatedEnd = true;
		}

		// Control points' points are currently based on connected segments, so need to be updated.
		if (!bUpdatedStart && Start->Mesh)
		{
			Start->UpdateSplinePoints();
		}
		if (!bUpdatedEnd && End->Mesh)
		{
			End->UpdateSplinePoints();
		}

		// If we've called UpdateSplinePoints on either control point it will already have called UpdateSplinePoints on the new segment
		if (!(bUpdatedStart || bUpdatedEnd))
		{
			NewSegment->UpdateSplinePoints();
		}

		// Adding a segment will change the linear navigation path, reset it
		SplineSelection->ResetNavigationPath();
		SplineSelection->SelectNavigationControlPoint(Start);
	}

	void FlipSelectedSplineSegments()
	{
		for (ULandscapeSplineSegment* Segment : SplineSelection->GetSelectedSplineSegments())
		{
			FlipSegment(Segment);
		}
		EdMode->AutoUpdateDirtyLandscapeSplines();
	}

	// called when alt-dragging a newly added end segment
	bool UpdateAddSegment(ULandscapeSplineControlPoint* ControlPoint, FVector Location)
	{
		if (ControlPoint->ConnectedSegments.Num() != 1)
		{
			return false;
		}

		ULandscapeSplineSegment* Segment = ControlPoint->ConnectedSegments[0].Segment;
		bool bAutoRotateStart = ControlPoint == Segment->Connections[0].ControlPoint ? false : bAutoRotateOnJoin;
		bool bAutoRotateEnd = ControlPoint == Segment->Connections[1].ControlPoint ? false : bAutoRotateOnJoin;
	
		ULandscapeSplineControlPoint* Start = Segment->Connections[0].ControlPoint;
		ULandscapeSplineControlPoint* End = Segment->Connections[1].ControlPoint;

		ControlPoint->Location = Location;

		FVector StartLocation; FRotator StartRotation;
		Start->GetConnectionLocationAndRotation(Segment->Connections[0].SocketName, StartLocation, StartRotation);
		FVector EndLocation; FRotator EndRotation;
		End->GetConnectionLocationAndRotation(Segment->Connections[1].SocketName, EndLocation, EndRotation);

		// Set up tangent lengths
		Segment->Connections[0].TangentLen = static_cast<float>((EndLocation - StartLocation).Size());
		Segment->Connections[1].TangentLen = Segment->Connections[0].TangentLen;

		Segment->AutoFlipTangents();

		bool bUpdatedStart = false;
		bool bUpdatedEnd = false;
		if (bAutoRotateStart)
		{
			Start->AutoCalcRotation(bAlwaysRotateForward);
			Start->UpdateSplinePoints();
			bUpdatedStart = true;
		}

		if (bAutoRotateEnd)
		{
			End->AutoCalcRotation(bAlwaysRotateForward);
			End->UpdateSplinePoints();
			bUpdatedEnd = true;
		}

		// Control points' points are currently based on connected segments, so need to be updated.
		if (!bUpdatedStart && (Start->Mesh || Start == ControlPoint))
		{
			Start->UpdateSplinePoints();
			bUpdatedStart = true;
		}
		if (!bUpdatedEnd && (End->Mesh || End == ControlPoint))
		{
			End->UpdateSplinePoints();
			bUpdatedEnd = true;
		}

		// If we've called UpdateSplinePoints on either control point it will already have called UpdateSplinePoints on the new segment
		if (!(bUpdatedStart || bUpdatedEnd))
		{
			Segment->UpdateSplinePoints();
		}

		ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();
		SplinesComponent->MarkRenderStateDirty();

		// Adding a segment will change the linear navigation path, reset it
		SplineSelection->ResetNavigationPath();
		SplineSelection->SelectNavigationControlPoint(Start);

		return true;
	}

	void AddControlPoint(ULandscapeSplinesComponent* SplinesComponent, const FVector& LocalLocation)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AddControlPoint", "Add Landscape Spline Control Point"));

		SplinesComponent->Modify();

		ULandscapeSplineControlPoint* NewControlPoint = NewObject<ULandscapeSplineControlPoint>(SplinesComponent, NAME_None, RF_Transactional);
		SplinesComponent->ControlPoints.Add(NewControlPoint);

		NewControlPoint->Location = LocalLocation;

		const TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		if (SelectedSplineControlPoints.Num() > 0)
		{
			ULandscapeSplineControlPoint* FirstPoint = *SelectedSplineControlPoints.CreateConstIterator();

			if (bDuplicatingControlPoint)
			{
				NewControlPoint->Rotation = FirstPoint->Rotation;
			}
			else
			{
				FVector NewSegmentDirection = (NewControlPoint->Location - FirstPoint->Location) * (FirstPoint->ConnectedSegments.Num() == 0 || FirstPoint->ConnectedSegments[0].End ? 1.0f : -1.0f);
				NewControlPoint->Rotation = NewSegmentDirection.Rotation();
			}

			NewControlPoint->Width       = FirstPoint->Width;
			NewControlPoint->LayerWidthRatio = FirstPoint->LayerWidthRatio;
			NewControlPoint->SideFalloff = FirstPoint->SideFalloff;
			NewControlPoint->LeftSideFalloffFactor = FirstPoint->LeftSideFalloffFactor;
			NewControlPoint->RightSideFalloffFactor = FirstPoint->RightSideFalloffFactor;
			NewControlPoint->LeftSideLayerFalloffFactor = FirstPoint->LeftSideLayerFalloffFactor;
			NewControlPoint->RightSideLayerFalloffFactor = FirstPoint->RightSideLayerFalloffFactor;
			NewControlPoint->EndFalloff  = FirstPoint->EndFalloff;

			if (bCopyMeshToNewControlPoint)
			{
				NewControlPoint->Mesh             = FirstPoint->Mesh;
				NewControlPoint->MeshScale        = FirstPoint->MeshScale;
				NewControlPoint->bPlaceSplineMeshesInStreamingLevels = FirstPoint->bPlaceSplineMeshesInStreamingLevels;
				NewControlPoint->BodyInstance = FirstPoint->BodyInstance;
				NewControlPoint->bCastShadow      = FirstPoint->bCastShadow;
			}

			for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
			{
				if (ControlPoint->ConnectedSegments.Num() == 0 || ControlPoint->ConnectedSegments[0].End)
				{
					AddSegment(ControlPoint, NewControlPoint, bAutoRotateOnJoin, !bDuplicatingControlPoint);
				}
				else
				{
					AddSegment(NewControlPoint, ControlPoint, !bDuplicatingControlPoint, bAutoRotateOnJoin);
				}
			}
		}
		else
		{
			// required to make control point visible
			NewControlPoint->UpdateSplinePoints();
		}

		SplineSelection->ResetNavigationPath();
		SplineSelection->SelectControlPoint(NewControlPoint, ESplineNavigationFlags::UpdatePropertiesWindows);

		EdMode->AutoUpdateDirtyLandscapeSplines();
		if (!SplinesComponent->IsRegistered())
		{
			SplinesComponent->RegisterComponent();
		}
		else
		{
			SplinesComponent->MarkRenderStateDirty();
		}
	}

	void DeleteSegment(ULandscapeSplineSegment* ToDelete, bool bInDeleteLooseEnds)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_DeleteSegment", "Delete Landscape Spline Segment"));

		ULandscapeSplinesComponent* SplinesComponent = ToDelete->GetOuterSafe();
		if (!SplinesComponent)
		{
			// Segment is already deleted
			return;
		}
		SplinesComponent->Modify();

		ToDelete->Modify();
		ToDelete->DeleteSplinePoints();

		ToDelete->Connections[0].ControlPoint->Modify();
		ToDelete->Connections[1].ControlPoint->Modify();
		ToDelete->Connections[0].ControlPoint->ConnectedSegments.Remove(FLandscapeSplineConnection(ToDelete, 0));
		ToDelete->Connections[1].ControlPoint->ConnectedSegments.Remove(FLandscapeSplineConnection(ToDelete, 1));

		if (bInDeleteLooseEnds)
		{
			if (ToDelete->Connections[0].ControlPoint->ConnectedSegments.Num() == 0)
			{
				SplinesComponent->ControlPoints.Remove(ToDelete->Connections[0].ControlPoint);
			}
			if (ToDelete->Connections[1].ControlPoint != ToDelete->Connections[0].ControlPoint
				&& ToDelete->Connections[1].ControlPoint->ConnectedSegments.Num() == 0)
			{
				SplinesComponent->ControlPoints.Remove(ToDelete->Connections[1].ControlPoint);
			}
		}

		SplinesComponent->Segments.Remove(ToDelete);
		SplineSelection->ResetNavigationPath();

		ToDelete->Connections[0].ControlPoint->UpdateSplinePoints();
		ToDelete->Connections[1].ControlPoint->UpdateSplinePoints();

		EdMode->AutoUpdateDirtyLandscapeSplines();
		SplinesComponent->MarkRenderStateDirty();
	}

	void DeleteControlPoint(ULandscapeSplineControlPoint* ToDelete, bool bInDeleteLooseEnds)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_DeleteControlPoint", "Delete Landscape Spline Control Point"));

		ULandscapeSplinesComponent* SplinesComponent = ToDelete->GetOuterSafe();
		if (!SplinesComponent)
		{
			// Point is already deleted
			return;
		}

		// If needed, create prompt window before we start editing.  Creating the window can cause the selection code to touch the
		// spline network when it updates hit proxies.  Don't let that code observe an incomplete state.
		bool bConnectable = ToDelete->ConnectedSegments.Num() == 2 && ToDelete->ConnectedSegments[0].Segment != ToDelete->ConnectedSegments[1].Segment;
		int32 ConnectChoice = EAppReturnType::No;
		if (bConnectable)
		{
			ConnectChoice = FMessageDialog::Open(EAppMsgType::YesNoCancel, LOCTEXT("WantToJoinControlPoint", "Control point has two segments attached, do you want to join them?"));

			if (ConnectChoice == EAppReturnType::Cancel)
			{
				return;
			}
		}

		SplinesComponent->Modify();
		ToDelete->Modify();
		ToDelete->DeleteSplinePoints();
		SplineSelection->ResetNavigationPath();

		if (bConnectable)
		{
			switch (ConnectChoice)
			{
			case EAppReturnType::Yes:
			{
				// Copy the other end of connection 1 into the near end of connection 0, then delete connection 1
				TArray<FLandscapeSplineConnection>& Connections = ToDelete->ConnectedSegments;
				Connections[0].Segment->Modify();
				Connections[1].Segment->Modify();

				Connections[0].GetNearConnection() = Connections[1].GetFarConnection();
				Connections[0].Segment->UpdateSplinePoints();

				Connections[1].Segment->DeleteSplinePoints();

				// Get the control point at the *other* end of the segment and remove it from it
				ULandscapeSplineControlPoint* OtherEnd = Connections[1].GetFarConnection().ControlPoint;
				OtherEnd->Modify();

				FLandscapeSplineConnection* OtherConnection = OtherEnd->ConnectedSegments.FindByKey(FLandscapeSplineConnection(Connections[1].Segment, 1 - Connections[1].End));
				*OtherConnection = FLandscapeSplineConnection(Connections[0].Segment, Connections[0].End);

				SplinesComponent->Segments.Remove(Connections[1].Segment);

				ToDelete->ConnectedSegments.Empty();

				SplinesComponent->ControlPoints.Remove(ToDelete);
				EdMode->AutoUpdateDirtyLandscapeSplines();
				SplinesComponent->MarkRenderStateDirty();

				return;
			}
			case EAppReturnType::No:
				// Use the "delete all segments" code below
				break;
			default:
				check(false); //unreachable
				break;
			}
		}

		for (FLandscapeSplineConnection& Connection : ToDelete->ConnectedSegments)
		{
			Connection.Segment->Modify();
			Connection.Segment->DeleteSplinePoints();

			// Get the control point at the *other* end of the segment and remove it from it
			// Note: ULandscapeSplineControlPoint::UpdateSplinePoints is called for this case, but not for the "Yes, joint segments" case
			// above.  It depends on the connected socket names only, not on the connected segment directions/values.
			ULandscapeSplineControlPoint* OtherEnd = Connection.GetFarConnection().ControlPoint;
			OtherEnd->Modify();
			OtherEnd->ConnectedSegments.Remove(FLandscapeSplineConnection(Connection.Segment, 1 - Connection.End));
			// OtherEnd might already be DeleteSplinePoints-ed when batch-deleting multiple points.
			if (OtherEnd->GetOuterSafe() != nullptr)
			{
				OtherEnd->UpdateSplinePoints();
			}

			SplinesComponent->Segments.Remove(Connection.Segment);

			if (bInDeleteLooseEnds)
			{
				if (OtherEnd != ToDelete
					&& OtherEnd->ConnectedSegments.Num() == 0)
				{
					SplinesComponent->ControlPoints.Remove(OtherEnd);
				}
			}
		}

		ToDelete->ConnectedSegments.Empty();

		SplinesComponent->ControlPoints.Remove(ToDelete);
		EdMode->AutoUpdateDirtyLandscapeSplines();
		SplinesComponent->MarkRenderStateDirty();
	}

	void SplitSegment(ULandscapeSplineSegment* Segment, const FVector& LocalLocation)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SplitSegment", "Split Landscape Spline Segment"));

		ULandscapeSplinesComponent* SplinesComponent = Segment->GetOuterULandscapeSplinesComponent();
		SplinesComponent->Modify();
		Segment->Modify();
		Segment->Connections[1].ControlPoint->Modify();

		float t;
		FVector Location;
		FVector Tangent;
		Segment->FindNearest(LocalLocation, t, Location, Tangent);

		if (bDuplicatingControlPoint)
		{
			DuplicateCacheSplitSegmentParam = t;
			DuplicateCacheSplitSegmentTangentLenStart = Segment->Connections[0].TangentLen;
			DuplicateCacheSplitSegmentTangentLenEnd = Segment->Connections[1].TangentLen;
			DuplicateCacheSplitSegmentTangentLen = static_cast<float>(Tangent.Size());
		}

		ULandscapeSplineControlPoint* NewControlPoint = NewObject<ULandscapeSplineControlPoint>(SplinesComponent, NAME_None, RF_Transactional);
		SplinesComponent->ControlPoints.Add(NewControlPoint);

		if (bDuplicatingControlPoint)
		{
			NewControlPoint->Location = LocalLocation;
			NewControlPoint->Rotation = DuplicateCachedRotation;
		}
		else
		{
			NewControlPoint->Location = Location;
			NewControlPoint->Rotation = Tangent.Rotation();
			NewControlPoint->Rotation.Roll = FMath::Lerp(Segment->Connections[0].ControlPoint->Rotation.Roll, Segment->Connections[1].ControlPoint->Rotation.Roll, t);
		}

		NewControlPoint->Width = FMath::Lerp(Segment->Connections[0].ControlPoint->Width, Segment->Connections[1].ControlPoint->Width, t);
		NewControlPoint->LayerWidthRatio = FMath::Lerp(Segment->Connections[0].ControlPoint->LayerWidthRatio, Segment->Connections[1].ControlPoint->LayerWidthRatio, t);
		NewControlPoint->SideFalloff = FMath::Lerp(Segment->Connections[0].ControlPoint->SideFalloff, Segment->Connections[1].ControlPoint->SideFalloff, t);
		NewControlPoint->EndFalloff = FMath::Lerp(Segment->Connections[0].ControlPoint->EndFalloff, Segment->Connections[1].ControlPoint->EndFalloff, t);
		NewControlPoint->LeftSideFalloffFactor = FMath::Clamp(FMath::Lerp(Segment->Connections[0].ControlPoint->LeftSideFalloffFactor, Segment->Connections[1].ControlPoint->LeftSideFalloffFactor, t), 0.f, 1.f);
		NewControlPoint->RightSideFalloffFactor = FMath::Clamp(FMath::Lerp(Segment->Connections[0].ControlPoint->RightSideFalloffFactor, Segment->Connections[1].ControlPoint->RightSideFalloffFactor, t), 0.f, 1.f);
		NewControlPoint->LeftSideLayerFalloffFactor = FMath::Clamp(FMath::Lerp(Segment->Connections[0].ControlPoint->LeftSideLayerFalloffFactor, Segment->Connections[1].ControlPoint->LeftSideLayerFalloffFactor, t), 0.f, 1.f);
		NewControlPoint->RightSideLayerFalloffFactor = FMath::Clamp(FMath::Lerp(Segment->Connections[0].ControlPoint->RightSideLayerFalloffFactor, Segment->Connections[1].ControlPoint->RightSideLayerFalloffFactor, t), 0.f, 1.f);

		ULandscapeSplineSegment* NewSegment = NewObject<ULandscapeSplineSegment>(SplinesComponent, NAME_None, RF_Transactional);
		SplinesComponent->Segments.Add(NewSegment);

		NewSegment->Connections[0].ControlPoint = NewControlPoint;
		NewSegment->Connections[0].TangentLen = static_cast<float>(Tangent.Size() * (1 - t));
		NewSegment->Connections[0].ControlPoint->ConnectedSegments.Add(FLandscapeSplineConnection(NewSegment, 0));
		NewSegment->Connections[1].ControlPoint = Segment->Connections[1].ControlPoint;
		NewSegment->Connections[1].TangentLen = Segment->Connections[1].TangentLen * (1 - t);
		NewSegment->Connections[1].ControlPoint->ConnectedSegments.Add(FLandscapeSplineConnection(NewSegment, 1));
		NewSegment->LayerName = Segment->LayerName;
		NewSegment->SplineMeshes = Segment->SplineMeshes;
		NewSegment->LDMaxDrawDistance = Segment->LDMaxDrawDistance;
		NewSegment->bRaiseTerrain = Segment->bRaiseTerrain;
		NewSegment->bLowerTerrain = Segment->bLowerTerrain;
		NewSegment->BodyInstance = Segment->BodyInstance;
		NewSegment->bCastShadow = Segment->bCastShadow;
		NewSegment->TranslucencySortPriority = Segment->TranslucencySortPriority;
		NewSegment->RuntimeVirtualTextures = Segment->RuntimeVirtualTextures;
		NewSegment->VirtualTextureLodBias = Segment->VirtualTextureLodBias;
		NewSegment->VirtualTextureCullMips = Segment->VirtualTextureCullMips;
		NewSegment->VirtualTextureRenderPassType = Segment->VirtualTextureRenderPassType;
		NewSegment->bRenderCustomDepth = Segment->bRenderCustomDepth;
		NewSegment->CustomDepthStencilWriteMask = Segment->CustomDepthStencilWriteMask;
		NewSegment->CustomDepthStencilValue = Segment->CustomDepthStencilValue;

		Segment->Connections[0].TangentLen *= t;
		Segment->Connections[1].ControlPoint->ConnectedSegments.Remove(FLandscapeSplineConnection(Segment, 1));
		Segment->Connections[1].ControlPoint = NewControlPoint;
		Segment->Connections[1].TangentLen = static_cast<float>(-Tangent.Size() * t);
		Segment->Connections[1].ControlPoint->ConnectedSegments.Add(FLandscapeSplineConnection(Segment, 1));

		Segment->UpdateSplinePoints();
		NewSegment->UpdateSplinePoints();

		SplineSelection->ClearSelection();

		SplinesComponent->MarkRenderStateDirty();
		SplinesComponent->CheckSplinesValid();
	}

	bool UpdateSplitSegment(ULandscapeSplineControlPoint* ControlPoint, const FVector& LocalLocation)
	{
		check(ControlPoint->ConnectedSegments.Num() == 2);
		ULandscapeSplineSegment* Segment0 = ControlPoint->ConnectedSegments[0].Segment;
		ULandscapeSplineSegment* Segment1 = ControlPoint->ConnectedSegments[1].Segment;
		ULandscapeSplineSegment *Segment, *NewSegment;
		if (ControlPoint->ConnectedSegments[0].End == 0)
		{
			Segment = ControlPoint->ConnectedSegments[1].Segment;
			NewSegment = ControlPoint->ConnectedSegments[0].Segment;
		}
		else
		{
			Segment = ControlPoint->ConnectedSegments[0].Segment;
			NewSegment = ControlPoint->ConnectedSegments[1].Segment;
		}

		float t, t0, t1, tseg;
		FVector Location0, Location1;
		FVector Tangent0, Tangent1;
		Segment->FindNearest(LocalLocation, t0, Location0, Tangent0);
		NewSegment->FindNearest(LocalLocation, t1, Location1, Tangent1);

		ULandscapeSplineSegment* UseSegment;
		if (FVector::Distance(LocalLocation, Location0) < FVector::Distance(LocalLocation, Location1))
		{
			t = DuplicateCacheSplitSegmentParam * t0;
			tseg = t0;
			UseSegment = Segment;
		}
		else
		{
			t = DuplicateCacheSplitSegmentParam + (1 - DuplicateCacheSplitSegmentParam) * t1;
			tseg = t1;
			UseSegment = NewSegment;
		}
		DuplicateCacheSplitSegmentParam = t;

		ControlPoint->Location = LocalLocation;

		// Do not update rotation during alt-drag.
		//ControlPoint->Rotation.Roll = FMath::Lerp(UseSegment->Connections[0].ControlPoint->Rotation.Roll, UseSegment->Connections[1].ControlPoint->Rotation.Roll, tseg);
		ControlPoint->Width = FMath::Lerp(UseSegment->Connections[0].ControlPoint->Width, UseSegment->Connections[1].ControlPoint->Width, tseg);
		ControlPoint->LayerWidthRatio = FMath::Lerp(UseSegment->Connections[0].ControlPoint->LayerWidthRatio, UseSegment->Connections[1].ControlPoint->LayerWidthRatio, tseg);
		ControlPoint->SideFalloff = FMath::Lerp(UseSegment->Connections[0].ControlPoint->SideFalloff, UseSegment->Connections[1].ControlPoint->SideFalloff, tseg);
		ControlPoint->LeftSideFalloffFactor = FMath::Clamp(FMath::Lerp(UseSegment->Connections[0].ControlPoint->LeftSideFalloffFactor, UseSegment->Connections[1].ControlPoint->LeftSideFalloffFactor, tseg), 0.f, 1.f);
		ControlPoint->RightSideFalloffFactor = FMath::Clamp(FMath::Lerp(UseSegment->Connections[0].ControlPoint->RightSideFalloffFactor, UseSegment->Connections[1].ControlPoint->RightSideFalloffFactor, tseg), 0.f, 1.f);
		ControlPoint->LeftSideLayerFalloffFactor = FMath::Clamp(FMath::Lerp(UseSegment->Connections[0].ControlPoint->LeftSideLayerFalloffFactor, UseSegment->Connections[1].ControlPoint->LeftSideLayerFalloffFactor, tseg), 0.f, 1.f);
		ControlPoint->RightSideLayerFalloffFactor = FMath::Clamp(FMath::Lerp(UseSegment->Connections[0].ControlPoint->RightSideLayerFalloffFactor, UseSegment->Connections[1].ControlPoint->RightSideLayerFalloffFactor, tseg), 0.f, 1.f);
		ControlPoint->EndFalloff = FMath::Lerp(UseSegment->Connections[0].ControlPoint->EndFalloff, UseSegment->Connections[1].ControlPoint->EndFalloff, tseg);

		Segment->Connections[0].TangentLen = DuplicateCacheSplitSegmentTangentLenStart * t;
		Segment->Connections[1].TangentLen = -DuplicateCacheSplitSegmentTangentLen * t;

		NewSegment->Connections[0].TangentLen = DuplicateCacheSplitSegmentTangentLen * (1 - t);
		NewSegment->Connections[1].TangentLen = DuplicateCacheSplitSegmentTangentLenEnd * (1 - t);

		if (bAutoChangeConnectionsOnMove)
		{
			ControlPoint->AutoSetConnections(true);
		}

		ControlPoint->UpdateSplinePoints();
		Segment->UpdateSplinePoints();
		NewSegment->UpdateSplinePoints();

		ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();
		SplinesComponent->MarkRenderStateDirty();
		SplinesComponent->CheckSplinesValid();

		return true;
	}

	void FlipSegment(ULandscapeSplineSegment* Segment)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_FlipSegment", "Flip Landscape Spline Segment"));

		ULandscapeSplinesComponent* SplinesComponent = Segment->GetOuterULandscapeSplinesComponent();
		SplinesComponent->Modify();
		Segment->Modify();

		Segment->Connections[0].ControlPoint->Modify();
		Segment->Connections[1].ControlPoint->Modify();
		Segment->Connections[0].ControlPoint->ConnectedSegments.FindByKey(FLandscapeSplineConnection(Segment, 0))->End = 1;
		Segment->Connections[1].ControlPoint->ConnectedSegments.FindByKey(FLandscapeSplineConnection(Segment, 1))->End = 0;
		Swap(Segment->Connections[0], Segment->Connections[1]);

		Segment->UpdateSplinePoints();
	}

	void SnapControlPointToGround(ULandscapeSplineControlPoint* ControlPoint)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SnapToGround", "Snap Landscape Spline to Ground"));

		ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();
		SplinesComponent->Modify();
		ControlPoint->Modify();

		const FTransform LocalToWorld = SplinesComponent->GetComponentToWorld();
		const FVector Start = LocalToWorld.TransformPosition(ControlPoint->Location);
		const FVector End = Start + FVector(0, 0, -HALF_WORLD_MAX);

		static FName TraceTag = FName(TEXT("SnapLandscapeSplineControlPointToGround"));
		FHitResult Hit;
		UWorld* World = SplinesComponent->GetWorld();
		check(World);
		if (World->LineTraceSingleByObjectType(Hit, Start, End, FCollisionObjectQueryParams(ECC_WorldStatic), FCollisionQueryParams(NAME_None, FCollisionQueryParams::GetUnknownStatId(),true)))
		{
			ControlPoint->Location = LocalToWorld.InverseTransformPosition(Hit.Location);
			ControlPoint->UpdateSplinePoints();
			SplinesComponent->MarkRenderStateDirty();
		}
	}

	void UpdateSplineMeshLevels()
	{
		for (ULandscapeSplineControlPoint* ControlPoint : SplineSelection->GetSelectedSplineControlPoints())
		{
			const bool bUpdateCollision = true;
			const bool bUpdateSegments = false;
			const bool bUpdateMeshLevel = true;
			ControlPoint->UpdateSplinePoints(bUpdateCollision, bUpdateSegments, bUpdateMeshLevel);
		}

		for (ULandscapeSplineSegment* Segment : SplineSelection->GetSelectedSplineSegments())
		{
			const bool bUpdateCollision = true;
			const bool bUpdateMeshLevel = true;
			Segment->UpdateSplinePoints(bUpdateCollision, bUpdateMeshLevel);
		}
	}

	bool CanMoveSelectedToLevel() const
	{
		// Move to level only supportd on LandscapeProxy Splines
		for (ULandscapeSplineControlPoint* ControlPoint : SplineSelection->GetSelectedSplineControlPoints())
		{
			ULandscapeSplinesComponent* LandscapeSplinesComp = ControlPoint->GetOuterULandscapeSplinesComponent();
			ALandscapeProxy* FromProxy = LandscapeSplinesComp ? Cast<ALandscapeProxy>(LandscapeSplinesComp->GetOuter()) : nullptr;
			if (!FromProxy)
			{
				return false;
			}
		}

		return true;
	}

	void MoveSelectedToLevel()
	{
		TSet<ALandscapeProxy*> FromProxies;
		ALandscapeProxy* ToLandscape = nullptr;

		for (ULandscapeSplineControlPoint* ControlPoint : SplineSelection->GetSelectedSplineControlPoints())
		{
			ULandscapeSplinesComponent* LandscapeSplinesComp = ControlPoint->GetOuterULandscapeSplinesComponent();
			ALandscapeProxy* FromProxy = LandscapeSplinesComp ? Cast<ALandscapeProxy>(LandscapeSplinesComp->GetOuter()) : nullptr;
			if (FromProxy)
			{
				ULandscapeInfo* ProxyLandscapeInfo = FromProxy->GetLandscapeInfo();
				check(ProxyLandscapeInfo);
				if (!ToLandscape)
				{
					ToLandscape = ProxyLandscapeInfo->GetCurrentLevelLandscapeProxy(true);
					if (!ToLandscape)
					{
						// No Landscape Proxy, don't support for creating only for Spline now
						return;
					}
				}

				ProxyLandscapeInfo->MoveSplineToLevel(ControlPoint, ToLandscape->GetLevel());
			}
		}
				
		ULandscapeSplinesComponent* SplineComponent = ToLandscape ? ToLandscape->GetSplinesComponent() : nullptr;
		if (SplineComponent)
		{
			if (!SplineComponent->IsRegistered())
			{
				SplineComponent->RegisterComponent();
			}
			else
			{
				SplineComponent->MarkRenderStateDirty();
			}
			SplineComponent->CheckSplinesValid();
		}

		GUnrealEd->RedrawLevelEditingViewports();
	}

	void ShowSplineProperties()
	{
		TArray<UObject*> Objects;
		Objects.Reset(SplineSelection->GetSelectedSplineControlPoints().Num() + SplineSelection->GetSelectedSplineSegments().Num());
		Algo::Copy(SplineSelection->GetSelectedSplineControlPoints(), Objects);
		Algo::Copy(SplineSelection->GetSelectedSplineSegments(), Objects);

		FPropertyEditorModule& PropertyModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		if (!PropertyModule.HasUnlockedDetailViews())
		{
			PropertyModule.CreateFloatingDetailsView(Objects, true);
		}
		else
		{
			PropertyModule.UpdatePropertyViews(Objects);
		}
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& InTarget, const FVector& InHitLocation) override
	{
		HHitProxy* HitProxy = ViewportClient->Viewport->GetHitProxy(ViewportClient->Viewport->GetMouseX(), ViewportClient->Viewport->GetMouseY());
		if (HitProxy && ViewportClient->IsCtrlPressed())
		{
			LandscapeInfo = InTarget.LandscapeInfo.Get();
			ILandscapeSplineInterface* SplineOwner = nullptr;

			// If we have a selection use the landscape of the selected spline
			const TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

			if (SelectedSplineControlPoints.Num() > 0)
			{
				ULandscapeSplineControlPoint* FirstPoint = *SelectedSplineControlPoints.CreateConstIterator();
				ULandscapeSplinesComponent* SelectedSplinesComponent = FirstPoint->GetOuterULandscapeSplinesComponent();
			
				if (SelectedSplinesComponent)
				{
					SplineOwner = SelectedSplinesComponent->GetSplineOwner();
				}
			}
					
			const bool bIsGridBased = EdMode->IsGridBased();

			// Hit Test
			if (!SplineOwner)
			{
				if (HitProxy->IsA(HActor::StaticGetType()))
				{
					HActor* ActorProxy = (HActor*)HitProxy;
					// Here we want to make sure we are selecting the valid type (Grid based or not)
					if (bIsGridBased)
					{
						SplineOwner = Cast<ALandscapeSplineActor>(ActorProxy->Actor);
					}
					else
					{
						SplineOwner = Cast<ALandscapeProxy>(ActorProxy->Actor);
					}
				}
			}
				
			// Open transaction here because we might be creating an actor or later a spline component
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AddControlPoint", "Add Landscape Spline Control Point"));
			if (!SplineOwner)
			{
				if (bIsGridBased)
				{
					// Create Spline Actor at World Position (InHitLocation is local to Landscape)
					FVector WorldHitLocation = LandscapeInfo->LandscapeActor.Get()->LandscapeActorToWorld().TransformPosition(InHitLocation);
					SplineOwner = LandscapeInfo->CreateSplineActor(WorldHitLocation);
				}
				else
				{
					// Default to Current level Landscape
					SplineOwner = LandscapeInfo->GetCurrentLevelLandscapeProxy(true);
				}
			}

			// No Spline Owner found
			if (!SplineOwner || !SplineOwner->IsSplineOwnerValid())
			{
				return false;
			}

			ULandscapeSplinesComponent* SplinesComponent = SplineOwner->GetSplinesComponent();
			if (!SplinesComponent)
			{
				SplineOwner->CreateSplineComponent();
				SplinesComponent = SplineOwner->GetSplinesComponent();
				check(SplinesComponent);
			}

			// Get Main Landscape Actor to Spline Component Transform
			const FTransform LandscapeToSpline = SplineOwner->LandscapeActorToWorld().GetRelativeTransform(SplinesComponent->GetComponentTransform());

			// Local to SplineComponent
			AddControlPoint(SplinesComponent, LandscapeToSpline.TransformPosition(InHitLocation));

			GUnrealEd->RedrawLevelEditingViewports();

			return true;
		}

		return false;
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		LandscapeInfo = nullptr;
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		FVector HitLocation;
		if (EdMode->LandscapeMouseTrace(ViewportClient, x, y, HitLocation))
		{
			//if( bToolActive )
			//{
			//	// Apply tool
			//	ApplyTool(ViewportClient);
			//}
		}

		return true;
	}

	virtual void ApplyTool(FEditorViewportClient* ViewportClient)
	{
	}

	template <typename T>
	void SetTargetLandscapeBasedOnSelection(T* Selection)
	{
		check(Selection);
		if (ALandscapeProxy* LandscapeProxy = Selection->template GetTypedOuter<ALandscapeProxy>())
		{
			ALandscape* NewLandscapeActor = LandscapeProxy->GetLandscapeActor();
			if (NewLandscapeActor && (NewLandscapeActor != EdMode->GetLandscape()))
			{
				EdMode->SetTargetLandscape(LandscapeProxy->GetLandscapeInfo());
			}
		}
	}

	virtual bool HandleClick(HHitProxy* HitProxy, const FViewportClick& Click) override
	{
		ULandscapeSplineControlPoint* ClickedControlPoint = nullptr;
		ULandscapeSplineSegment* ClickedSplineSegment = nullptr;

		bool bIsValidSplineHitProxy = false;

		if (HitProxy)
		{
			if (HitProxy->IsA(HLandscapeSplineProxy_ControlPoint::StaticGetType()))
			{
				HLandscapeSplineProxy_ControlPoint* SplineProxy = (HLandscapeSplineProxy_ControlPoint*)HitProxy;
				ClickedControlPoint = SplineProxy->ControlPoint;
			}
			else if (HitProxy->IsA(HLandscapeSplineProxy_Segment::StaticGetType()))
			{
				HLandscapeSplineProxy_Segment* SplineProxy = (HLandscapeSplineProxy_Segment*)HitProxy;
				ClickedSplineSegment = SplineProxy->SplineSegment;
			}
			else if (HitProxy->IsA(HWidgetAxis::StaticGetType()) || HitProxy->IsA(HLandscapeSplineProxy_Tangent::StaticGetType()))
			{
				bIsValidSplineHitProxy = true;
			}
			else if (HitProxy->IsA(HActor::StaticGetType()))
			{
				HActor* ActorProxy = (HActor*)HitProxy;
				AActor* Actor = ActorProxy->Actor;
				const UMeshComponent* MeshComponent = Cast<UMeshComponent>(ActorProxy->PrimComponent);
				if (MeshComponent)
				{
					ULandscapeSplinesComponent* SplineComponent = Actor->FindComponentByClass<ULandscapeSplinesComponent>();
					if (SplineComponent)
					{
						UObject* ComponentOwner = SplineComponent->GetOwnerForMeshComponent(MeshComponent);
						if (ComponentOwner)
						{
							if (ULandscapeSplineControlPoint* ControlPoint = Cast<ULandscapeSplineControlPoint>(ComponentOwner))
							{
								ClickedControlPoint = ControlPoint;
							}
							else if (ULandscapeSplineSegment* SplineSegment = Cast<ULandscapeSplineSegment>(ComponentOwner))
							{
								ClickedSplineSegment = SplineSegment;
							}
						}
					}
				}
			}
		}
		
		bIsValidSplineHitProxy = bIsValidSplineHitProxy || ClickedSplineSegment || ClickedControlPoint;

		if (!HitProxy || !bIsValidSplineHitProxy)
		{
			SplineSelection->ClearSelection();
			return false;
		}

		if (ClickedControlPoint != nullptr)
		{
			if (Click.IsShiftDown() && ClickedControlPoint->IsSplineSelected())
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_DeselectPoint", "Deselect Landscape Spline Point"));
				SplineSelection->DeselectControlPoint(ClickedControlPoint, ESplineNavigationFlags::UpdatePropertiesWindows);
				GEditor->SelectNone(true, true);
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectPoint", "Select Landscape Spline Point"));
				SetTargetLandscapeBasedOnSelection(ClickedControlPoint);
				SplineSelection->SelectControlPoint(ClickedControlPoint, ESplineNavigationFlags::UpdatePropertiesWindows | (Click.IsShiftDown() ? ESplineNavigationFlags::AddToSelection : ESplineNavigationFlags::None));
				GEditor->SelectNone(true, true);
			}
			return true;
		}
		else if (ClickedSplineSegment != nullptr)
		{
			// save info about what we grabbed
			if (Click.IsShiftDown() && ClickedSplineSegment->IsSplineSelected())
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_DeselectSegment", "Deselect Landscape Spline Segment"));
				SplineSelection->DeSelectSegment(ClickedSplineSegment, ESplineNavigationFlags::UpdatePropertiesWindows);
				GEditor->SelectNone(true, true);
			}
			else
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectSegment", "Select Landscape Spline Segment"));
				SetTargetLandscapeBasedOnSelection(ClickedSplineSegment);
				SplineSelection->SelectSegment(ClickedSplineSegment, ESplineNavigationFlags::UpdatePropertiesWindows | (Click.IsShiftDown() ? ESplineNavigationFlags::AddToSelection : ESplineNavigationFlags::None));
				GEditor->SelectNone(true, true);
			}
			return true;
		}
		
		return false;
	}

	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();
		TArray<ULandscapeSplineSegment*> SelectedSplineSegments = SplineSelection->GetSelectedSplineSegments();

		if (InKey == EKeys::F4 && InEvent == IE_Pressed)
		{
			if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
			{
				ShowSplineProperties();
				return true;
			}
		}

		if (InKey == EKeys::R && InEvent == IE_Pressed)
		{
			if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AutoRotate", "Auto-rotate Landscape Spline Control Points"));

				for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
				{
					ControlPoint->AutoCalcRotation(bAlwaysRotateForward);
					ControlPoint->UpdateSplinePoints();
				}

				for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
				{
					Segment->Connections[0].ControlPoint->AutoCalcRotation(bAlwaysRotateForward);
					Segment->Connections[0].ControlPoint->UpdateSplinePoints();
					Segment->Connections[1].ControlPoint->AutoCalcRotation(bAlwaysRotateForward);
					Segment->Connections[1].ControlPoint->UpdateSplinePoints();
				}

				EdMode->AutoUpdateDirtyLandscapeSplines();
				return true;
			}
		}

		if (InKey == EKeys::F && InEvent == IE_Pressed)
		{
			if (SelectedSplineSegments.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_FlipSegments", "Flip Selected Landscape Spline Segments"));
				FlipSelectedSplineSegments();
				return true;
			}
		}

		if (InKey == EKeys::T && InEvent == IE_Pressed)
		{
			if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AutoFlipTangents", "Auto-flip Landscape Spline Tangents"));

				for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
				{
					ControlPoint->AutoFlipTangents();
					ControlPoint->UpdateSplinePoints();
				}

				for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
				{
					Segment->Connections[0].ControlPoint->AutoFlipTangents();
					Segment->Connections[0].ControlPoint->UpdateSplinePoints();
					Segment->Connections[1].ControlPoint->AutoFlipTangents();
					Segment->Connections[1].ControlPoint->UpdateSplinePoints();
				}

				EdMode->AutoUpdateDirtyLandscapeSplines();
				return true;
			}
		}

		if (InKey == EKeys::End && InEvent == IE_Pressed)
		{
			if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SnapToGround", "Snap Landscape Spline to Ground"));

				for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
				{
					SnapControlPointToGround(ControlPoint);
				}
				for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
				{
					SnapControlPointToGround(Segment->Connections[0].ControlPoint);
					SnapControlPointToGround(Segment->Connections[1].ControlPoint);
				}
				SplineSelection->UpdatePropertiesWindows();
				EdMode->AutoUpdateDirtyLandscapeSplines();
				return true;
			}
		}

		if (InKey == EKeys::A && InEvent == IE_Pressed
			&& IsCtrlDown(InViewport))
		{
			if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
			{
				FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectConnectedSegments", "Select Landscape Spline Connected Segments"));
				SplineSelection->SelectConnected();
				SplineSelection->UpdatePropertiesWindows();
				return true;
			}
		}

		if (SelectedSplineControlPoints.Num() > 0)
		{
			if (InKey == EKeys::LeftMouseButton && InEvent == IE_Pressed
				&& IsCtrlDown(InViewport))
			{
				int32 HitX = InViewport->GetMouseX();
				int32 HitY = InViewport->GetMouseY();
				HHitProxy* HitProxy = InViewport->GetHitProxy(HitX, HitY);
				if (HitProxy != nullptr)
				{
					ULandscapeSplineControlPoint* ClickedControlPoint = nullptr;

					if (HitProxy->IsA(HLandscapeSplineProxy_ControlPoint::StaticGetType()))
					{
						HLandscapeSplineProxy_ControlPoint* SplineProxy = (HLandscapeSplineProxy_ControlPoint*)HitProxy;
						ClickedControlPoint = SplineProxy->ControlPoint;
					}
					else if (HitProxy->IsA(HActor::StaticGetType()))
					{
						HActor* ActorProxy = (HActor*)HitProxy;
						AActor* Actor = ActorProxy->Actor;
						const UMeshComponent* MeshComponent = Cast<UMeshComponent>(ActorProxy->PrimComponent);
						if (MeshComponent)
						{
							ULandscapeSplinesComponent* SplineComponent = Actor->FindComponentByClass<ULandscapeSplinesComponent>();
							if (SplineComponent)
							{
								UObject* ComponentOwner = SplineComponent->GetOwnerForMeshComponent(MeshComponent);
								if (ComponentOwner)
								{
									if (ULandscapeSplineControlPoint* ControlPoint = Cast<ULandscapeSplineControlPoint>(ComponentOwner))
									{
										ClickedControlPoint = ControlPoint;
									}
								}
							}
						}
					}

					if (ClickedControlPoint != nullptr)
					{
						// Merge Spline into the same actor if a single Control Point is currently selected and the ClickedControlPoint is from a different owner
						if (SelectedSplineControlPoints.Num() == 1)
						{
							ULandscapeSplineControlPoint* SourceControlPoint = *SelectedSplineControlPoints.CreateIterator();

							ULandscapeSplinesComponent* SourceComponent = SourceControlPoint->GetOuterULandscapeSplinesComponent();
							ALandscapeSplineActor* SourceSplineActor = SourceComponent ? Cast<ALandscapeSplineActor>(SourceComponent->GetOuter()) : nullptr;

							ULandscapeSplinesComponent* ClickedComponent = ClickedControlPoint->GetOuterULandscapeSplinesComponent();
							ALandscapeSplineActor* ClickedSplineActor = ClickedComponent ? Cast<ALandscapeSplineActor>(ClickedComponent->GetOuter()) : nullptr;

							if (SourceSplineActor && ClickedSplineActor && SourceSplineActor != ClickedSplineActor)
							{
								if (SourceSplineActor->GetLandscapeGuid() != ClickedSplineActor->GetLandscapeGuid())
								{
									UE_LOGF(LogLandscapeEdMode, Warning, "Can't merge LandscapeSplineActors belonging to different Landscapes");
									return true;
								}

								{
									FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_MergeSpline", "Merge Spline"));

									ULandscapeInfo* SourceLandscapeInfo = SourceSplineActor->GetLandscapeInfo();
									check(SourceLandscapeInfo);

									SourceLandscapeInfo->MoveSpline(ClickedControlPoint, SourceSplineActor);
									AddSegment(SourceControlPoint, ClickedControlPoint, true, true);

									// Moving the spline should leave us with an empty actor that we can delete
									if (ClickedComponent->GetControlPoints().Num() == 0)
									{
										ClickedSplineActor->GetWorld()->EditorDestroyActor(ClickedSplineActor, true);
									}
								}
								SourceComponent->CheckSplinesValid();
								
								return true;
							}
						}

						FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_AddSegment", "Add Landscape Spline Segment"));
						for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
						{
							AddSegment(ControlPoint, ClickedControlPoint, bAutoRotateOnJoin, bAutoRotateOnJoin);
						}

						EdMode->AutoUpdateDirtyLandscapeSplines();
						GUnrealEd->RedrawLevelEditingViewports();

						return true;
					}
				}
			}
		}

		if (SelectedSplineControlPoints.Num() == 0)
		{
			if (InKey == EKeys::LeftMouseButton && InEvent == IE_Pressed
				&& IsCtrlDown(InViewport))
			{
				int32 HitX = InViewport->GetMouseX();
				int32 HitY = InViewport->GetMouseY();
				HHitProxy* HitProxy = InViewport->GetHitProxy(HitX, HitY);
				if (HitProxy)
				{
					ULandscapeSplineSegment* ClickedSplineSegment = nullptr;
					FTransform LandscapeToSpline;

					if (HitProxy->IsA(HLandscapeSplineProxy_Segment::StaticGetType()))
					{
						HLandscapeSplineProxy_Segment* SplineProxy = (HLandscapeSplineProxy_Segment*)HitProxy;
						ULandscapeSplinesComponent* SplineComponent = SplineProxy->SplineSegment->GetOuterULandscapeSplinesComponent();
						ILandscapeSplineInterface* SplineOwner = SplineComponent->GetSplineOwner();
						check(SplineOwner);
						if (SplineOwner->IsSplineOwnerValid())
						{
							ClickedSplineSegment = SplineProxy->SplineSegment;
							LandscapeToSpline = SplineOwner->LandscapeActorToWorld().GetRelativeTransform(SplineComponent->GetComponentTransform());
						}
					}
					else if (HitProxy->IsA(HActor::StaticGetType()))
					{
						HActor* ActorProxy = (HActor*)HitProxy;
						AActor* Actor = ActorProxy->Actor;
						const UMeshComponent* MeshComponent = Cast<UMeshComponent>(ActorProxy->PrimComponent);
						if (MeshComponent)
						{
							ULandscapeSplinesComponent* SplineComponent = Actor->FindComponentByClass<ULandscapeSplinesComponent>();
							if (SplineComponent)
							{
								UObject* ComponentOwner = SplineComponent->GetOwnerForMeshComponent(MeshComponent);
								if (ComponentOwner)
								{
									if (ULandscapeSplineSegment* SplineSegment = Cast<ULandscapeSplineSegment>(ComponentOwner))
									{
										// Find actual SplineComponent owner of the Segment (not the SplineComponent owner of the mesh)
										SplineComponent = SplineSegment->GetTypedOuter<ULandscapeSplinesComponent>();
										ILandscapeSplineInterface* SplineOwner = SplineComponent->GetSplineOwner();
										if (SplineOwner->IsSplineOwnerValid())
										{
											ClickedSplineSegment = SplineSegment;
											LandscapeToSpline = SplineOwner->LandscapeActorToWorld().GetRelativeTransform(SplineComponent->GetComponentTransform());
										}
									}
								}
							}
						}
					}

					if (ClickedSplineSegment != nullptr)
					{
						FVector HitLocation;
						if (EdMode->LandscapeMouseTrace(InViewportClient, HitLocation))
						{
							FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SplitSegment", "Split Landscape Spline Segment"));

							SplitSegment(ClickedSplineSegment, LandscapeToSpline.TransformPosition(HitLocation));

							GUnrealEd->RedrawLevelEditingViewports();
						}

						return true;
					}
				}
			}
		}

		if (InKey == EKeys::LeftMouseButton)
		{
			// Press mouse button
			if (InEvent == IE_Pressed)
			{
				// See if we clicked on a spline handle..
				int32 HitX = InViewport->GetMouseX();
				int32 HitY = InViewport->GetMouseY();
				HHitProxy*	HitProxy = InViewport->GetHitProxy(HitX, HitY);
				if (HitProxy)
				{
					if (HitProxy->IsA(HWidgetAxis::StaticGetType()))
					{
						// BeginTransform isn't called for Legacy gizmos, so we call it here for a common codepath with new gizmos
						const bool bIsDuplicatingControlPoints = InViewportClient->IsAltPressed();
						BeginTransformInternal(FGizmoState(), bIsDuplicatingControlPoints);

						return false; // We're not actually handling this case ourselves, just wrapping it in a transaction
					}
					else if (HitProxy->IsA(HLandscapeSplineProxy_Tangent::StaticGetType()))
					{
						HLandscapeSplineProxy_Tangent* SplineProxy = (HLandscapeSplineProxy_Tangent*)HitProxy;
						DraggingTangent_Segment = SplineProxy->SplineSegment;
						DraggingTangent_End = SplineProxy->End;
						DraggingTangent_Length = DraggingTangent_Segment->Connections[DraggingTangent_End].TangentLen;

						// Coord system MUST be set here, even if widget coord system space claims to already be in local space.
						DraggingTangent_CacheCoordSpace = InViewportClient->GetWidgetCoordSystemSpace();
						InViewportClient->SetWidgetCoordSystemSpace(ECoordSystem::COORD_Local);
						InViewportClient->SetRequiredCursorOverride(true, EMouseCursor::GrabHandClosed);

						GEditor->BeginTransaction(LOCTEXT("LandscapeSpline_ModifyTangent", "Modify Landscape Spline Tangent"));
						ULandscapeSplinesComponent* SplinesComponent = DraggingTangent_Segment->GetOuterULandscapeSplinesComponent();
						SplinesComponent->Modify();
						DraggingTangent_Segment->Modify();

						return false; // false to let FEditorViewportClient.InputKey start mouse tracking and enable InputDelta() so we can use it
					}
				}
			}
			else if (InEvent == IE_Released)
			{
				if (bMovingControlPoint)
				{
					EndTransform(FGizmoState());
				}
				else if (DraggingTangent_Segment)
				{
					DraggingTangent_Segment->UpdateSplinePoints(true);

					DraggingTangent_Segment = nullptr;

					InViewportClient->SetWidgetCoordSystemSpace(DraggingTangent_CacheCoordSpace);
					InViewportClient->SetRequiredCursorOverride(false);

					EdMode->AutoUpdateDirtyLandscapeSplines();
					GEditor->EndTransaction();

					return false; // false to let FEditorViewportClient.InputKey end mouse tracking
				}
			}
		}

		// To avoid updating while Ctrl+LMB / Ctrl+RMB+LMB, handle the case one button(s) are released
		if (InKey == EKeys::RightMouseButton && IsCtrlDown(InViewport) && InEvent == IE_Released && SelectedSplineControlPoints.Num() > 0)
		{
			EdMode->AutoUpdateDirtyLandscapeSplines();
		}

		return false;
	}

	virtual void ResetAllowDuplication()
	{
		bAllowDuplication = true;
		bDuplicatingControlPoint = false;
		bUpdatingAddSegment = false;
		DuplicateDelay = 0;
		DuplicateDelayAccumulatedDrag = FVector::ZeroVector;
		DuplicateCachedRotation = FRotator::ZeroRotator;
		DuplicateCacheSplitSegmentParam = 0.0f;
		DuplicateCacheSplitSegmentTangentLenStart = 0.0f;
		DuplicateCacheSplitSegmentTangentLenEnd = 0.0f;
		DuplicateCacheSplitSegmentTangentLen = 0.0f;
	}

	virtual bool DuplicateControlPoint(FVector& InDrag)
	{
		if (InDrag.IsZero())
		{
			return false;
		}

		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_DuplicatePoint", "Duplicate Landscape Spline Point"));

		ULandscapeSplineControlPoint* SelectedControlPoint = *SplineSelection->GetSelectedSplineControlPoints().CreateConstIterator();
		ULandscapeSplinesComponent* SplinesComponent = SelectedControlPoint->GetOuterULandscapeSplinesComponent();
		FVector LocalDrag = SplinesComponent->GetComponentTransform().InverseTransformVector(InDrag);

		ULandscapeSplineSegment* SegmentToSplit = nullptr;

		if (SelectedControlPoint->ConnectedSegments.Num() > 0)
		{
			bool bHasPrevAngle = false;
			float PrevAngle = 0.0f;

			for (const FLandscapeSplineConnection& Connection : SelectedControlPoint->ConnectedSegments)
			{
				ULandscapeSplineControlPoint* AdjacentControlPoint = Connection.Segment->Connections[Connection.End == 1 ? 0 : 1].ControlPoint;
				FVector SegmentDirection = AdjacentControlPoint->Location - SelectedControlPoint->Location;
				if (SegmentDirection.IsZero())
				{
					continue;
				}

				float CurrentAngle = static_cast<float>(FMath::Acos(FVector::DotProduct(LocalDrag, SegmentDirection) / (LocalDrag.Size() * SegmentDirection.Size())));

				// Create a new segment if there is no segment within 90 degrees of drag direction.
				// Otherwise split segment that is closest to the drag direction.
				if ((SelectedControlPoint->ConnectedSegments.Num() == 1 && CurrentAngle < HALF_PI) ||
					(SelectedControlPoint->ConnectedSegments.Num() > 1 && (!bHasPrevAngle || CurrentAngle < PrevAngle)))
				{
					SegmentToSplit = Connection.Segment;
				}

				bHasPrevAngle = true;
				PrevAngle = CurrentAngle;
			}
		}

		FVector Location = SelectedControlPoint->Location + LocalDrag;
		DuplicateCachedRotation = SelectedControlPoint->Rotation;

		if (SegmentToSplit)
		{
			SplitSegment(SegmentToSplit, Location);

			UE::Widget::EWidgetMode WidgetMode = EdMode->GetModeManager()->GetWidgetMode(); 
			SplineSelection->SelectControlPoint(SplinesComponent->ControlPoints.Last());
			EdMode->GetModeManager()->SetWidgetMode(WidgetMode);
		}
		else
		{
			AddControlPoint(SplinesComponent, Location);
			bUpdatingAddSegment = true;
			GUnrealEd->RedrawLevelEditingViewports();
		}

		// Get newly-created control point
		SelectedControlPoint = *SplineSelection->GetSelectedSplineControlPoints().CreateConstIterator();

		if (bAutoChangeConnectionsOnMove)
		{
			SelectedControlPoint->AutoSetConnections(true);
		}

		SelectedControlPoint->UpdateSplinePoints(false);

		return true;
	}

	// called when alt-dragging duplicated control point 
	virtual bool UpdateDuplicateControlPoint(FVector& InDrag)
	{
		ULandscapeSplineControlPoint* SelectedControlPoint = *SplineSelection->GetSelectedSplineControlPoints().CreateConstIterator();
		ULandscapeSplinesComponent* SplinesComponent = SelectedControlPoint->GetOuterULandscapeSplinesComponent();
		FVector LocalDrag = SplinesComponent->GetComponentTransform().InverseTransformVector(InDrag);
		FVector Location = SelectedControlPoint->Location + LocalDrag;

		if (bUpdatingAddSegment)
		{
			return UpdateAddSegment(SelectedControlPoint, Location);
		}

		return UpdateSplitSegment(SelectedControlPoint, Location);
	}

	virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const override
	{
		if (DraggingTangent_Segment)
		{
			bWantsOverride = true;
			bHardwareCursorVisible = true;
			bSoftwareCursorVisible = false;
			return true;
		}

		bWantsOverride = false;
		
		return false;
	}

	virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) override
	{
		if (DraggingTangent_Segment)
		{
			InViewportClient->SetWidgetModeOverride(UE::Widget::WM_Translate);
			InViewportClient->SetCurrentWidgetAxis(EAxisList::X);
			return true;
		}

		return false;
	}

	virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) override
	{
		if (DraggingTangent_Segment)
		{
			InViewportClient->SetWidgetModeOverride(UE::Widget::WM_Scale);
			InViewportClient->SetCurrentWidgetAxis(EAxisList::None);
			return true;
		}

		return false;
	}

	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override
	{
		FVector Drag = InDrag;

		if (DraggingTangent_Segment)
		{
			InViewportClient->SetRequiredCursorOverride(true, EMouseCursor::GrabHandClosed);

			const ULandscapeSplinesComponent* SplinesComponent = DraggingTangent_Segment->GetOuterULandscapeSplinesComponent();
			FLandscapeSplineSegmentConnection& Connection = DraggingTangent_Segment->Connections[DraggingTangent_End];

			FVector StartLocation; FRotator StartRotation;
			Connection.ControlPoint->GetConnectionLocationAndRotation(Connection.SocketName, StartLocation, StartRotation);
			FVector ForwardVector = FQuatRotationMatrix(StartRotation.Quaternion()).TransformVector(FVector(1.0f, 0.0f, 0.0f));

			FVector DragLocal = SplinesComponent->GetComponentTransform().InverseTransformVector(Drag);
			float Angle = static_cast<float>(FMath::Acos(FVector::DotProduct(DragLocal, ForwardVector) / DragLocal.Size()));
			float OldTangentLen = Connection.TangentLen;
			Connection.TangentLen = static_cast<float>(DraggingTangent_Length + (Angle < HALF_PI ? 2.0 : -2.0) * DragLocal.Size());

			// Disallow a tangent of exactly 0 and don't allow tangents to flip
			if ((Connection.TangentLen > 0 && OldTangentLen < 0) || 
				(Connection.TangentLen < 0 && OldTangentLen > 0) ||
				 Connection.TangentLen == 0)
			{
				if (OldTangentLen > 0)
				{
					Connection.TangentLen = SMALL_NUMBER;
				}
				else
				{
					Connection.TangentLen = -SMALL_NUMBER;
				}
			}

			// Flipping the tangent is only allowed if not using a socket
			if (Connection.SocketName != NAME_None)
			{
				Connection.TangentLen = FMath::Max(SMALL_NUMBER, Connection.TangentLen);
			}

			DraggingTangent_Segment->UpdateSplinePoints(false);

			return true;
		}

		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		if (SelectedSplineControlPoints.Num() == 1 && InViewportClient->IsAltPressed() && InViewportClient->GetWidgetMode() == UE::Widget::WM_Translate && InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
		{
			static const int MaxDuplicationDelay = 3;

			if (bAllowDuplication)
			{
				if (DuplicateDelay < MaxDuplicationDelay)
				{
					DuplicateDelay++;
					DuplicateDelayAccumulatedDrag += Drag;

					return true;
				}
				else
				{
					Drag += DuplicateDelayAccumulatedDrag;
					DuplicateDelayAccumulatedDrag = FVector::ZeroVector;
				}

				bAllowDuplication = false;
				bDuplicatingControlPoint = true;

				return DuplicateControlPoint(Drag);
			}
			else
			{
				return UpdateDuplicateControlPoint(Drag);
			}
		}

		if (SelectedSplineControlPoints.Num() > 0 && InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
		{
			for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
			{
				const ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();

				ControlPoint->Location += SplinesComponent->GetComponentTransform().InverseTransformVector(Drag);

				FVector RotAxis; float RotAngle;
				InRot.Quaternion().ToAxisAndAngle(RotAxis, RotAngle);
				RotAxis = (SplinesComponent->GetComponentTransform().GetRotation().Inverse() * ControlPoint->Rotation.Quaternion().Inverse()).RotateVector(RotAxis);

				// Hack: for some reason FQuat.Rotator() Clamps to 0-360 range, so use .GetNormalized() to recover the original negative rotation.
				ControlPoint->Rotation += FQuat(RotAxis, RotAngle).Rotator().GetNormalized();

				ControlPoint->Rotation.Yaw = FRotator::NormalizeAxis(ControlPoint->Rotation.Yaw);
				ControlPoint->Rotation.Pitch = FMath::Clamp(ControlPoint->Rotation.Pitch, -85.0f, 85.0f);
				ControlPoint->Rotation.Roll = FMath::Clamp(ControlPoint->Rotation.Roll, -85.0f, 85.0f);

				if (bAutoChangeConnectionsOnMove)
				{
					ControlPoint->AutoSetConnections(true);
				}

				ControlPoint->UpdateSplinePoints(false);
			}

			return true;
		}

		return false;
	}

	virtual bool BeginTransform(const FGizmoState& InState) override
	{
		const TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		const bool bShouldDuplicateSelectedControlPoints = 
			SelectedSplineControlPoints.Num() == 1 
			&& FSlateApplication::Get().GetModifierKeys().IsAltDown()
			&& InState.TransformMode == EGizmoTransformMode::Translate;
		return BeginTransformInternal(InState, bShouldDuplicateSelectedControlPoints);
	}

	bool BeginTransformInternal(const FGizmoState& InState, const bool bInIsDuplicating)
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		checkSlow(SelectedSplineControlPoints.Num() > 0);
		bMovingControlPoint = true;

		if (bInIsDuplicating)
		{
			GEditor->BeginTransaction(LOCTEXT("LandscapeSpline_DuplicateControlPoint", "Duplicate Landscape Spline Control Point"));
		}
		else
		{
			GEditor->BeginTransaction(LOCTEXT("LandscapeSpline_ModifyControlPoint", "Modify Landscape Spline Control Point"));
		}

		for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
		{
			ControlPoint->Modify();
			ControlPoint->GetOuterULandscapeSplinesComponent()->Modify();
		}

		return false;
	}

	virtual bool EndTransform(const FGizmoState& InState) override
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		bMovingControlPoint = false;

		for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
		{
			if (bDuplicatingControlPoint && bAutoRotateOnJoin)
			{
				ControlPoint->AutoCalcRotation(bAlwaysRotateForward);
			}

			ControlPoint->UpdateSplinePoints(true);
		}

		ResetAllowDuplication();

		EdMode->AutoUpdateDirtyLandscapeSplines();
		GEditor->EndTransaction();

		return false; // We're not actually handling this case ourselves, just wrapping it in a transaction
	}

	void OnUndo()
	{
		if (EdMode && EdMode->CurrentToolMode && EdMode->CurrentToolMode->CurrentToolName == "Splines" && SplineSelection)
		{
			SplineSelection->UpdatePropertiesWindows();
		}
	}

	virtual void EnterTool() override
	{
		GEditor->SelectNone(true, true, false);
		SplineSelection = NewObject<ULandscapeSplineSelection>(GetTransientPackage(), TEXT("LandscapeSplineSelection"), RF_Transactional);

		for (const FLandscapeListInfo& Info : EdMode->GetLandscapeList())
		{
			Info.Info->ForAllSplineActors([this](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
			{
				if (ULandscapeSplinesComponent* SplineComponent = SplineOwner->GetSplinesComponent())
				{
					SplineComponent->ShowSplineEditorMesh(true);
				}
			});
		}
	}

	virtual void ExitTool() override
	{
		SplineSelection->UpdatePropertiesWindows();

		for (const FLandscapeListInfo& Info : EdMode->GetLandscapeList())
		{
			Info.Info->ForAllSplineActors([this](TScriptInterface<ILandscapeSplineInterface> SplineOwner)
			{
				if (ULandscapeSplinesComponent* SplineComponent = SplineOwner->GetSplinesComponent())
				{
					SplineComponent->ShowSplineEditorMesh(false);
				}
			});
		}

		SplineSelection->ClearSelection();
		SplineSelection = nullptr;
	}

	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) override
	{
		// The editor can try to render the tool before the UpdateLandscapeEditorData command runs and the landscape editor realizes that the landscape has been hidden/deleted
		if (const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr)
		{
			for (ULandscapeSplineControlPoint* ControlPoint : SplineSelection->GetSelectedSplineControlPoints())
			{
				const ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();

				FVector HandlePos0 = SplinesComponent->GetComponentTransform().TransformPosition(ControlPoint->Location + ControlPoint->Rotation.Vector() * -20);
				FVector HandlePos1 = SplinesComponent->GetComponentTransform().TransformPosition(ControlPoint->Location + ControlPoint->Rotation.Vector() * 20);
				DrawDashedLine(PDI, HandlePos0, HandlePos1, FColor::White, 20, SDPG_Foreground);

				if (GLevelEditorModeTools().GetWidgetMode() == UE::Widget::WM_Scale && !Viewport->GetClient()->IsOrtho())
				{
					for (const FLandscapeSplineConnection& Connection : ControlPoint->ConnectedSegments)
					{
						FVector StartLocation; FRotator StartRotation;
						Connection.GetNearConnection().ControlPoint->GetConnectionLocationAndRotation(Connection.GetNearConnection().SocketName, StartLocation, StartRotation);

						FVector StartPos = SplinesComponent->GetComponentTransform().TransformPosition(StartLocation);
						FVector HandlePos = SplinesComponent->GetComponentTransform().TransformPosition(StartLocation + StartRotation.Vector() * Connection.GetNearConnection().TangentLen / 2);

						FColor TangentColor = (Connection.Segment == DraggingTangent_Segment && Connection.End == DraggingTangent_End) ? FColor::Yellow : FColor::White;
						PDI->DrawLine(StartPos, HandlePos, TangentColor, SDPG_Foreground);
						if (PDI->IsHitTesting()) PDI->SetHitProxy(new HLandscapeSplineProxy_Tangent(Connection.Segment, Connection.End));
						PDI->DrawPoint(HandlePos, TangentColor, 10.0f, SDPG_Foreground);
						if (PDI->IsHitTesting()) PDI->SetHitProxy(nullptr);
					}
				}
			}

			if (GLevelEditorModeTools().GetWidgetMode() == UE::Widget::WM_Scale && !Viewport->GetClient()->IsOrtho())
			{
				for (ULandscapeSplineSegment* Segment : SplineSelection->GetSelectedSplineSegments())
				{
					const ULandscapeSplinesComponent* SplinesComponent = Segment->GetOuterULandscapeSplinesComponent();
					for (int32 End = 0; End <= 1; End++)
					{
						const FLandscapeSplineSegmentConnection& Connection = Segment->Connections[End];

						FVector StartLocation; FRotator StartRotation;
						Connection.ControlPoint->GetConnectionLocationAndRotation(Connection.SocketName, StartLocation, StartRotation);

						FVector EndPos = SplinesComponent->GetComponentTransform().TransformPosition(StartLocation);
						FVector EndHandlePos = SplinesComponent->GetComponentTransform().TransformPosition(StartLocation + StartRotation.Vector() * Connection.TangentLen / 2);

						FColor TangentColor = (Segment == DraggingTangent_Segment && End == DraggingTangent_End) ? FColor::Yellow : FColor::White;
						PDI->DrawLine(EndPos, EndHandlePos, TangentColor, SDPG_Foreground);
						if (PDI->IsHitTesting()) PDI->SetHitProxy(new HLandscapeSplineProxy_Tangent(Segment, !!End));
						PDI->DrawPoint(EndHandlePos, TangentColor, 10.0f, SDPG_Foreground);
						if (PDI->IsHitTesting()) PDI->SetHitProxy(nullptr);
					}
				}
			}
		}
	}

	virtual bool OverrideSelection() const override
	{
		return true;
	}

	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const override
	{
		// Only filter selection not deselection
		if (bInSelection)
		{
			return false;
		}

		return true;
	}

	virtual bool UsesTransformWidget() const override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || DraggingTangent_Segment)
		{
			// The editor can try to render the transform widget before the landscape editor ticks and realizes that the landscape has been hidden/deleted
			return EdMode->CurrentToolTarget.LandscapeInfo.IsValid() && (EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() != nullptr);
		}

		return false;
	}

	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode CheckMode) const override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0)
		{
			//if (CheckMode == UE::Widget::WM_Rotate
			//	&& SplineSelection->GetSelectedSplineControlPoints().Num() >= 2)
			//{
			//	return AXIS_X;
			//}
			//else
			if (CheckMode != UE::Widget::WM_Scale)
			{
				return EAxisList::XYZ;
			}
			else
			{
				return EAxisList::None;
			}
		}

		return EAxisList::None;
	}

	virtual FVector GetWidgetLocation() const override
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		if (const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr)
		{
			if (DraggingTangent_Segment)
			{
				const FLandscapeSplineSegmentConnection& Connection = DraggingTangent_Segment->Connections[DraggingTangent_End];
				ULandscapeSplineControlPoint* ControlPoint = Connection.ControlPoint;
				ULandscapeSplinesComponent* SplinesComponent = ControlPoint->GetOuterULandscapeSplinesComponent();
				FVector StartLocation; FRotator StartRotation;
				ControlPoint->GetConnectionLocationAndRotation(Connection.SocketName, StartLocation, StartRotation);

				// Return tangent handle location.
				return SplinesComponent->GetComponentTransform().TransformPosition(StartLocation + StartRotation.Vector() * DraggingTangent_Length / 2);

			}
			else if (SelectedSplineControlPoints.Num() > 0)
			{
				ULandscapeSplineControlPoint* FirstPoint = *SelectedSplineControlPoints.CreateConstIterator();
				ULandscapeSplinesComponent* SplinesComponent = FirstPoint->GetOuterULandscapeSplinesComponent();
				return SplinesComponent->GetComponentTransform().TransformPosition(FirstPoint->Location);
			}
		}

		return FVector::ZeroVector;
	}

	virtual FMatrix GetWidgetRotation() const override
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();

		const ALandscapeProxy* LandscapeProxy = EdMode->CurrentToolTarget.LandscapeInfo.IsValid() ? EdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy() : nullptr;
		if (LandscapeProxy)
		{
			if (DraggingTangent_Segment)
			{
				const FLandscapeSplineSegmentConnection& Connection = DraggingTangent_Segment->Connections[DraggingTangent_End];
				ULandscapeSplinesComponent* SplinesComponent = Connection.ControlPoint->GetOuterULandscapeSplinesComponent();
				FVector StartLocation; FRotator StartRotation;
				Connection.ControlPoint->GetConnectionLocationAndRotation(Connection.SocketName, StartLocation, StartRotation);
				return FQuatRotationTranslationMatrix(StartRotation.Quaternion() * SplinesComponent->GetComponentTransform().GetRotation(), FVector::ZeroVector);
			}
			else if (SelectedSplineControlPoints.Num() > 0)
			{
				ULandscapeSplineControlPoint* FirstPoint = *SelectedSplineControlPoints.CreateConstIterator();
				ULandscapeSplinesComponent* SplinesComponent = FirstPoint->GetOuterULandscapeSplinesComponent();
				return FQuatRotationTranslationMatrix(FirstPoint->Rotation.Quaternion() * SplinesComponent->GetComponentTransform().GetRotation(), FVector::ZeroVector);
			}
		}

		return FMatrix::Identity;
	}

	virtual EEditAction::Type GetActionEditDuplicate() override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			return EEditAction::Process;
		}

		return EEditAction::Skip;
	}

	virtual EEditAction::Type GetActionEditDelete() override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			return EEditAction::Process;
		}

		return EEditAction::Skip;
	}

	virtual EEditAction::Type GetActionEditCut() override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			return EEditAction::Process;
		}

		return EEditAction::Skip;
	}

	virtual EEditAction::Type GetActionEditCopy() override
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			return EEditAction::Process;
		}

		return EEditAction::Skip;
	}

	virtual EEditAction::Type GetActionEditPaste() override
	{
		FString PasteString;
		FPlatformApplicationMisc::ClipboardPaste(PasteString);
		if (PasteString.StartsWith(FLandscapeSplineTextObjectFactory::SplineBeginTag))
		{
			return EEditAction::Process;
		}

		return EEditAction::Skip;
	}

	virtual bool ProcessEditDuplicate() override
	{
		InternalProcessEditDuplicate();
		return true;
	}

	virtual bool ProcessEditDelete() override
	{
		InternalProcessEditDelete();
		return true;
	}

	virtual bool ProcessEditCut() override
	{
		InternalProcessEditCut();
		return true;
	}

	virtual bool ProcessEditCopy() override
	{
		InternalProcessEditCopy();
		return true;
	}

	virtual bool ProcessEditPaste() override
	{
		InternalProcessEditPaste();
		return true;
	}

	void InternalProcessEditDuplicate()
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_Duplicate", "Duplicate Landscape Splines"));

			FString Data;
			InternalProcessEditCopy(&Data);
			InternalProcessEditPaste(&Data, true);
		}
	}

	void InternalProcessEditDelete()
	{
		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();
		TArray<ULandscapeSplineSegment*> SelectedSplineSegments = SplineSelection->GetSelectedSplineSegments();
		TSet<ULandscapeSplinesComponent*> AffectedSplineComponents;

		if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_Delete", "Delete Landscape Splines"));

			for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
			{
				AffectedSplineComponents.Add(ControlPoint->GetOuterSafe());
				DeleteControlPoint(ControlPoint, bDeleteLooseEnds);
			}
			for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
			{
				AffectedSplineComponents.Add(Segment->GetOuterSafe());
				DeleteSegment(Segment, bDeleteLooseEnds);
			}
			SplineSelection->ClearSelection();
		}

		for (ULandscapeSplinesComponent* Component : AffectedSplineComponents)
		{
			// Could have nulls here because deleting control points might have already deleted some segments.
			if (Component)
			{
				Component->CheckSplinesValid();
			}
		}
	}

	void InternalProcessEditCut()
	{
		if (SplineSelection->GetSelectedSplineControlPoints().Num() > 0 || SplineSelection->GetSelectedSplineSegments().Num() > 0)
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_Cut", "Cut Landscape Splines"));

			InternalProcessEditCopy();
			InternalProcessEditDelete();
		}
	}

	void InternalProcessEditCopy(FString* OutData = nullptr)
	{
		bool bFirstSplineLocation = true;
		FVector SplineLocation;

		auto GetSplineLocation = [&bFirstSplineLocation, &SplineLocation](ULandscapeSplineControlPoint* ControlPoint, const FVector& Location)
		{
			if (bFirstSplineLocation)
			{
				ILandscapeSplineInterface* SplineOwner = ControlPoint->GetOuterULandscapeSplinesComponent()->GetSplineOwner();
				const FTransform LocalToWorld = ControlPoint->GetOuterULandscapeSplinesComponent()->GetComponentTransform();
				SplineLocation = LocalToWorld.TransformPosition(ControlPoint->Location);
				bFirstSplineLocation = false;
			}
		};

		TArray<ULandscapeSplineControlPoint*> SelectedSplineControlPoints = SplineSelection->GetSelectedSplineControlPoints();
		TArray<ULandscapeSplineSegment*> SelectedSplineSegments = SplineSelection->GetSelectedSplineSegments();

		if (SelectedSplineControlPoints.Num() > 0 || SelectedSplineSegments.Num() > 0)
		{
			TArray<UObject*> Objects;
			Objects.Reserve(SelectedSplineControlPoints.Num() + SelectedSplineSegments.Num() * 3); // worst case

			// Control Points then segments
			for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
			{
				GetSplineLocation(ControlPoint, ControlPoint->Location);
				Objects.Add(ControlPoint);
			}
			for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
			{
				GetSplineLocation(Segment->Connections[0].ControlPoint, Segment->Connections[0].ControlPoint->Location);
				Objects.AddUnique(Segment->Connections[0].ControlPoint);
				Objects.AddUnique(Segment->Connections[1].ControlPoint);
			}
			for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
			{
				Objects.Add(Segment);
			}

			// Perform export to text format
			FStringOutputDevice Ar;
			const FExportObjectInnerContext Context;

			Ar.Logf(TEXT("%s\r\n"), *FLandscapeSplineTextObjectFactory::SplineBeginTag);
			Ar.Logf(TEXT("%s%s\r\n"), *FLandscapeSplineTextObjectFactory::SplineLocationTag, *SplineLocation.ToString());
			for (UObject* Object : Objects)
			{
				UExporter::ExportToOutputDevice(&Context, Object, nullptr, Ar, TEXT("copy"), 3, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, false, Object->GetOuter());
			}
			Ar.Logf(TEXT("%s\r\n"), *FLandscapeSplineTextObjectFactory::SplineEndTag);

			if (OutData != nullptr)
			{
				*OutData = MoveTemp(Ar);
			}
			else
			{
				FPlatformApplicationMisc::ClipboardCopy(*Ar);
			}
		}
	}

	void InternalProcessEditPaste(FString* InData = nullptr, bool bOffset = false)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_Paste", "Paste Landscape Splines"));

		TScriptInterface<ILandscapeSplineInterface> SplineOwner;
		const bool bGridBased = EdMode->IsGridBased();
		if (bGridBased)
		{
			if (ULandscapeInfo* CurrentLandscapeInfo = EdMode->CurrentToolTarget.LandscapeInfo.Get())
			{
				SplineOwner = CurrentLandscapeInfo->CreateSplineActor(FVector::ZeroVector);
			}
		}
		else
		{
			SplineOwner = EdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true);
		}

		if (!SplineOwner)
		{
			return;
		}
		
		ULandscapeSplinesComponent* SplineComponent = SplineOwner->GetSplinesComponent();
		if (!SplineComponent)
		{
			SplineOwner->CreateSplineComponent();
			SplineComponent = SplineOwner->GetSplinesComponent();
			check(SplineComponent);
		}
		
		SplineComponent->Modify();

		const TCHAR* Data = nullptr;
		FString PasteString;
		if (InData != nullptr)
		{
			Data = **InData;
		}
		else
		{
			FPlatformApplicationMisc::ClipboardPaste(PasteString);
			Data = *PasteString;
		}

		FLandscapeSplineTextObjectFactory Factory;
		TArray<UObject*> OutObjects = Factory.ImportSplines(SplineComponent, Data);

		if (ALandscapeSplineActor* SplineActor = Cast<ALandscapeSplineActor>(SplineOwner.GetObject()))
		{
			SplineActor->SetActorLocation(Factory.SplineLocation);
		}

		if (bOffset)
		{
			for (UObject* Object : OutObjects)
			{
				ULandscapeSplineControlPoint* ControlPoint = Cast<ULandscapeSplineControlPoint>(Object);
				if (ControlPoint != nullptr)
				{
					ControlPoint->Location += FVector(500, 500, 0);
					ControlPoint->UpdateSplinePoints();
				}
			}
		}

		SplineComponent->CheckSplinesValid();
	}

protected:
	// Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override { OnUndo(); }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

protected:
	FEdModeLandscape* EdMode;
	TObjectPtr<ULandscapeInfo> LandscapeInfo;

	/** Tracks the selected points/segments and handles linear spline navigation */
	TObjectPtr<ULandscapeSplineSelection> SplineSelection;

	TObjectPtr<ULandscapeSplineSegment> DraggingTangent_Segment;
	float DraggingTangent_Length;
	ECoordSystem DraggingTangent_CacheCoordSpace;
	uint32 DraggingTangent_End : 1;

	uint32 bMovingControlPoint : 1;

	uint32 bAutoRotateOnJoin : 1;
	uint32 bAlwaysRotateForward : 1;
	uint32 bAutoChangeConnectionsOnMove : 1;
	uint32 bDeleteLooseEnds : 1;
	uint32 bCopyMeshToNewControlPoint : 1;

	/** Alt-drag: True when control point may be duplicated. */
	uint32 bAllowDuplication : 1;

	/** Alt-drag: True when in process of duplicating a control point. */
	uint32 bDuplicatingControlPoint : 1;

	/** Alt-drag: True when in process of adding end segment. */
	uint32 bUpdatingAddSegment : 1;

	/** Alt-drag: Delays duplicating control point to accumulate sufficient drag input offset. */
	uint32 DuplicateDelay;

	/** Alt-drag: Accumulates delayed drag offset. */
	FVector DuplicateDelayAccumulatedDrag;

	/** Alt-drag: Cached control point rotation when adding new control point at end of the spline. */
	FRotator DuplicateCachedRotation;

	/** Alt-drag: Cached segment parameter for split segment at new control point */
	float DuplicateCacheSplitSegmentParam;

	/** Alt-drag: Cached pre-split segment start tangent length. */
	float DuplicateCacheSplitSegmentTangentLenStart;

	/** Alt-drag: Cached pre-split segment end tangent length. */
	float DuplicateCacheSplitSegmentTangentLenEnd;

	/** Alt-drag: Cached tangent length for split segment at new control point. */
	float DuplicateCacheSplitSegmentTangentLen;

	friend FEdModeLandscape;
};


bool FEdModeLandscape::HasSelectedSplineSegments() const
{
	return SplinesTool && (SplinesTool->SplineSelection->GetSelectedSplineSegments().Num() > 0);
}

bool FEdModeLandscape::HasAdjacentLinearSplineConnection(ESplineNavigationFlags Flags) const
{
	if (!SplinesTool)
	{
		return false;
	}

	// check only one selection mode is set
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled))
	{
		return SplinesTool->SplineSelection->HasAdjacentSegmentInLinearPath(Flags);
	}
	else
	{
		return SplinesTool->SplineSelection->HasAdjacentControlPointInLinearPath(Flags);
	}
}

bool FEdModeLandscape::HasEndLinearSplineConnection(ESplineNavigationFlags Flags) const
{
	if (!SplinesTool)
	{
		return false;
	}
	// check only one selection mode is set
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled))
	{
		const ULandscapeSplineSegment* EndSegment = SplinesTool->SplineSelection->GetEndSegmentInLinearPath(Flags);
		return EndSegment != nullptr && !EndSegment->IsSplineSelected() && SplinesTool->SplineSelection->IsSelectionValidForNavigation();
	}
	else
	{
		const ULandscapeSplineControlPoint* EndControlPoint = SplinesTool->SplineSelection->GetEndControlPointInLinearPath(Flags);
		return EndControlPoint != nullptr && !EndControlPoint->IsSplineSelected() && SplinesTool->SplineSelection->IsSelectionValidForNavigation();
	}
}

void FEdModeLandscape::FlipSelectedSplineSegments()
{
	if (!SplinesTool)
	{
		return;
	}

	// Do Flip
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_FlipSegments", "Flip Selected Landscape Spline Segments"));
	SplinesTool->FlipSelectedSplineSegments();
}

void FEdModeLandscape::ShowSplineProperties()
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->ShowSplineProperties();
	}
}

void FEdModeLandscape::GetSelectedSplineOwners(TSet<AActor*>& SelectedSplineOwners) const
{
	for (ULandscapeSplineSegment* Segment : SplinesTool->SplineSelection->GetSelectedSplineSegments())
	{
		SelectedSplineOwners.Add(Segment->GetTypedOuter<AActor>());
	}

	for (ULandscapeSplineControlPoint* ControlPoint : SplinesTool->SplineSelection->GetSelectedSplineControlPoints())
	{
		SelectedSplineOwners.Add(ControlPoint->GetTypedOuter<AActor>());
	}
}

void FEdModeLandscape::SelectAllSplineControlPoints(ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::AddToSelection));

	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectAllPoints", "Select All Landscape Spline Points"));

	ULandscapeInfo* CurrentLandscapeInfo = CurrentToolTarget.LandscapeInfo.Get();
	if (SplinesTool && CurrentLandscapeInfo)
	{
		if (!EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
		{
			SplinesTool->SplineSelection->ClearSelection();
		}
		SplinesTool->SplineSelection->SelectAllControlPoints(*CurrentLandscapeInfo);
		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectAllSplineSegments(ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::AddToSelection));

	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectAllSegments", "Select All Landscape Spline Segments"));

	ULandscapeInfo* CurrentLandscapeInfo = CurrentToolTarget.LandscapeInfo.Get();
	if (SplinesTool && CurrentLandscapeInfo)
	{
		if (!EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
		{
			SplinesTool->SplineSelection->ClearSelection();
		}
		SplinesTool->SplineSelection->SelectAllSplineSegments(*CurrentLandscapeInfo);
		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectAllConnectedSplineControlPoints()
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectConnectedPoints", "Select Landscape Spline Connected Points"));

	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->SplineSelection->SelectAdjacentControlPoints();
		SplinesTool->SplineSelection->ClearSelectedSegments();
		SplinesTool->SplineSelection->SelectConnected();

		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectAllConnectedSplineSegments()
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectConnectedSegments", "Select Landscape Spline Connected Segments"));

	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->SplineSelection->SelectAdjacentSegments();
		SplinesTool->SplineSelection->ClearSelectedControlPoints();
		SplinesTool->SplineSelection->SelectConnected();

		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectAdjacentLinearSplineElement(ESplineNavigationFlags Flags) const
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		// check only one selection mode is set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);
		const bool bAddToSelection = EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection);

		if (EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled))
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectAdjacentSegment", "Select Landscape Spline Segment"));
			ULandscapeSplineSegment* AdjacentSegment = SplinesTool->SplineSelection->GetAdjacentSegmentInLinearPath(Flags);

			// SelectSegment will reset selection unless AddToSelection flag is passed
			SplinesTool->SplineSelection->SelectSegment(AdjacentSegment, bAddToSelection ? ESplineNavigationFlags::AddToSelection : ESplineNavigationFlags::None);

			// When AddToSelection is set, SelectSegment will not clear control points
			SplinesTool->SplineSelection->ClearSelectedControlPoints();
		}
		else 
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectAdjacentPoint", "Select Landscape Spline Point"));
			ULandscapeSplineControlPoint* AdjacentPoint = SplinesTool->SplineSelection->GetAdjacentControlPointInPath(Flags);

			// SelectSegment will reset selection unless AddToSelection flag is passed
			SplinesTool->SplineSelection->SelectControlPoint(AdjacentPoint, bAddToSelection ? ESplineNavigationFlags::AddToSelection : ESplineNavigationFlags::None);

			// When AddToSelection is set, SelectControlPoint will not clear Segments
			SplinesTool->SplineSelection->ClearSelectedSegments();
		}
		
		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectEndLinearSplineElement(ESplineNavigationFlags Flags) const
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		// check only one selection mode is set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

		if (EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled))
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectEndSegment", "Select Landscape Spline Segment"));

			ULandscapeSplineSegment* EndSegment = SplinesTool->SplineSelection->GetEndSegmentInLinearPath(Flags);
			if (EndSegment != nullptr)
			{
				SplinesTool->SplineSelection->SelectSegment(EndSegment, ESplineNavigationFlags::UpdatePropertiesWindows);
			}
		}
		else
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_SelectEndPoint", "Select Landscape Spline Point"));

			ULandscapeSplineControlPoint* EndPoint = SplinesTool->SplineSelection->GetEndControlPointInLinearPath(Flags);
			if (EndPoint != nullptr)
			{
				SplinesTool->SplineSelection->SelectControlPoint(EndPoint, ESplineNavigationFlags::UpdatePropertiesWindows);
			}
		}
	}
}

void FEdModeLandscape::SelectSplineControlPointsFromCurrentSegmentSelection() const
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_ConvertToPoints", "Switch selected Segments to Points"));

	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->SplineSelection->SelectAdjacentControlPoints();
		SplinesTool->SplineSelection->ClearSelectedSegments();
		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SelectSplineSegmentsFromCurrentControlPointSelection() const
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_ConvertToSegments", "Switch selected Points to Segments"));

	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->SplineSelection->SelectAdjacentSegments();
		SplinesTool->SplineSelection->ClearSelectedControlPoints();
		SplinesTool->SplineSelection->UpdatePropertiesWindows();
	}
}

void FEdModeLandscape::SplineMoveToCurrentLevel()
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_MoveToCurrentLevel", "Move Landscape Spline to current level"));

	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		// Select all connected control points
		SplinesTool->SplineSelection->SelectAdjacentSegments();
		SplinesTool->SplineSelection->SelectAdjacentControlPoints();
		SplinesTool->SplineSelection->SelectConnected();

		SplinesTool->MoveSelectedToLevel();

		SplinesTool->SplineSelection->ClearSelection();
	}
}

bool FEdModeLandscape::CanMoveSplineToCurrentLevel() const
{
	if (SplinesTool)
	{
		return SplinesTool->CanMoveSelectedToLevel();
	}

	return false;
}

void FEdModeLandscape::UpdateSplineMeshLevels()
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeSpline_UpdateSplineMeshes", "Update Spline Meshes Level"));

	if(SplinesTool)
	{
		SelectAllConnectedSplineSegments();

		SplinesTool->UpdateSplineMeshLevels();
	}
}

void FEdModeLandscape::SetbUseAutoRotateOnJoin(bool InbAutoRotateOnJoin)
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->bAutoRotateOnJoin = InbAutoRotateOnJoin;
	}
}

bool FEdModeLandscape::GetbUseAutoRotateOnJoin()
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		return SplinesTool->bAutoRotateOnJoin;
	}
	return true; // default value
}

void FEdModeLandscape::SetbAlwaysRotateForward(bool InbAlwaysRotateForward)
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		SplinesTool->bAlwaysRotateForward = InbAlwaysRotateForward;
	}
}

bool FEdModeLandscape::GetbAlwaysRotateForward()
{
	if (SplinesTool /*&& SplinesTool == CurrentTool*/)
	{
		return SplinesTool->bAlwaysRotateForward;
	}
	return false; // default value
}

void FEdModeLandscape::InitializeTool_Splines()
{
	auto Tool_Splines = MakeUnique<FLandscapeToolSplines>(this);
	Tool_Splines->ValidBrushes.Add("BrushSet_Splines");
	SplinesTool = Tool_Splines.Get();
	LandscapeTools.Add(MoveTemp(Tool_Splines));
}

#undef LOCTEXT_NAMESPACE
