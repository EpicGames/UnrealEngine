// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/GCObject.h"
#include "LandscapeProxy.h"
#include "LandscapeToolInterface.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditTypes.h"
#include "EditorViewportClient.h"
#include "LandscapeEdit.h"
#include "LandscapeComponent.h"
#include "LandscapeDataAccess.h"
#include "LandscapeEditorObject.h"
#include "LandscapeHeightfieldCollisionComponent.h"
#include "InstancedFoliageActor.h"
#include "AI/NavigationSystemBase.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "LandscapeEditorPrivate.h"
#include "Logging/LogMacros.h"
#include "VisualLogger/VisualLogger.h"


//
//	FNoiseParameter - Perlin noise
//
struct FNoiseParameter
{
	float Base;
	float NoiseScale;
	float NoiseAmount;

	// Constructors.

	FNoiseParameter()
	{
	}
	FNoiseParameter(float InBase, float InScale, float InAmount) :
		Base(InBase),
		NoiseScale(InScale),
		NoiseAmount(InAmount)
	{
	}

	// Sample
	float Sample(int32 X, int32 Y) const
	{
		float	Noise = 0.0f;
		X = FMath::Abs(X);
		Y = FMath::Abs(Y);

		if (NoiseScale > DELTA)
		{
			for (uint32 Octave = 0; Octave < 4; Octave++)
			{
				float	OctaveShift = static_cast<float>(1 << Octave);
				float	OctaveScale = OctaveShift / NoiseScale;
				Noise += PerlinNoise2D(X * OctaveScale, Y * OctaveScale) / OctaveShift;
			}
		}

		return Base + Noise * NoiseAmount;
	}

	// TestGreater - Returns 1 if TestValue is greater than the parameter.
	bool TestGreater(int32 X, int32 Y, float TestValue) const
	{
		float	ParameterValue = Base;

		if (NoiseScale > DELTA)
		{
			for (uint32 Octave = 0; Octave < 4; Octave++)
			{
				float	OctaveShift = static_cast<float>(1 << Octave);
				float	OctaveAmplitude = NoiseAmount / OctaveShift;

				// Attempt to avoid calculating noise if the test value is outside of the noise amplitude.

				if (TestValue > ParameterValue + OctaveAmplitude)
					return 1;
				else if (TestValue < ParameterValue - OctaveAmplitude)
					return 0;
				else
				{
					float	OctaveScale = OctaveShift / NoiseScale;
					ParameterValue += PerlinNoise2D(X * OctaveScale, Y * OctaveScale) * OctaveAmplitude;
				}
			}
		}

		return TestValue >= ParameterValue;
	}

	// TestLess
	bool TestLess(int32 X, int32 Y, float TestValue) const { return !TestGreater(X, Y, TestValue); }

private:
	static const int32 Permutations[256];

	bool operator==(const FNoiseParameter& SrcNoise)
	{
		if ((Base == SrcNoise.Base) &&
			(NoiseScale == SrcNoise.NoiseScale) &&
			(NoiseAmount == SrcNoise.NoiseAmount))
		{
			return true;
		}

		return false;
	}

	void operator=(const FNoiseParameter& SrcNoise)
	{
		Base = SrcNoise.Base;
		NoiseScale = SrcNoise.NoiseScale;
		NoiseAmount = SrcNoise.NoiseAmount;
	}


	float Fade(float T) const
	{
		return T * T * T * (T * (T * 6 - 15) + 10);
	}


	float Grad(int32 Hash, float X, float Y) const
	{
		int32		H = Hash & 15;
		float	U = H < 8 || H == 12 || H == 13 ? X : Y,
			V = H < 4 || H == 12 || H == 13 ? Y : 0;
		return ((H & 1) == 0 ? U : -U) + ((H & 2) == 0 ? V : -V);
	}

	float PerlinNoise2D(float X, float Y) const
	{
		int32		TruncX = FMath::TruncToInt(X),
			TruncY = FMath::TruncToInt(Y),
			IntX = TruncX & 255,
			IntY = TruncY & 255;
		float	FracX = X - TruncX,
			FracY = Y - TruncY;

		float	U = Fade(FracX),
			V = Fade(FracY);

		int32	A = Permutations[IntX] + IntY,
			AA = Permutations[A & 255],
			AB = Permutations[(A + 1) & 255],
			B = Permutations[(IntX + 1) & 255] + IntY,
			BA = Permutations[B & 255],
			BB = Permutations[(B + 1) & 255];

		return	FMath::Lerp(FMath::Lerp(Grad(Permutations[AA], FracX, FracY),
			Grad(Permutations[BA], FracX - 1, FracY), U),
			FMath::Lerp(Grad(Permutations[AB], FracX, FracY - 1),
			Grad(Permutations[BB], FracX - 1, FracY - 1), U), V);
	}
};



#if WITH_KISSFFT
#include "tools/kiss_fftnd.h"
#endif

template<typename DataType>
inline void LowPassFilter(int32 X1, int32 Y1, int32 X2, int32 Y2, FLandscapeBrushData& BrushInfo, TArray<DataType>& Data, const float DetailScale, const float ApplyRatio = 1.0f)
{
#if WITH_KISSFFT
	// Low-pass filter
	int32 FFTWidth = X2 - X1 - 1;
	int32 FFTHeight = Y2 - Y1 - 1;

	if (FFTWidth <= 1 && FFTHeight <= 1)
	{
		// nothing to do
		return;
	}

	const int32 NDims = 2;
	const int32 Dims[NDims] = { FFTHeight, FFTWidth };
	kiss_fftnd_cfg stf = kiss_fftnd_alloc(Dims, NDims, 0, NULL, NULL),
		sti = kiss_fftnd_alloc(Dims, NDims, 1, NULL, NULL);

	kiss_fft_cpx *buf = (kiss_fft_cpx *)KISS_FFT_MALLOC(sizeof(kiss_fft_cpx) * Dims[0] * Dims[1]);
	kiss_fft_cpx *out = (kiss_fft_cpx *)KISS_FFT_MALLOC(sizeof(kiss_fft_cpx) * Dims[0] * Dims[1]);

	for (int32 Y = Y1 + 1; Y <= Y2 - 1; Y++)
	{
		const typename TArray<DataType>::ElementType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
		kiss_fft_cpx* BufScanline = buf + (Y - (Y1 + 1)) * Dims[1] + (0 - (X1 + 1));

		for (int32 X = X1 + 1; X <= X2 - 1; X++)
		{
			BufScanline[X].r = DataScanline[X];
			BufScanline[X].i = 0;
		}
	}

	// Forward FFT
	kiss_fftnd(stf, buf, out);

	int32 CenterPos[2] = { Dims[0] >> 1, Dims[1] >> 1 };
	for (int32 Y = 0; Y < Dims[0]; Y++)
	{
		float DistFromCenter;
		for (int32 X = 0; X < Dims[1]; X++)
		{
			if (Y < CenterPos[0])
			{
				if (X < CenterPos[1])
				{
					// 1
					DistFromCenter = static_cast<float>(X*X + Y*Y);
				}
				else
				{
					// 2
					DistFromCenter = static_cast<float>((X - Dims[1])*(X - Dims[1]) + Y*Y);
				}
			}
			else
			{
				if (X < CenterPos[1])
				{
					// 3
					DistFromCenter = static_cast<float>(X*X + (Y - Dims[0])*(Y - Dims[0]));
				}
				else
				{
					// 4
					DistFromCenter = static_cast<float>((X - Dims[1])*(X - Dims[1]) + (Y - Dims[0])*(Y - Dims[0]));
				}
			}
			// High frequency removal
			float Ratio = 1.0f - DetailScale;
			float Dist = FMath::Min<float>((Dims[0] * Ratio)*(Dims[0] * Ratio), (Dims[1] * Ratio)*(Dims[1] * Ratio));
			float Filter = 1.0f / (1.0f + DistFromCenter / Dist);
			CA_SUPPRESS(6385);
			out[X + Y*Dims[1]].r *= Filter;
			out[X + Y*Dims[1]].i *= Filter;
		}
	}

	// Inverse FFT
	kiss_fftnd(sti, out, buf);

	const float Scale = static_cast<float>(Dims[0] * Dims[1]);
	const int32 BrushX1 = FMath::Max<int32>(BrushInfo.GetBounds().Min.X, X1 + 1);
	const int32 BrushY1 = FMath::Max<int32>(BrushInfo.GetBounds().Min.Y, Y1 + 1);
	const int32 BrushX2 = FMath::Min<int32>(BrushInfo.GetBounds().Max.X, X2);
	const int32 BrushY2 = FMath::Min<int32>(BrushInfo.GetBounds().Max.Y, Y2);
	for (int32 Y = BrushY1; Y < BrushY2; Y++)
	{
		const float* BrushScanline = BrushInfo.GetDataPtr(FIntPoint(0, Y));
		typename TArray<DataType>::ElementType* DataScanline = Data.GetData() + (Y - Y1) * (X2 - X1 + 1) + (0 - X1);
		const kiss_fft_cpx* BufScanline = buf + (Y - (Y1 + 1)) * Dims[1] + (0 - (X1 + 1));

		for (int32 X = BrushX1; X < BrushX2; X++)
		{
			const float BrushValue = BrushScanline[X];

			if (BrushValue > 0.0f)
			{
				DataScanline[X] = static_cast<DataType>(FMath::Lerp(static_cast<float>(DataScanline[X]), BufScanline[X].r / Scale, BrushValue * ApplyRatio));
			}
		}
	}

	// Free FFT allocation
	KISS_FFT_FREE(stf);
	KISS_FFT_FREE(sti);
	KISS_FFT_FREE(buf);
	KISS_FFT_FREE(out);
#endif
}



//
// TLandscapeEditCache
//
template<class Accessor, typename AccessorType>
struct TLandscapeEditCache
{
public:
	typedef AccessorType DataType;
	typedef Accessor AccessorClass;

	Accessor DataAccess;
	ULandscapeLayerInfoObject* LayerInfo = nullptr;

	TLandscapeEditCache(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo)
		: DataAccess(InLandscapeInfo, InLayerInfo)
		, LayerInfo(InLayerInfo)
		, LandscapeInfo(InLandscapeInfo)
	{
		check(LandscapeInfo != nullptr);
	}

	// X2/Y2 Coordinates are "inclusive" max values
	// ...but can extend outside valid landscape coords.  GetData/GetDataFast will generate and store values for out-of-range coords and then return coords clamped to the valid range.
	// Note that this should maybe be called "ExtendDataCache" because the region here will be combined with the existing cached region, not loaded independently, giving a cached region that is the bounding box of previous and new
	void CacheData(int32 X1, int32 Y1, int32 X2, int32 Y2, bool bCacheOriginalData = false)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_CacheData);

		if (!bIsValid)
		{
			if (Accessor::bUseInterp)
			{
				// GetData alters its args, so make temp copies to avoid screwing things up
				int32 ValidX1 = X1;
				int32 ValidY1 = Y1;
				int32 ValidX2 = X2;
				int32 ValidY2 = Y2;
				DataAccess.GetData(ValidX1, ValidY1, ValidX2, ValidY2, CachedData);

				if (ValidX1 > ValidX2 || ValidY1 > ValidY2)
				{
					// there was no data in the requested region
					bIsValid = false;
					return;
				}
			}
			else
			{
				DataAccess.GetDataFast(X1, Y1, X2, Y2, CachedData);
			}
			CachedX1 = X1;
			CachedY1 = Y1;
			CachedX2 = X2;
			CachedY2 = Y2;

			// Drop a visual log to indicate the area covered by this cache region extension :
			VisualizeLandscapeRegion(CachedX1, CachedY1, CachedX2, CachedY2, FColor::Red, FString::Printf(TEXT("Cache Data (X1:%i, Y1:%i, X2:%i, Y2:%i)"), CachedX1, CachedY1, CachedX2, CachedY2));

			if (bCacheOriginalData)
			{
				OriginalData = CachedData;
			}
			bIsValid = true;
		}
		else
		{
			auto ExtendCache = [this, bCacheOriginalData](const int32 InX1, const int32 InY1, const int32 InX2, const int32 InY2)
			{
				if (Accessor::bUseInterp)
				{
					// GetData alters its args, so make temp copies to avoid screwing things up :
					int32 ValidX1 = InX1;
					int32 ValidY1 = InY1;
					int32 ValidX2 = InX2;
					int32 ValidY2 = InY2;
					DataAccess.GetData(ValidX1, ValidY1, ValidX2, ValidY2, CachedData);
				}
				else
				{
					DataAccess.GetDataFast(InX1, InY1, InX2, InY2, CachedData);
				}

				if (bCacheOriginalData)
				{
					CacheOriginalData(InX1, InY1, InX2, InY2);
					// Drop a visual log to indicate the area covered by this cache region extension :
					VisualizeLandscapeRegion(InX1, InY1, InX2, InY2, FColor::Purple, FString::Printf(TEXT("Cache Original Data (X1:%i, Y1:%i, X2:%i, Y2:%i)"), InX1, InY1, InX2, InY2), /*InZOffset = */10.0);
				}
			};

			// Extend the cache area if needed
			bool bCacheExtended = false;
			if (X1 < CachedX1)
			{
				ExtendCache(X1, CachedY1, CachedX1 - 1, CachedY2);
				CachedX1 = X1;
				bCacheExtended = true;
			}

			if (X2 > CachedX2)
			{
				ExtendCache(CachedX2 + 1, CachedY1, X2, CachedY2);
				CachedX2 = X2;
				bCacheExtended = true;
			}

			if (Y1 < CachedY1)
			{
				ExtendCache(CachedX1, Y1, CachedX2, CachedY1 - 1);
				CachedY1 = Y1;
				bCacheExtended = true;
			}

			if (Y2 > CachedY2)
			{
				ExtendCache(CachedX1, CachedY2 + 1, CachedX2, Y2);
				CachedY2 = Y2;
				bCacheExtended = true;
			}

			if (bCacheExtended)
			{
				// Drop a visual log to indicate the area covered by this cache region extension :
				VisualizeLandscapeRegion(CachedX1, CachedY1, CachedX2, CachedY2, FColor::Red, FString::Printf(TEXT("Cache Data (X1:%i, Y1:%i, X2:%i, Y2:%i)"), CachedX1, CachedY1, CachedX2, CachedY2));
			}
		}
	}

	// Store data in the cache from some external source.
	// Rect is half-open (exclusive), not inclusive like CacheData.
	void CacheDataDirect(FIntRect NewDataRect, FIntRect RequestedRect, FIntRect LandscapeExtent, TConstArrayView<AccessorType> Data)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_CacheDataDirect);
		check(!NewDataRect.IsEmpty());
		check(Data.Num() >= NewDataRect.Area());
		check(LandscapeExtent.Contains(NewDataRect));
		check(RequestedRect.Contains(NewDataRect));

		// Allow caching values up to 1 pixel outside the real landscape extent.  
		FIntRect AllowedCacheExtent = LandscapeExtent;
		AllowedCacheExtent.InflateRect(1);

		FIntRect OldCachedRect(CachedX1, CachedY1, CachedX2 + 1, CachedY2 + 1);

		FIntRect NewCachedRect = RequestedRect;
		NewCachedRect.Clip(AllowedCacheExtent);
		check(NewCachedRect.Contains(NewDataRect));

		// Clamp coord to the closest valid landscape coord (exclusive extent)
		// This replaces the complicated bUseInterp code (GetData instead of GetDataFast) for coords outside valid landscape extent.  A better option
		// might be to stop storing outside values at all and adjust coords on data lookups.  That can be explored as part of removing the TMap.
		auto ClampToLandscape = [&LandscapeExtent](FInt32Point Point) -> FInt32Point
			{
				return { FMath::Clamp(Point.X, LandscapeExtent.Min.X, LandscapeExtent.Max.X - 1),
						 FMath::Clamp(Point.Y, LandscapeExtent.Min.Y, LandscapeExtent.Max.Y - 1) };
			};

		int32 Stride = NewDataRect.Max.X - NewDataRect.Min.X;  //Stride of the data buffer
		CachedData.Reserve(NewCachedRect.Area());
		for (int32 Y = NewCachedRect.Min.Y; Y < NewCachedRect.Max.Y; ++Y)
		{
			for (int32 X = NewCachedRect.Min.X; X < NewCachedRect.Max.X; ++X)
			{
				FIntPoint Coord(X, Y);
				if (!bIsValid || !OldCachedRect.Contains(Coord))
				{
					FIntPoint Clamped = ClampToLandscape(Coord);
					FIntPoint BufferRelative = Clamped - NewDataRect.Min;
					int32 Idx = BufferRelative.Y * Stride + BufferRelative.X;
					CachedData.Add(Coord, Data[Idx]);
				}
			}
		}

		bIsValid = true;

		CachedX1 = NewCachedRect.Min.X;
		CachedY1 = NewCachedRect.Min.Y;
		CachedX2 = NewCachedRect.Max.X - 1;
		CachedY2 = NewCachedRect.Max.Y - 1;

		ensure(CachedData.Contains({ CachedX1 , CachedY1 }));
		ensure(CachedData.Contains({ CachedX1 , CachedY2 }));
		ensure(CachedData.Contains({ CachedX2 , CachedY1 }));
		ensure(CachedData.Contains({ CachedX2 , CachedY2 }));
	}

	AccessorType* GetValueRef(int32 LandscapeX, int32 LandscapeY)
	{
		return CachedData.Find(FIntPoint(LandscapeX, LandscapeY));
	}

	float GetValue(float LandscapeX, float LandscapeY)
	{
		int32 X = FMath::FloorToInt(LandscapeX);
		int32 Y = FMath::FloorToInt(LandscapeY);
		AccessorType* P00 = CachedData.Find(FIntPoint(X, Y));
		AccessorType* P10 = CachedData.Find(FIntPoint(X + 1, Y));
		AccessorType* P01 = CachedData.Find(FIntPoint(X, Y + 1));
		AccessorType* P11 = CachedData.Find(FIntPoint(X + 1, Y + 1));

		// Search for nearest value if missing data
		float V00 = P00 ? *P00 : (P10 ? *P10 : (P01 ? *P01 : (P11 ? *P11 : 0.0f)));
		float V10 = P10 ? *P10 : (P00 ? *P00 : (P11 ? *P11 : (P01 ? *P01 : 0.0f)));
		float V01 = P01 ? *P01 : (P00 ? *P00 : (P11 ? *P11 : (P10 ? *P10 : 0.0f)));
		float V11 = P11 ? *P11 : (P10 ? *P10 : (P01 ? *P01 : (P00 ? *P00 : 0.0f)));

		return FMath::Lerp(
			FMath::Lerp(V00, V10, LandscapeX - X),
			FMath::Lerp(V01, V11, LandscapeX - X),
			LandscapeY - Y);
	}

	FVector GetNormal(int32 X, int32 Y)
	{
		AccessorType* P00 = CachedData.Find(FIntPoint(X, Y));
		AccessorType* P10 = CachedData.Find(FIntPoint(X + 1, Y));
		AccessorType* P01 = CachedData.Find(FIntPoint(X, Y + 1));
		AccessorType* P11 = CachedData.Find(FIntPoint(X + 1, Y + 1));

		// Search for nearest value if missing data
		float V00 = P00 ? *P00 : (P10 ? *P10 : (P01 ? *P01 : (P11 ? *P11 : 0.0f)));
		float V10 = P10 ? *P10 : (P00 ? *P00 : (P11 ? *P11 : (P01 ? *P01 : 0.0f)));
		float V01 = P01 ? *P01 : (P00 ? *P00 : (P11 ? *P11 : (P10 ? *P10 : 0.0f)));
		float V11 = P11 ? *P11 : (P10 ? *P10 : (P01 ? *P01 : (P00 ? *P00 : 0.0f)));

		FVector Vert00 = FVector(0.0f, 0.0f, V00);
		FVector Vert01 = FVector(0.0f, 1.0f, V01);
		FVector Vert10 = FVector(1.0f, 0.0f, V10);
		FVector Vert11 = FVector(1.0f, 1.0f, V11);

		FVector FaceNormal1 = ((Vert00 - Vert10) ^ (Vert10 - Vert11)).GetSafeNormal();
		FVector FaceNormal2 = ((Vert11 - Vert01) ^ (Vert01 - Vert00)).GetSafeNormal();
		return (FaceNormal1 + FaceNormal2).GetSafeNormal();
	}

	void SetValue(int32 LandscapeX, int32 LandscapeY, AccessorType Value)
	{
		CachedData.Add(FIntPoint(LandscapeX, LandscapeY), Forward<AccessorType>(Value));
	}

	bool IsZeroValue(const FVector& Value)
	{
		return (FMath::IsNearlyZero(Value.X) && FMath::IsNearlyZero(Value.Y));
	}

	bool IsZeroValue(const FVector2D& Value)
	{
		return (FMath::IsNearlyZero(Value.X) && FMath::IsNearlyZero(Value.Y));
	}

	bool IsZeroValue(const uint16& Value)
	{
		return Value == 0;
	}

	bool IsZeroValue(const uint8& Value)
	{
		return Value == 0;
	}

	bool HasCachedData(int32 X1, int32 Y1, int32 X2, int32 Y2) const
	{
		return (bIsValid && X1 >= CachedX1 && Y1 >= CachedY1 && X2 <= CachedX2 && Y2 <= CachedY2);
	}
	
	bool GetDataAndCacheWithVisibilityOverride(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<AccessorType>& OutData, TBitArray<> LayerVisibility, ALandscape* Landscape)
	{
		// Drop a visual log to indicate the area requested by this data access :
		VisualizeLandscapeRegion(X1, Y1, X2, Y2, FColor::Blue, TEXT("CacheDataRequest"));

		if (!HasCachedData(X1, Y1, X2, Y2))
		{
			// The cache needs to be expanded, compute the new bounds : 
			// The bounds we calculate here need to be what would be the result of calling CacheData with this region, meaning that they should include the previous bounds
			FIntRect NewCacheBounds(bIsValid ? FMath::Min(X1, CachedX1) : X1, bIsValid ? FMath::Min(Y1, CachedY1) : Y1, bIsValid ? FMath::Max(X2, CachedX2) : X2, bIsValid ? FMath::Max(Y2, CachedY2) : Y2);

			// SelectiveRender only supports heightmaps at the moment.
			const bool bIsHeightmap = std::is_same<AccessorType, uint16>::value;
			check(bIsHeightmap);

			// For now, do the same bounds expansion as OnCacheUpdating.  Align with component boundaries.
			// Note: FloorToInt32 is "strictly less", not "towards zero".
			// Note, because of the +1/-1 expansion at a higher level in the callstack, this will pad out an entire component outside the landscape.
			// CacheDataDirect is currently ignoring this excessive padding and limiting to only 1 row/column extra.
			// TODO:  simplify when removing IntermediateRender.
			FIntRect DesiredCacheBounds;
			DesiredCacheBounds.Min.X = (FMath::FloorToInt((float)NewCacheBounds.Min.X / LandscapeInfo->ComponentSizeQuads)) * LandscapeInfo->ComponentSizeQuads;
			DesiredCacheBounds.Min.Y = (FMath::FloorToInt((float)NewCacheBounds.Min.Y / LandscapeInfo->ComponentSizeQuads)) * LandscapeInfo->ComponentSizeQuads;
			DesiredCacheBounds.Max.X = (FMath::CeilToInt((float)NewCacheBounds.Max.X / LandscapeInfo->ComponentSizeQuads)) * LandscapeInfo->ComponentSizeQuads;
			DesiredCacheBounds.Max.Y = (FMath::CeilToInt((float)NewCacheBounds.Max.Y / LandscapeInfo->ComponentSizeQuads)) * LandscapeInfo->ComponentSizeQuads;

			FIntRect LandscapeExtent;
			bool bRet = LandscapeInfo->GetLandscapeExtent(LandscapeExtent);
			ensure(bRet);
			if (!bRet)
			{
				// Failure likely means no proxies loaded
				return false;
			}

			// FLandscapeToolStrokeSmooth::Apply is extending bounds by 1 in all directions, outside landscape bounds, and asserts if the result bounds don't include that.
			// landscape with 2017 verts/samples, DesiredCacheBounds is (-1,2017).  For now, don't change the return rect here.
			FIntRect EvaluateRect = DesiredCacheBounds;

			// Limit to the extent of the loaded landscape.  The calling code will ask for data outside the landscape and the old GetData code would generate it to fill in, but that doesn't
			// appear to have been useful.
			EvaluateRect.Clip(LandscapeExtent);  
			EvaluateRect.Max += FInt32Point(1, 1);  // Convert inclusive range to exclusive (half-open).  [a,b)

#if 0
			UE_LOGF(LogLandscapeTools, Warning, "PartialRender - New ---- (fr%u)  Base [%d, %d]-[%d, %d]  s[%d, %d],   Padded [%d, %d]-[%d, %d] s[%d, %d],   Evaluate [%d, %d]-[%d, %d] s[%d, %d]", GFrameNumber,
				NewCacheBounds.Min.X, NewCacheBounds.Min.Y, NewCacheBounds.Max.X, NewCacheBounds.Max.Y, NewCacheBounds.Size().X, NewCacheBounds.Size().Y,
				DesiredCacheBounds.Min.X, DesiredCacheBounds.Min.Y, DesiredCacheBounds.Max.X, DesiredCacheBounds.Max.Y, DesiredCacheBounds.Size().X, DesiredCacheBounds.Size().Y,
				EvaluateRect.Min.X, EvaluateRect.Min.Y, EvaluateRect.Max.X, EvaluateRect.Max.Y, EvaluateRect.Size().X, EvaluateRect.Size().Y);
#endif

			TArray<AccessorType> Results;
			Results.AddUninitialized(EvaluateRect.Area());

			bRet = AccessorClass::GetSelectiveLayerData(Landscape, EvaluateRect, LayerVisibility, Results);
			ensure(bRet);  // what to do if this fails?

			// Convert to exclusive rects
			FIntRect DesiredCacheBoundsExcl = DesiredCacheBounds;
			DesiredCacheBoundsExcl.Max += FInt32Point(1, 1);
			FIntRect LandscapeExtentExcl = LandscapeExtent;
			LandscapeExtentExcl.Max += FInt32Point(1, 1);

			CacheDataDirect(EvaluateRect, DesiredCacheBoundsExcl, LandscapeExtentExcl, Results);
		}
		ensureMsgf(!bIsValid || HasCachedData(X1, Y1, X2, Y2), TEXT("Data failed to cache (X1:%i, Y1:%i, X2:%i, Y2:%i), (CachedX1:%i, CachedY1:%i, CachedX2:%i, CachedY2:%i)"), X1, Y1, X2, Y2, CachedX1, CachedY1, CachedX2, CachedY2);
		return GetCachedData(X1, Y1, X2, Y2, OutData);
	}

	// X2/Y2 Coordinates are "inclusive" max values
	bool GetCachedData(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<AccessorType>& OutData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_GetCachedData);

		const int32 XSize = (1 + X2 - X1);
		const int32 YSize = (1 + Y2 - Y1);
		const int32 NumSamples = XSize * YSize;
		OutData.Empty(NumSamples);
		OutData.AddUninitialized(NumSamples);
		bool bHasNonZero = false;

		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			const int32 YOffset = (Y - Y1) * XSize;
			for (int32 X = X1; X <= X2; X++)
			{
				const int32 XYOffset = YOffset + (X - X1);
				AccessorType* Ptr = GetValueRef(X, Y);
				if (Ptr)
				{
					OutData[XYOffset] = *Ptr;
					if (!IsZeroValue(*Ptr))
					{
						bHasNonZero = true;
					}
				}
				else
				{
					OutData[XYOffset] = (AccessorType)0;
				}
			}
		}

		return bHasNonZero;
	}

	// X2/Y2 Coordinates are "inclusive" max values
	void SetCachedData(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<AccessorType>& Data, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None, bool bUpdateData = true)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_SetCachedData);

		checkSlow(Data.Num() == (1 + Y2 - Y1) * (1 + X2 - X1));

		// Update cache
		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			for (int32 X = X1; X <= X2; X++)
			{
				SetValue(X, Y, Data[(X - X1) + (Y - Y1)*(1 + X2 - X1)]);
			}
		}

		if (bUpdateData)
		{
			// Update real data
			DataAccess.SetData(X1, Y1, X2, Y2, Data.GetData(), PaintingRestriction);
		}
	}

	// Get the original data before we made any changes with the SetCachedData interface.
	// X2/Y2 Coordinates are "inclusive" max values
	void GetOriginalData(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<AccessorType>& OutOriginalData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_GetOriginalData);

		int32 NumSamples = (1 + X2 - X1)*(1 + Y2 - Y1);
		OutOriginalData.Empty(NumSamples);
		OutOriginalData.AddUninitialized(NumSamples);

		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			for (int32 X = X1; X <= X2; X++)
			{
				AccessorType* Ptr = OriginalData.Find(FIntPoint(X, Y));
				OutOriginalData[(X - X1) + (Y - Y1) * (1 + X2 - X1)] = Ptr ? *Ptr : (AccessorType)0;
			}
		}
	}

	void Flush()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TLandscapeEditCache_Flush);
		DataAccess.Flush();
	}

private:
	// X2/Y2 Coordinates are "inclusive" max values
	void CacheOriginalData(int32 X1, int32 Y1, int32 X2, int32 Y2)
	{
		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			for (int32 X = X1; X <= X2; X++)
			{
				FIntPoint Key = FIntPoint(X, Y);
				AccessorType* Ptr = CachedData.Find(Key);
				if (Ptr)
				{
					check(OriginalData.Find(Key) == NULL);
					OriginalData.Add(Key, *Ptr);
				}
			}
		}
	}

	// X2/Y2 Coordinates are "inclusive" max values
	void VisualizeLandscapeRegion(int32 InX1, int32 InY1, int32 InX2, int32 InY2, const FColor& InColor, const FString& InDescription, double InZOffset = 0.0f)
	{
		check(LandscapeInfo != nullptr);
		ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy();
		check(LandscapeProxy != nullptr);
		const FTransform& LandscapeTransform = LandscapeProxy->GetTransform();
		static constexpr double BaseZOffset = 10.0; // Add a small Z offset to avoid Z-flickering for flat landscapes
		FVector ZOffset(0.0, 0.0, BaseZOffset + InZOffset);
		// the offset is given in world space so unapply the scale before applying the transform
		ZOffset /= LandscapeTransform.GetScale3D();
		FTransform WorldTransformForVisLog = FTransform(ZOffset) * LandscapeTransform;
		// Use (-0.5, -0.5) for min and (+0.5, +0.5) for max because both are inclusive (i.e. caching at 0, 0, 0, 0 (X1, Y1, X2, Y2)) will draw a box of size (1, 1) around (0, 0))
		UE_VLOG_OBOX(LandscapeProxy, LogLandscapeTools, Log, FBox(FVector(InX1, InY1, 0.0) - FVector(0.5, 0.5, 0.0), FVector(InX2, InY2, 0.0) + FVector(0.5, 0.5, 0.0)),
			WorldTransformForVisLog.ToMatrixWithScale(), InColor, TEXT("%s"), *InDescription);
	}

	TMap<FIntPoint, AccessorType> CachedData;
	TMap<FIntPoint, AccessorType> OriginalData;
	// Keep the landscape info for visual logging purposes :
	TWeakObjectPtr<ULandscapeInfo> LandscapeInfo;

	bool bIsValid = false;

	// Inclusive bounds of the current cached region (CachedData)
	int32 CachedX1 = INDEX_NONE;
	int32 CachedY1 = INDEX_NONE;
	int32 CachedX2 = INDEX_NONE;
	int32 CachedY2 = INDEX_NONE;
};

template<bool bInUseInterp>
struct FHeightmapAccessorTool : public FHeightmapAccessor<bInUseInterp>
{
	FHeightmapAccessorTool(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo = nullptr)
	:	FHeightmapAccessor<bInUseInterp>(InLandscapeInfo, InLayerInfo)
	{
		// We don't care about InLayerInfo since heightmaps are not a target layer per se
		check(InLayerInfo == nullptr);
	}
};

struct FLandscapeHeightCache : public TLandscapeEditCache<FHeightmapAccessorTool</*bInUseInterp = */true>, uint16>
{
	static uint16 ClampValue(int32 Value) { return static_cast<uint16>(FMath::Clamp(Value, 0, LandscapeDataAccess::MaxValue)); }

	FLandscapeHeightCache(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo = nullptr)
		: TLandscapeEditCache<FHeightmapAccessorTool</*bInUseInterp = */true>, uint16>(InLandscapeInfo, /*InLayerInfo = */nullptr)
	{
		// We don't care about InLayerInfo since heightmaps are not a target layer per se
		check(InLayerInfo == nullptr);
	}
};

template<bool bInUseInterp>
struct TAlphamapAccessorTool : public TAlphamapAccessor<bInUseInterp>
{
	TAlphamapAccessorTool(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo)
		: TAlphamapAccessor<bInUseInterp>(InLandscapeInfo, InLayerInfo)
	{}
};

struct FLandscapeAlphaCache : public TLandscapeEditCache<TAlphamapAccessorTool</*bInUseInterp = */true>, uint8>
{
	static uint8 ClampValue(int32 Value) { return static_cast<uint8>(FMath::Clamp(Value, 0, 255)); }

	FLandscapeAlphaCache(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo)
		: TLandscapeEditCache<TAlphamapAccessorTool</*bInUseInterp = */true>, uint8>(InLandscapeInfo, InLayerInfo)
	{
	}
};

struct FLandscapeVisCache : public TLandscapeEditCache<TAlphamapAccessorTool</*bInUseInterp = */false>, uint8>
{
	static uint8 ClampValue(int32 Value) { return static_cast<uint8>(FMath::Clamp(Value, 0, 255)); }

	FLandscapeVisCache(ULandscapeInfo* InLandscapeInfo)
		: TLandscapeEditCache<TAlphamapAccessorTool</*bInUseInterp = */false>, uint8>(InLandscapeInfo, ALandscapeProxy::VisibilityLayer)
	{
	}
};

// Allows to cache data from the combined edit layer stack (from the bottom of the stack up to the specified edit layer if bCombinedLayerOperation, 
//  normal cache operation otherwise
template<class ToolTarget>
class FLandscapeEditLayerStackDataCache
{
public:
	FLandscapeEditLayerStackDataCache(ULandscapeInfo* InLandscapeInfo, ToolTarget::CacheClass& Cache)
		: LandscapeInfo(InLandscapeInfo)
		, Landscape(nullptr)
		, EditingLayerGuid()
		, EditingLayerIndex(MAX_uint8)
		, bIsInitialized(false)
		, bCombinedLayerOperation(false)
		, bTargetIsHeightmap(ToolTarget::TargetType == ELandscapeToolTargetType::Heightmap)
		, CacheUpToEditingLayer(Cache)
		, CacheBottomLayers(InLandscapeInfo, Cache.LayerInfo)
	{
	}

	void SetCacheEditingLayer(const FGuid& InEditLayerGUID)
	{
		CacheUpToEditingLayer.DataAccess.SetEditLayer(InEditLayerGUID);
		CacheBottomLayers.DataAccess.SetEditLayer(InEditLayerGUID);
		EditingLayerGuid = InEditLayerGUID;
	}

	void Initialize(bool bInCombinedLayerOperation)
	{
		check(EditingLayerGuid.IsSet());	// you must call SetCacheEditingLayer before Initialize
		if (!bIsInitialized)
		{
			Landscape = LandscapeInfo ? LandscapeInfo->LandscapeActor.Get() : nullptr;
			bCombinedLayerOperation = Landscape && bInCombinedLayerOperation && bTargetIsHeightmap;
			if (bCombinedLayerOperation)
			{
				int32 I = 0;
				for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
				{
					BackupLayerVisibility.Add(EditLayer->IsVisible());
					if (EditLayer->GetGuid() == EditingLayerGuid.GetValue())
					{
						EditingLayerIndex = I;
					}
					++I;
				}
				check(Landscape->GetEditLayersConst().IsValidIndex(EditingLayerIndex));
			}
			bIsInitialized = true;
		}
	}

	// read values in the specified rectangle into the array
	void Read(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<typename ToolTarget::CacheClass::DataType>& Data)
	{
		check(bIsInitialized);
		if (bCombinedLayerOperation)
		{
			TBitArray<> NewLayerVisibility;
			int32 I = 0;
			for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
			{
				NewLayerVisibility.Add((I > EditingLayerIndex) ? false : EditLayer->IsVisible());
				++I;
			}

			// temporarily switch to working on the final runtime data, so we can gather the combined layer data into the caches
			FGuid PreviousLayerGUID = EditingLayerGuid.GetValue();
			SetCacheEditingLayer(FGuid());

			CacheUpToEditingLayer.GetDataAndCacheWithVisibilityOverride(X1, Y1, X2, Y2, Data, NewLayerVisibility, Landscape);

			// Now turn off visibility on the current layer in order to have the data of all bottom layers except the current one
			NewLayerVisibility[EditingLayerIndex] = false;
			CacheBottomLayers.GetDataAndCacheWithVisibilityOverride(X1, Y1, X2, Y2, BottomLayersData, NewLayerVisibility, Landscape);
			
			SetCacheEditingLayer(PreviousLayerGUID);
			check(PreviousLayerGUID == CacheUpToEditingLayer.DataAccess.GetEditLayer());
		}
		else
		{
			CacheUpToEditingLayer.CacheData(X1, Y1, X2, Y2);
			CacheUpToEditingLayer.GetCachedData(X1, Y1, X2, Y2, Data);
		}
	}

	void Write(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<typename ToolTarget::CacheClass::DataType>& Data, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None)
	{
		check(bIsInitialized);
		if (bCombinedLayerOperation)
		{
			const ULandscapeEditLayerBase* EditLayer = Landscape->GetEditLayerConst(EditingLayerIndex);
			check(EditLayer != nullptr);
			check(bTargetIsHeightmap);
			const float Alpha = EditLayer->GetAlphaForTargetType(ELandscapeToolTargetType::Heightmap);
			const float InverseAlpha = (Alpha != 0.f) ? 1.f / Alpha : 1.f;
			TArray<typename ToolTarget::CacheClass::DataType> DataContribution;
			DataContribution.Empty(Data.Num());
			DataContribution.AddUninitialized(Data.Num());
			check(Data.Num() == BottomLayersData.Num());
			for (int i = 0; i < Data.Num(); ++i)
			{
				float Contribution = (LandscapeDataAccess::GetLocalHeight(Data[i]) - LandscapeDataAccess::GetLocalHeight(BottomLayersData[i])) * InverseAlpha;
				DataContribution[i] = static_cast<typename ToolTarget::CacheClass::DataType>(LandscapeDataAccess::GetTexHeight(Contribution));
			}

			FGuid CacheAccessorLayerGuid = CacheUpToEditingLayer.DataAccess.GetEditLayer();
 			checkf(EditingLayerGuid.GetValue() == CacheAccessorLayerGuid, TEXT("Editing Layer has changed between Initialize and Write. Was: %s (%s). Is now: %s (%s)"),
 				Landscape->GetEditLayerConst(*EditingLayerGuid) ? *(Landscape->GetEditLayerConst(*EditingLayerGuid)->GetName().ToString()) : TEXT("<unknown>"), *EditingLayerGuid->ToString(),
 				Landscape->GetEditLayerConst(CacheAccessorLayerGuid) ? *(Landscape->GetEditLayerConst(CacheAccessorLayerGuid)->GetName().ToString()) : TEXT("<unknown>"), *CacheAccessorLayerGuid.ToString());

			CacheUpToEditingLayer.SetCachedData(X1, Y1, X2, Y2, Data, PaintingRestriction, false);
			// Effectively write the contribution
			CacheUpToEditingLayer.DataAccess.SetData(X1, Y1, X2, Y2, DataContribution.GetData(), PaintingRestriction);
			CacheUpToEditingLayer.DataAccess.Flush();
		}
		else
		{
			CacheUpToEditingLayer.SetCachedData(X1, Y1, X2, Y2, Data, PaintingRestriction);
			CacheUpToEditingLayer.Flush();
		}
	}

private:
	ULandscapeInfo* LandscapeInfo;
	ALandscape* Landscape;
	TOptional<FGuid> EditingLayerGuid;
	uint8 EditingLayerIndex;
	TBitArray<> BackupLayerVisibility;
	TArray<typename ToolTarget::CacheClass::DataType> BottomLayersData;
	bool bIsInitialized;
	bool bCombinedLayerOperation;
	bool bTargetIsHeightmap;

	typename ToolTarget::CacheClass& CacheUpToEditingLayer;
	typename ToolTarget::CacheClass CacheBottomLayers;
};

//
// FFullWeightmapAccessor
//
template<bool bInUseInterp>
struct FFullWeightmapAccessor
{
	enum { bUseInterp = bInUseInterp };
	FFullWeightmapAccessor(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo)
		: LandscapeInfo(InLandscapeInfo)
		, LandscapeEdit(InLandscapeInfo)
	{
		// We don't care about InLayerInfo, actually, since we access all target layers here
	}

	void SetEditLayer(const FGuid& InEditLayerGUID)
	{
		LandscapeEdit.SetEditLayer(InEditLayerGUID);
	}

	FGuid GetEditLayer() const
	{
		return LandscapeEdit.GetEditLayer();
	}

	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, TArray<uint8>>& Data)
	{
		// Do not Support for interpolation....
		check(false && TEXT("Do not support interpolation for FullWeightmapAccessor for now"));
	}

	void GetDataFast(int32 X1, int32 Y1, int32 X2, int32 Y2, TMap<FIntPoint, TArray<uint8>>& Data)
	{
		DirtyLayerInfos.Empty();
		LandscapeEdit.GetWeightDataFast(NULL, X1, Y1, X2, Y2, Data);
	}

	void SetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, ELandscapeLayerPaintingRestriction PaintingRestriction)
	{
		TSet<ULandscapeComponent*> Components;
		if (LandscapeEdit.GetComponentsInRegion(X1, Y1, X2, Y2, &Components))
		{
			LandscapeEdit.SetAlphaData(DirtyLayerInfos, X1, Y1, X2, Y2, Data, /*Stride = */0, PaintingRestriction);
		}
		DirtyLayerInfos.Empty();
	}

	void Flush()
	{
		LandscapeEdit.Flush();
	}

	TSet<ULandscapeLayerInfoObject*> DirtyLayerInfos;

private:
	ULandscapeInfo* LandscapeInfo;
	FLandscapeEditDataInterface LandscapeEdit;
	TSet<ULandscapeComponent*> ModifiedComponents;
};

struct FLandscapeFullWeightCache : public TLandscapeEditCache<FFullWeightmapAccessor<false>, TArray<uint8>>
{
	FLandscapeFullWeightCache(ULandscapeInfo* InLandscapeInfo)
		: TLandscapeEditCache<FFullWeightmapAccessor</*bInUseInterp = */false>, TArray<uint8>>(InLandscapeInfo, /*InLayerInfo = */ nullptr)
	{
	}

	// Only for all weight case... the accessor type should be TArray<uint8>
	void GetCachedData(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<uint8>& OutData, int32 ArraySize)
	{
		if (ArraySize == 0)
		{
			OutData.Empty();
			return;
		}

		const int32 XSize = (1 + X2 - X1);
		const int32 YSize = (1 + Y2 - Y1);
		const int32 Stride = XSize * ArraySize;
		int32 NumSamples = XSize * YSize * ArraySize;
		OutData.Empty(NumSamples);
		OutData.AddUninitialized(NumSamples);

		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			const int32 YOffset = (Y - Y1) * Stride;
			for (int32 X = X1; X <= X2; X++)
			{
				const int32 XYOffset = YOffset + (X - X1) * ArraySize;
				TArray<uint8>* Ptr = GetValueRef(X, Y);
				if (Ptr)
				{
					for (int32 Z = 0; Z < ArraySize; Z++)
					{
						OutData[XYOffset + Z] = (*Ptr)[Z];
					}
				}
				else
				{
					FMemory::Memzero((void*)&OutData[XYOffset], (SIZE_T)ArraySize);
				}
			}
		}
	}

	// Only for all weight case... the accessor type should be TArray<uint8>
	void SetCachedData(int32 X1, int32 Y1, int32 X2, int32 Y2, TArray<uint8>& Data, int32 ArraySize, ELandscapeLayerPaintingRestriction PaintingRestriction)
	{
		// Update cache
		for (int32 Y = Y1; Y <= Y2; Y++)
		{
			for (int32 X = X1; X <= X2; X++)
			{
				TArray<uint8> Value;
				Value.Empty(ArraySize);
				Value.AddUninitialized(ArraySize);
				for (int32 Z = 0; Z < ArraySize; Z++)
				{
					Value[Z] = Data[((X - X1) + (Y - Y1)*(1 + X2 - X1)) * ArraySize + Z];
				}
				SetValue(X, Y, MoveTemp(Value));
			}
		}

		// Update real data
		DataAccess.SetData(X1, Y1, X2, Y2, Data.GetData(), PaintingRestriction);
	}

	void AddDirtyLayer(ULandscapeLayerInfoObject* InLayerInfo)
	{
		DataAccess.DirtyLayerInfos.Add(InLayerInfo);
	}
};

// 
// FDatamapAccessor
//
template<bool bInUseInterp>
struct FDatamapAccessor
{
	enum { bUseInterp = bInUseInterp };
	FDatamapAccessor(ULandscapeInfo* InLandscapeInfo, ULandscapeLayerInfoObject* InLayerInfo = nullptr)
		: LandscapeEdit(InLandscapeInfo)
	{
		// We don't care about InLayerInfo since data map is not a target layer per se
		check(InLayerInfo == nullptr);
	}

	void GetData(int32& X1, int32& Y1, int32& X2, int32& Y2, TMap<FIntPoint, uint8>& Data)
	{
		LandscapeEdit.GetSelectData(X1, Y1, X2, Y2, Data);
	}

	void GetDataFast(const int32 X1, const int32 Y1, const int32 X2, const int32 Y2, TMap<FIntPoint, uint8>& Data)
	{
		LandscapeEdit.GetSelectData(X1, Y1, X2, Y2, Data);
	}

	void SetData(int32 X1, int32 Y1, int32 X2, int32 Y2, const uint8* Data, ELandscapeLayerPaintingRestriction PaintingRestriction = ELandscapeLayerPaintingRestriction::None)
	{
		if (LandscapeEdit.GetComponentsInRegion(X1, Y1, X2, Y2))
		{
			LandscapeEdit.SetSelectData(X1, Y1, X2, Y2, Data, 0);
		}
	}

	void Flush()
	{
		LandscapeEdit.Flush();
	}

private:
	FLandscapeEditDataInterface LandscapeEdit;
};

struct FLandscapeDataCache : public TLandscapeEditCache<FDatamapAccessor</*bInUseInterp = */false>, uint8>
{
	static uint8 ClampValue(int32 Value) { return static_cast<uint8>(FMath::Clamp(Value, 0, 255)); }

	FLandscapeDataCache(ULandscapeInfo* InLandscapeInfo)
		: TLandscapeEditCache<FDatamapAccessor</*bInUseInterp = */false>, uint8>(InLandscapeInfo, /*InLayerInfo = */ nullptr)
	{
	}
};


//
// Tool targets
//
struct FHeightmapToolTarget
{
	typedef FLandscapeHeightCache CacheClass;
	static const ELandscapeToolTargetType TargetType = ELandscapeToolTargetType::Heightmap;

	static float StrengthMultiplier(ULandscapeInfo* LandscapeInfo, float BrushRadius)
	{
		if (LandscapeInfo)
		{
			// Adjust strength based on brush size and drawscale, so strength 1 = one hemisphere
			return static_cast<float>(BrushRadius * LANDSCAPE_INV_ZSCALE / LandscapeInfo->DrawScale.Z);
		}
		return 5.0f * LANDSCAPE_INV_ZSCALE;
	}

	static FMatrix ToWorldMatrix(ULandscapeInfo* LandscapeInfo)
	{
		FMatrix Result = FTranslationMatrix(FVector(0, 0, -LandscapeDataAccess::MidValue));
		Result *= FScaleMatrix(FVector(1.0f, 1.0f, LANDSCAPE_ZSCALE) * LandscapeInfo->DrawScale);
		return Result;
	}

	static FMatrix FromWorldMatrix(ULandscapeInfo* LandscapeInfo)
	{
		FMatrix Result = FScaleMatrix(FVector(1.0f, 1.0f, LANDSCAPE_INV_ZSCALE) / (LandscapeInfo->DrawScale));
		Result *= FTranslationMatrix(FVector(0, 0, LandscapeDataAccess::MidValue));
		return Result;
	}
};


struct FWeightmapToolTarget
{
	typedef FLandscapeAlphaCache CacheClass;
	static const ELandscapeToolTargetType TargetType = ELandscapeToolTargetType::Weightmap;

	static float StrengthMultiplier(ULandscapeInfo* LandscapeInfo, float BrushRadius)
	{
		return 255.0f;
	}

	static FMatrix ToWorldMatrix(ULandscapeInfo* LandscapeInfo) { return FMatrix::Identity; }
	static FMatrix FromWorldMatrix(ULandscapeInfo* LandscapeInfo) { return FMatrix::Identity; }
};

/**
 * FLandscapeToolStrokeBase - base class for tool strokes (used by FLandscapeToolBase)
 */

class FLandscapeToolStrokeBase : protected FGCObject
{
public:
	// This is also the expected signature of derived class constructor used by FLandscapeToolBase
	FLandscapeToolStrokeBase(FEdModeLandscape* InEdMode, FEditorViewportClient* InViewportClient, const FLandscapeToolTarget& InTarget)
		: EdMode(InEdMode)
		, LandscapeInfo(InTarget.LandscapeInfo.Get())
	{
		check(InEdMode->CurrentTool != nullptr);
		ELandscapeToolTargetType SupportedTargetType = InEdMode->CurrentTool->GetSupportedTargetType();
		bUseContinuousApply = (SupportedTargetType == ELandscapeToolTargetType::Weightmap) ? InEdMode->UISettings->bApplyWithoutMovingPaint : InEdMode->UISettings->bApplyWithoutMovingSculpt;
	}

	virtual void SetEditLayer(const FGuid& EditLayerGUID)
	{
		// if this function is not overridden, then the tool uses the old method of getting the edit layer (using the shared EditingLayer on ALandscape)
		// we should migrate tools to use SetEditLayer() and deprecate the reliance on the shared EditingLayer
		// once all tools use SetEditLayer we can move this to be part of the constructor
	}

	// Signature of Apply() method for derived strokes
	// void Apply(FEditorViewportClient* ViewportClient, FLandscapeBrush* Brush, const ULandscapeEditorObject* UISettings, const TArray<FLandscapeToolMousePosition>& MousePositions);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override
	{
		Collector.AddReferencedObject(LandscapeInfo);
	}
	
	virtual FString GetReferencerName() const override
	{
		return TEXT("FLandscapeToolStrokeBase");
	}

	bool GetUseContinuousApply() const
	{
		return bUseContinuousApply;
	}

protected:
	FEdModeLandscape* EdMode = nullptr;
	TObjectPtr<ULandscapeInfo> LandscapeInfo = nullptr;
	// Whether to call Apply() every frame even if the mouse hasn't moved.  May be overwritten by child classes.
	bool bUseContinuousApply = false;
};

/**
 * FLandscapeToolBase - base class for painting tools
 *		ToolTarget - the target for the tool (weight or heightmap)
 *		StrokeClass - the class that implements the behavior for a mouse stroke applying the tool.
 */
template<class TStrokeClass>
class FLandscapeToolBase : public FLandscapeTool
{
	using Super = FLandscapeTool;

public:
	FLandscapeToolBase(FEdModeLandscape* InEdMode)
		: LastInteractorPosition(FVector2D::ZeroVector)
		, TimeSinceLastInteractorMove(0.0f)
		, EdMode(InEdMode)
		, bCanToolBeActivated(true)
		, ToolStroke()
	{
	}

	virtual bool ShouldUpdateEditingLayer() const 
	{ 
		return AffectsEditLayers();
	}

	virtual ELandscapeLayerUpdateMode GetBeginToolContentUpdateFlag() const
	{
		bool bUpdateHeightmap = this->EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap; 
		return bUpdateHeightmap ? ELandscapeLayerUpdateMode::Update_Heightmap_Editing : ELandscapeLayerUpdateMode::Update_Weightmap_Editing;
	}

	virtual ELandscapeLayerUpdateMode GetTickToolContentUpdateFlag() const
	{
		return GetBeginToolContentUpdateFlag();
	}

	virtual ELandscapeLayerUpdateMode GetEndToolContentUpdateFlag() const
	{
		bool bUpdateHeightmap = this->EdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap;
		return bUpdateHeightmap ? ELandscapeLayerUpdateMode::Update_Heightmap_All_Modes : ELandscapeLayerUpdateMode::Update_Weightmap_All_Modes;
	}

	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& InTarget, const FVector& InHitLocation) override
	{
		TRACE_BOOKMARK(TEXT("BeginTool - %s"), GetToolName());

		ALandscape* Landscape = this->EdMode->GetLandscape();
		if (Landscape)
		{
			if (ShouldUpdateEditingLayer())
			{
				Landscape->RequestLayersContentUpdate(GetBeginToolContentUpdateFlag());
				Landscape->SetEditingLayer(this->EdMode->GetCurrentLayerGuid()); // legacy way to set the edit layer, via Landscape state
			}
			Landscape->SetGrassUpdateEnabled(false);
		}

		if (!ensure(InteractorPositions.Num() == 0))
		{
			InteractorPositions.Empty(1);
		}

		if (ensure(!IsToolActive()))
		{
			ToolStroke.Emplace(EdMode, ViewportClient, InTarget); // construct the tool stroke class
			ToolStroke->SetEditLayer(this->EdMode->GetCurrentLayerGuid()); // set the edit layer explicitly (if the tool supports this path)
			EdMode->CurrentBrush->BeginStroke(static_cast<float>(InHitLocation.X), static_cast<float>(InHitLocation.Y), this);
		}

		// Save the mouse position  
		LastInteractorPosition = FVector2D(InHitLocation);
		InteractorPositions.Emplace(LastInteractorPosition, IsModifierPressed(ViewportClient), IsAlternateModifierPressed(ViewportClient));
		TimeSinceLastInteractorMove = 0.0f;

		ToolStroke->Apply(ViewportClient, EdMode->CurrentBrush, EdMode->UISettings, InteractorPositions);

		InteractorPositions.Empty(1);
		return true;
	}

	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) override
	{
		if (IsToolActive())
		{
			if (InteractorPositions.Num() > 0)
			{
				ToolStroke->Apply(ViewportClient, EdMode->CurrentBrush, EdMode->UISettings, InteractorPositions);
				ViewportClient->Invalidate(false, false);
				InteractorPositions.Empty(1);
			}
			else if (ToolStroke->GetUseContinuousApply() && TimeSinceLastInteractorMove >= 0.25f)
			{
				InteractorPositions.Emplace(LastInteractorPosition, IsModifierPressed(ViewportClient), IsAlternateModifierPressed(ViewportClient));
				ToolStroke->Apply(ViewportClient, EdMode->CurrentBrush, EdMode->UISettings, InteractorPositions);
				ViewportClient->Invalidate(false, false);
				InteractorPositions.Empty(1);
			}
			TimeSinceLastInteractorMove += DeltaTime;

			if (ShouldUpdateEditingLayer())
			{
				ALandscape* Landscape = this->EdMode->CurrentToolTarget.LandscapeInfo->LandscapeActor.Get();
				if (Landscape != nullptr)
				{
					Landscape->RequestLayersContentUpdate(GetTickToolContentUpdateFlag());
				}
			}
		}
	}

	virtual void EndTool(FEditorViewportClient* ViewportClient) override
	{
		if (IsToolActive() && InteractorPositions.Num())
		{
			ToolStroke->Apply(ViewportClient, EdMode->CurrentBrush, EdMode->UISettings, InteractorPositions);
			InteractorPositions.Empty(1);
		}

		ToolStroke.Reset();		// destruct the tool stroke class
		EdMode->CurrentBrush->EndStroke();
		EdMode->UpdateLayerUsageInformation(&EdMode->CurrentToolTarget.LayerInfo);

		ALandscape* Landscape = this->EdMode->GetLandscape();
		if (Landscape)
		{
			if (ShouldUpdateEditingLayer())
			{
				Landscape->RequestLayersContentUpdate(GetEndToolContentUpdateFlag());
				Landscape->SetEditingLayer();
			}
			Landscape->SetGrassUpdateEnabled(true);
		}

		TRACE_BOOKMARK(TEXT("EndTool - %s"), GetToolName());
	}

	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) override
	{
		if (ViewportClient != nullptr && Viewport != nullptr)
		{
			FVector HitLocation;
			if (EdMode->LandscapeMouseTrace(ViewportClient, x, y, HitLocation))
			{
				// If we are moving the mouse to adjust the brush size, don't move the brush
				if (EdMode->CurrentBrush && !EdMode->IsAdjustingBrush(ViewportClient))
				{
					// Inform the brush of the current location, to update the cursor
					EdMode->CurrentBrush->MouseMove(static_cast<float>(HitLocation.X), static_cast<float>(HitLocation.Y));
				}

				if (IsToolActive())
				{
					// Save the interactor position
					if (InteractorPositions.Num() == 0 || LastInteractorPosition != FVector2D(HitLocation))
					{
						LastInteractorPosition = FVector2D(HitLocation);
						InteractorPositions.Emplace(LastInteractorPosition, IsModifierPressed(ViewportClient), IsAlternateModifierPressed(ViewportClient));
					}
					TimeSinceLastInteractorMove = 0.0f;
				}
			}
		}
		else
		{
			const FVector2D NewPosition(x, y);
			if (InteractorPositions.Num() == 0 || LastInteractorPosition != FVector2D(NewPosition))
			{
				LastInteractorPosition = FVector2D(NewPosition);
				InteractorPositions.Emplace(LastInteractorPosition, IsModifierPressed(ViewportClient), IsAlternateModifierPressed(ViewportClient));
			}
			TimeSinceLastInteractorMove = 0.0f;
		}

		return true;
	}

	virtual bool IsToolActive() const override { return ToolStroke.IsSet();  }

	virtual void SetCanToolBeActivated(bool Value) { bCanToolBeActivated = Value; }
	virtual bool CanToolBeActivated() const {	return bCanToolBeActivated; }

	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const override
	{
		return ELandscapeTargetLayerActionType::None;
	}

protected:
	TArray<FLandscapeToolInteractorPosition> InteractorPositions;
	FVector2D LastInteractorPosition;
	float TimeSinceLastInteractorMove;
	FEdModeLandscape* EdMode;
	bool bCanToolBeActivated;
	TOptional<TStrokeClass> ToolStroke;

	bool IsModifierPressed(const class FEditorViewportClient* ViewportClient)
	{
		UE_LOGF(LogLandscapeTools, VeryVerbose, "ViewportClient = %d, IsModifierDown = %d", (ViewportClient != nullptr), (ViewportClient != nullptr && IsShiftDown(ViewportClient->Viewport)));
		return ViewportClient != nullptr && IsShiftDown(ViewportClient->Viewport);
	}

	bool IsAlternateModifierPressed(const FEditorViewportClient* ViewportClient)
	{
		UE_LOGF(LogLandscapeTools, VeryVerbose, "ViewportClient = %d, IsAlternateModifierDown = %d", (ViewportClient != nullptr), (ViewportClient != nullptr && IsCtrlDown(ViewportClient->Viewport)));
		return ViewportClient != nullptr && IsCtrlDown(ViewportClient->Viewport);
	}
};
