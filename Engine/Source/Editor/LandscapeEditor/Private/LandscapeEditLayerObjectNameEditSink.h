// Copyright Epic Games, Inc. All Rights Reserved

#pragma once

#include "LandscapeEditLayer.h"
#include "IObjectNameEditSink.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "LandscapeEditLayer"

class FLandscapeEditLayerObjectNameEditSink : public UE::EditorWidgets::IObjectNameEditSink
{
	virtual UClass* GetSupportedClass() const override
	{
		return ULandscapeEditLayerBase::StaticClass();
	}

	virtual FText GetObjectDisplayName(UObject* Object) const override
	{
		ULandscapeEditLayerBase* EditLayer = CastChecked<ULandscapeEditLayerBase>(Object);
		return FText::FromString(EditLayer->GetName().ToString());
	}

	virtual bool IsObjectDisplayNameReadOnly(UObject* Object) const override
	{
		return false;
	};

	virtual bool SetObjectDisplayName(UObject* Object, FString DisplayName) override
	{
		if (ULandscapeEditLayerBase* EditLayer = CastChecked<ULandscapeEditLayerBase>(Object))
		{
			EditLayer->SetName(*DisplayName, /*bInModify =*/ true);
			return true;
		}

		return false;
	}

	virtual FText GetObjectNameTooltip(UObject* Object) const override
	{
		return LOCTEXT("NonEditableLandscapeEditLayerLabel_TooltipFmt", "Edit Layer Name");
	}
};

#undef LOCTEXT_NAMESPACE