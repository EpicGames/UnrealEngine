// Copyright Epic Games, Inc. All Rights Reserved.


#include "LandscapeEditorCommands.h"

#include "Framework/Commands/InputChord.h"
#include "Framework/Commands/UICommandInfo.h"
#include "GenericPlatform/GenericApplication.h"
#include "HAL/Platform.h"
#include "InputCoreTypes.h"
#include "Internationalization/Internationalization.h"
#include "Styling/AppStyle.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

#define LOCTEXT_NAMESPACE "LandscapeEditorCommands"

FName FLandscapeEditorCommands::LandscapeContext = TEXT("LandscapeEditor");

FLandscapeEditorCommands::FLandscapeEditorCommands()
	: TCommands<FLandscapeEditorCommands>
(
	FLandscapeEditorCommands::LandscapeContext, // Context name for fast lookup
	NSLOCTEXT("Contexts", "LandscapeEditor", "Landscape Editor"), // Localized context name for displaying
	NAME_None, //"LevelEditor" // Parent
	FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
{
}

void FLandscapeEditorCommands::RegisterCommands()
{
	UI_COMMAND(ManageMode, "Mode - Manage", "", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("ToolMode_Manage", ManageMode);
	UI_COMMAND(SculptMode, "Mode - Sculpt", "", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("ToolMode_Sculpt", SculptMode);
	UI_COMMAND(PaintMode, "Mode - Paint", "", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("ToolMode_Paint", PaintMode);

	UI_COMMAND(NewLandscape, "New", "Create or import a new landscape", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_NewLandscape", NewLandscape);

	UI_COMMAND(ResizeLandscape, "Resize", "Change landscape component size.\n\nResize is not supported when using World Partition.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_ResizeLandscape", ResizeLandscape);

	UI_COMMAND(ImportExportTool, "Import", "Import or Export landscape data", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_ImportExport", ImportExportTool);

	UI_COMMAND(SculptTool, "Sculpt", "Sculpt height data.\n\nClick to raise, Shift+Click to lower.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Sculpt", SculptTool);

	UI_COMMAND(EraseTool, "Erase", "Erase height data.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Erase", EraseTool);

	UI_COMMAND(PaintTool, "Paint", "Paint weight data.\n\nClick to paint, Shift+Click to erase.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Paint", PaintTool);

	UI_COMMAND(SmoothTool, "Smooth", "Smooths heightmaps or blend layers.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Smooth", SmoothTool);

	UI_COMMAND(FlattenTool, "Flatten", "Flattens an area of heightmap or blend layer.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Flatten", FlattenTool);

	UI_COMMAND(RampTool, "Ramp", "Creates a ramp between two points.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Ramp", RampTool);

	UI_COMMAND(ErosionTool, "Erosion", "Thermal Erosion - Simulates erosion caused by the movement of soil from higher areas to lower areas", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Erosion", ErosionTool);

	UI_COMMAND(HydroErosionTool, "Hydro", "Hydro Erosion - Simulates erosion caused by rainfall", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_HydraErosion", HydroErosionTool);

	UI_COMMAND(NoiseTool, "Noise", "Adds noise to the heightmap or blend layer", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Noise", NoiseTool);

	UI_COMMAND(VisibilityTool, "Visibility", "Mask out individual quads in the landscape, leaving a hole.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Visibility", VisibilityTool);

	UI_COMMAND(BlueprintBrushTool, "Blueprint", "Custom Blueprint-based brushes that will procedurally affect the landscape's heightmaps and/or weightmaps.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_BlueprintBrush", BlueprintBrushTool);

	UI_COMMAND(SelectComponentTool, "Select", "Select components to use with other tools", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Select", SelectComponentTool);

	UI_COMMAND(AddComponentTool, "Add", "Add components to the landscape", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_AddComponent", AddComponentTool);

	UI_COMMAND(DeleteComponentTool, "Delete", "Delete components from the landscape, leaving a hole", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_DeleteComponent", DeleteComponentTool);

	UI_COMMAND(MoveToLevelTool, "Move", "Move landscape components to a landscape proxy in the currently active streaming level, so that they can be streamed in/out independently of the rest of the landscape.\n\nMove is not supported when using World Partition.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_MoveToLevel", MoveToLevelTool);

	UI_COMMAND(RegionSelectTool, "Select", "Select a region of landscape to use as a mask for other tools", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Mask", RegionSelectTool);
	UI_COMMAND(RegionCopyPasteTool, "Copy", "Copy/Paste areas of the landscape, or import/export a copied area of landscape from disk", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_CopyPaste", RegionCopyPasteTool);
	UI_COMMAND(MirrorTool, "Mirror", "Copies one side of a landscape to the other, to easily create a mirrored landscape.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Mirror", MirrorTool);

	UI_COMMAND(SplineTool, "Splines", "Create splines that will procedurally affect the landscape's heightmaps and/or weightmaps.", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Tool_Splines", SplineTool);

	UI_COMMAND(CircleBrush, "Circle", "Simple circular brush", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("BrushSet_Circle", CircleBrush);
	UI_COMMAND(AlphaBrush, "Alpha", "Alpha brush, orients a mask image with the brush stroke", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("BrushSet_Alpha", AlphaBrush);
	UI_COMMAND(AlphaBrush_Pattern, "Pattern", "Pattern brush, tiles a mask image across the landscape", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("BrushSet_Pattern", AlphaBrush_Pattern);
	UI_COMMAND(ComponentBrush, "Component", "Work with entire landscape components", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("BrushSet_Component", ComponentBrush);
	UI_COMMAND(GizmoBrush, "Gizmo", "Work with the landscape gizmo, used for copy/pasting landscape", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("BrushSet_Gizmo", GizmoBrush);
	NameToCommandMap.Add("BrushSet_Splines", SplineTool);

	UI_COMMAND(CircleBrush_Smooth, "Smooth", "Smooth falloff", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Circle_Smooth", CircleBrush_Smooth);
	UI_COMMAND(CircleBrush_Linear, "Linear", "Sharp, linear falloff", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Circle_Linear", CircleBrush_Linear);
	UI_COMMAND(CircleBrush_Spherical, "Spherical", "Spherical falloff, smooth at the center and sharp at the edge", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Circle_Spherical", CircleBrush_Spherical);
	UI_COMMAND(CircleBrush_Tip, "Tip", "Tip falloff, sharp at the center and smooth at the edge", EUserInterfaceActionType::RadioButton, FInputChord());
	NameToCommandMap.Add("Circle_Tip", CircleBrush_Tip);

	UI_COMMAND(ViewModeNormal, "Normal", "", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewModeLOD, "LOD", "", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewModeLayerUsage, "Layer Usage", "", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewModeLayerDensity, "Layer Density", "", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewModeLayerDebug, "Layer Debug", "", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ViewModeWireframeOnTop, "Wireframe on Top", "", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ViewModeLayerContribution, "Layer Contribution", "", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(IncreaseBrushSize, "Increase Brush Size", "Press this key to increase brush size by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::RightBracket));
	UI_COMMAND(DecreaseBrushSize, "Decrease Brush Size", "Press this key to decrease brush size by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EKeys::LeftBracket));
	UI_COMMAND(IncreaseBrushFalloff, "Increase Brush Falloff", "Press this key to increase brush falloff by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::RightBracket));
	UI_COMMAND(DecreaseBrushFalloff, "Decrease Brush Falloff", "Press this key to decrease brush falloff by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::LeftBracket));
	UI_COMMAND(IncreaseBrushStrength, "Increase Brush Strength", "Press this key to increase brush strength by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::RightBracket));
	UI_COMMAND(DecreaseBrushStrength, "Decrease Brush Strength", "Press this key to decrease brush strength by a fixed increment.", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control, EKeys::LeftBracket));
	UI_COMMAND(IncreaseAlphaBrushRotation, "Increase Alpha Brush Rotation", "Press this key to increase alpha brush rotation by a fixed increment.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DecreaseAlphaBrushRotation, "Decrease Alpha Brush Rotation", "Press this key to decrease alpha brush rotation by a fixed increment.", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(DragBrushSizeAndFalloff, "Change Brush Size And Falloff", "Hold this key and then drag to increase or decrease brush size (right/left) or falloff (up/down).", EUserInterfaceActionType::RadioButton, FInputChord(EModifierKey::Control | EModifierKey::Alt, EKeys::RightMouseButton));
	UI_COMMAND(DragBrushSize, "Change Brush Size", "Hold this key and then drag to increase or decrease brush size.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DragBrushFalloff, "Change Brush Falloff", "Hold this key and then drag to increase or decrease brush falloff.", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(DragBrushStrength, "Change Brush Strength", "Hold this key and then drag to increase or decrease brush strength.", EUserInterfaceActionType::RadioButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE
