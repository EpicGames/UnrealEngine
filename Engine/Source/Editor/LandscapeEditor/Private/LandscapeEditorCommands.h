// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FName;
class FUICommandInfo;

/**
 * Unreal landscape editor actions
 */
class FLandscapeEditorCommands : public TCommands<FLandscapeEditorCommands>
{

public:
	FLandscapeEditorCommands();
	

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;

public:
	static FName LandscapeContext;

	// Mode Switch
	TSharedPtr<FUICommandInfo> ManageMode;
	TSharedPtr<FUICommandInfo> SculptMode;
	TSharedPtr<FUICommandInfo> PaintMode;

	// Tools
	TSharedPtr<FUICommandInfo> NewLandscape;
	TSharedPtr<FUICommandInfo> ResizeLandscape;
	TSharedPtr<FUICommandInfo> ImportExportTool;

	TSharedPtr<FUICommandInfo> SculptTool;
	TSharedPtr<FUICommandInfo> EraseTool;
	TSharedPtr<FUICommandInfo> PaintTool;
	TSharedPtr<FUICommandInfo> SmoothTool;
	TSharedPtr<FUICommandInfo> FlattenTool;
	TSharedPtr<FUICommandInfo> RampTool;
	TSharedPtr<FUICommandInfo> ErosionTool;
	TSharedPtr<FUICommandInfo> HydroErosionTool;
	TSharedPtr<FUICommandInfo> NoiseTool;
	TSharedPtr<FUICommandInfo> VisibilityTool;
	TSharedPtr<FUICommandInfo> BlueprintBrushTool;

	TSharedPtr<FUICommandInfo> SelectComponentTool;
	TSharedPtr<FUICommandInfo> AddComponentTool;
	TSharedPtr<FUICommandInfo> DeleteComponentTool;
	TSharedPtr<FUICommandInfo> MoveToLevelTool;

	TSharedPtr<FUICommandInfo> RegionSelectTool;
	TSharedPtr<FUICommandInfo> RegionCopyPasteTool;
	TSharedPtr<FUICommandInfo> MirrorTool;

	TSharedPtr<FUICommandInfo> SplineTool;

	// Brushes
	TSharedPtr<FUICommandInfo> CircleBrush;
	TSharedPtr<FUICommandInfo> AlphaBrush;
	TSharedPtr<FUICommandInfo> AlphaBrush_Pattern;
	TSharedPtr<FUICommandInfo> ComponentBrush;
	TSharedPtr<FUICommandInfo> GizmoBrush;

	TSharedPtr<FUICommandInfo> CircleBrush_Smooth;
	TSharedPtr<FUICommandInfo> CircleBrush_Linear;
	TSharedPtr<FUICommandInfo> CircleBrush_Spherical;
	TSharedPtr<FUICommandInfo> CircleBrush_Tip;

	TSharedPtr<FUICommandInfo> ViewModeNormal;
	TSharedPtr<FUICommandInfo> ViewModeLOD;
	TSharedPtr<FUICommandInfo> ViewModeLayerDensity;
	TSharedPtr<FUICommandInfo> ViewModeLayerDebug;
	TSharedPtr<FUICommandInfo> ViewModeWireframeOnTop;
	TSharedPtr<FUICommandInfo> ViewModeLayerUsage;
	TSharedPtr<FUICommandInfo> ViewModeLayerContribution;

	// Adjusting brushes
	TSharedPtr<FUICommandInfo> IncreaseBrushSize;
	TSharedPtr<FUICommandInfo> DecreaseBrushSize;
	TSharedPtr<FUICommandInfo> IncreaseBrushFalloff;
	TSharedPtr<FUICommandInfo> DecreaseBrushFalloff;
	TSharedPtr<FUICommandInfo> IncreaseBrushStrength;
	TSharedPtr<FUICommandInfo> DecreaseBrushStrength;
	TSharedPtr<FUICommandInfo> IncreaseAlphaBrushRotation;
	TSharedPtr<FUICommandInfo> DecreaseAlphaBrushRotation;

	TSharedPtr<FUICommandInfo> DragBrushSizeAndFalloff;
	TSharedPtr<FUICommandInfo> DragBrushSize;
	TSharedPtr<FUICommandInfo> DragBrushFalloff;
	TSharedPtr<FUICommandInfo> DragBrushStrength;

	// Map
	TMap<FName, TSharedPtr<FUICommandInfo>> NameToCommandMap;
};

///**
// * Implementation of various level editor action callback functions
// */
//class FLevelEditorActionCallbacks
//{
//public:
//};
