// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_AlphaBrush.h"
#include "UnrealClient.h"
#include "ViewportClient.h"
#include "Engine/Texture2D.h"
#include "Widgets/Layout/SBorder.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SViewport.h"
#include "CanvasItem.h"
#include "LandscapeEditorObject.h"

#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "SAssetDropTarget.h"

#include "Slate/SceneViewport.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "CanvasTypes.h"
#include "TextureCompiler.h"
#include "TextureResource.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.Brushes.Alpha"

class FTextureMaskThumbnailViewportClient : public FViewportClient
{
public:
	FTextureMaskThumbnailViewportClient(TSharedRef<class STextureMaskThumbnail> InParent)
		: Parent(InParent)
	{
	}

	/** FViewportClient interface */
	virtual void Draw(FViewport* Viewport, FCanvas* Canvas) override;

private:
	TWeakPtr<STextureMaskThumbnail> Parent;
};

class STextureMaskThumbnail : public SCompoundWidget
{
	friend FTextureMaskThumbnailViewportClient;

public:
	SLATE_BEGIN_ARGS( STextureMaskThumbnail )
		: _TextureChannel( 0 ) {}
	SLATE_ATTRIBUTE( UTexture2D*, Texture )
	SLATE_ATTRIBUTE( uint8, TextureChannel )
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs )
	{
		Texture = InArgs._Texture;
		TextureChannel = InArgs._TextureChannel;

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		FLinearColor AssetColor(ForceInitToZero);
		if (UTexture2D* Texture2D = Texture.Get())
		{
			TWeakPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(Texture2D->GetClass());
			AssetColor = AssetTypeActions.Pin()->GetTypeColor();
		}

		ChildSlot
		[
			SNew(SBorder)
			.Padding(4.0f)
			.BorderImage( FAppStyle::GetBrush("PropertyEditor.AssetTileItem.DropShadow") )
			.OnMouseDoubleClick(this, &STextureMaskThumbnail::OnAssetThumbnailDoubleClick)
			//.OnClicked_Static(&FLandscapeEditorDetailCustomization_AlphaBrush::OnTextureButtonClicked)
			[
				SNew(SBox)
				.ToolTipText(this, &STextureMaskThumbnail::OnGetToolTip)
				.WidthOverride(64.0f)
				.HeightOverride(64.0f)
				[
					SNew(SBorder)
					.Padding(2)
					.BorderImage(FAppStyle::GetBrush("AssetThumbnail", ".Border"))
					//.BorderBackgroundColor(this, &SAssetThumbnail::GetViewportBorderColorAndOpacity)
					.BorderBackgroundColor(AssetColor)
					//.ColorAndOpacity(this, &SAssetThumbnail::GetViewportColorAndOpacity)
					//.Visibility(this, &SAssetThumbnail::GetViewportVisibility)
					[
						SAssignNew(ViewportWidget, SViewport)
						.EnableGammaCorrection(false)
					]
				]
			]
		];

		ViewportClient = MakeShareable(new FTextureMaskThumbnailViewportClient(SharedThis(this)));

		Viewport = FSceneViewport::Create(ViewportClient, ViewportWidget);

		UTexture2D* Texture2D = Texture.Get();
		bCanRender = (Texture2D && !Texture2D->IsCompiling() && Texture2D->GetResource());
	}

	FReply OnAssetThumbnailDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
	{
		UTexture2D* Texture2D = Texture.Get();
		if (Texture2D)
		{
			GEditor->EditObject(Texture2D);
		}

		return FReply::Handled();
	}

	FText OnGetToolTip() const
	{
		// Display the package name which is a valid path to the object without redundant information
		UTexture2D* Texture2D = Texture.Get();
		return Texture2D ? FText::FromString(Texture2D->GetOutermost()->GetName()) : FText();
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
	{
		UTexture2D* NewTexture = Texture.Get();
		int32 NewTextureChannel = TextureChannel.Get();
		if (Texture.IsBound() || TextureChannel.IsBound())
		{
			if (NewTexture != CachedTexture
				|| NewTextureChannel != CachedTextureChannel)
			{
				CachedTexture = NewTexture;
				CachedTextureChannel = static_cast<uint8>(NewTextureChannel);
				Viewport->Invalidate();
			}
		}

		bool bCanRenderBefore = bCanRender;
		bCanRender = (NewTexture && !NewTexture->IsCompiling() && NewTexture->GetResource());
		// If we were waiting for the texture to become ready and it now is, invalidate the viewport to trigger a draw :
		if (!bCanRenderBefore && bCanRender)
		{
			Viewport->Invalidate();
		}

		return SCompoundWidget::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
	}

private:
	TAttribute<UTexture2D*> Texture;
	TAttribute<uint8> TextureChannel;

	mutable UTexture2D* CachedTexture;
	mutable uint8 CachedTextureChannel;
	mutable bool bCanRender = false;

	TSharedPtr<FTextureMaskThumbnailViewportClient> ViewportClient;
	TSharedPtr<FSceneViewport> Viewport;
	TSharedPtr<SViewport> ViewportWidget;
};

void FTextureMaskThumbnailViewportClient::Draw(FViewport* Viewport, FCanvas* Canvas)
{
	TSharedPtr<STextureMaskThumbnail> PinnedParent = Parent.Pin();
	if (!PinnedParent.IsValid())
	{
		return;
	}

	Canvas->Clear( FLinearColor::Black);

	if (UTexture2D* Texture = PinnedParent->Texture.Get(); Texture && !Texture->IsCompiling())
	{
		// Fully stream in the texture before drawing it.
		Texture->SetForceMipLevelsToBeResident(30.0f);
		Texture->WaitForStreaming();

		if (FTextureResource* TextureResource = Texture->GetResource())
		{
			//Draw the selected texture, uses ColourChannelBlend mode parameter to filter colour channels and apply grayscale
			FCanvasTileItem TileItem( FVector2D( 0.0f, 0.0f ), TextureResource, Viewport->GetSizeXY(), FLinearColor::White );
			TileItem.BlendMode = (ESimpleElementBlendMode)(SE_BLEND_RGBA_MASK_START + (1<<PinnedParent->TextureChannel.Get()) + 16);
			Canvas->DrawItem( TileItem );
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_AlphaBrush::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_AlphaBrush);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_AlphaBrush::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!IsBrushSetActive("BrushSet_Alpha") && !IsBrushSetActive("BrushSet_Pattern"))
	{
		return;
	}

	IDetailCategoryBuilder& BrushSettingsCategory = DetailBuilder.EditCategory("Brush Settings");

	TSharedRef<IPropertyHandle> PropertyHandle_AlphaTexture = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTexture));
	DetailBuilder.HideProperty(PropertyHandle_AlphaTexture);
	TSharedRef<IPropertyHandle> PropertyHandle_AlphaTextureChannel = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTextureChannel));
	DetailBuilder.HideProperty(PropertyHandle_AlphaTextureChannel);

	BrushSettingsCategory.AddProperty(PropertyHandle_AlphaTexture)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_AlphaTexture->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f) // copied from SPropertyEditorAsset::GetDesiredWidth
	.MaxDesiredWidth(0)
	[
		SNew(SAssetDropTarget)
		.OnAssetsDropped_Static(&FLandscapeEditorDetailCustomization_AlphaBrush::OnAssetDropped, PropertyHandle_AlphaTexture)
		.OnAreAssetsAcceptableForDrop_Static(&FLandscapeEditorDetailCustomization_AlphaBrush::OnAssetDraggedOver)
		.ToolTipText(PropertyHandle_AlphaTexture->GetToolTipText())
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f, 0.0f, 2.0f, 0.0f)
			.AutoWidth()
			[
				SNew(STextureMaskThumbnail)
				.Texture_Static(&FLandscapeEditorDetailCustomization_AlphaBrush::GetObjectPropertyValue<UTexture2D>, PropertyHandle_AlphaTexture)
				.TextureChannel_Static(&FLandscapeEditorDetailCustomization_AlphaBrush::GetPropertyValue<uint8>, PropertyHandle_AlphaTextureChannel)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SObjectPropertyEntryBox)
				.IsEnabled(true)
				.PropertyHandle(PropertyHandle_AlphaTexture)
				.AllowedClass(UTexture2D::StaticClass())
				.OnShouldFilterAsset_Lambda([](const FAssetData& AssetData) -> bool
				{
					// We cannot use cooked texture as parameter for now.
					if ((AssetData.PackageFlags & PKG_Cooked) != 0)
					{
						return true;
					}

					const UTexture2D* Texture = Cast<UTexture2D>(AssetData.GetAsset());

					return (Texture == nullptr) || !Texture->Source.IsValid();
				})
				.AllowClear(false)
			]
		]
	];

	BrushSettingsCategory.AddProperty(PropertyHandle_AlphaTextureChannel);

	if (IsBrushSetActive("BrushSet_Pattern"))
	{
		TSharedRef<IPropertyHandle> PropertyHandle_bUseWorldSpacePatternBrush = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseWorldSpacePatternBrush));
		TSharedRef<IPropertyHandle> PropertyHandle_AlphaBrushScale    = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushScale));
		TSharedRef<IPropertyHandle> PropertyHandle_AlphaBrushRotation = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushRotation));
		TSharedRef<IPropertyHandle> PropertyHandle_AlphaBrushPanU     = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanU));
		TSharedRef<IPropertyHandle> PropertyHandle_AlphaBrushPanV     = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanV));

		auto NonWorld_Visibility = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([=]() { return GetPropertyValue<bool>(PropertyHandle_bUseWorldSpacePatternBrush) ? EVisibility::Collapsed : EVisibility::Visible;   } ));
		auto World_Visibility    = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([=]() { return GetPropertyValue<bool>(PropertyHandle_bUseWorldSpacePatternBrush) ? EVisibility::Visible   : EVisibility::Collapsed; } ));

		BrushSettingsCategory.AddProperty(PropertyHandle_bUseWorldSpacePatternBrush);
		BrushSettingsCategory.AddProperty(PropertyHandle_AlphaBrushScale)
			.Visibility(NonWorld_Visibility);
		BrushSettingsCategory.AddProperty(PropertyHandle_AlphaBrushRotation)
			.Visibility(NonWorld_Visibility);
		BrushSettingsCategory.AddProperty(PropertyHandle_AlphaBrushPanU)
			.Visibility(NonWorld_Visibility);
		BrushSettingsCategory.AddProperty(PropertyHandle_AlphaBrushPanV)
			.Visibility(NonWorld_Visibility);

		TSharedRef<IPropertyHandle> PropertyHandle_WorldSpacePatternBrushSettings = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WorldSpacePatternBrushSettings));
		PropertyHandle_WorldSpacePatternBrushSettings->MarkHiddenByCustomization();
		uint32 NumWorldSpaceSettings = 0;
		PropertyHandle_WorldSpacePatternBrushSettings->GetNumChildren(NumWorldSpaceSettings);
		for (uint32 i = 0; i < NumWorldSpaceSettings; i++)
		{
			BrushSettingsCategory.AddProperty(PropertyHandle_WorldSpacePatternBrushSettings->GetChildHandle(i))
				.Visibility(World_Visibility);
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FLandscapeEditorDetailCustomization_AlphaBrush::OnAssetDraggedOver(TArrayView<FAssetData> InAssets)
{
	return Cast<UTexture2D>(InAssets[0].GetAsset()) != nullptr;
}

void FLandscapeEditorDetailCustomization_AlphaBrush::OnAssetDropped(const FDragDropEvent&, TArrayView<FAssetData> InAssets, TSharedRef<IPropertyHandle> PropertyHandle_AlphaTexture)
{
	ensure(PropertyHandle_AlphaTexture->SetValue(InAssets[0].GetAsset()) == FPropertyAccess::Success);
}

#undef LOCTEXT_NAMESPACE
