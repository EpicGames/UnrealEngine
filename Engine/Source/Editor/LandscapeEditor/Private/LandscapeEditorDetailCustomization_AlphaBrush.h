// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class IDetailLayoutBuilder;

//#include "LandscapeEditorObject.h"

/**
 * Slate widgets customizer for the landscape brushes
 */

class FLandscapeEditorDetailCustomization_AlphaBrush : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	static bool OnAssetDraggedOver(TArrayView<FAssetData> InAssets);
	static void OnAssetDropped(const FDragDropEvent&, TArrayView<FAssetData> InAssets, TSharedRef<IPropertyHandle> PropertyHandle_AlphaTexture);
};
