// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_Base.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LandscapeEdMode.h"

namespace LandscapeCustomization
{
	FEdModeLandscape* GetEditorMode()
	{
		return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
	}

	bool IsToolActive(FName ToolName)
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (LandscapeEdMode != NULL && LandscapeEdMode->CurrentTool != NULL)
		{
			const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();
			return CurrentToolName == ToolName;
		}

		return false;
	}

	bool IsBrushSetActive(FName BrushSetName)
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (LandscapeEdMode && LandscapeEdMode->CurrentBrushSetIndex >= 0)
		{
			const FName CurrentBrushSetName = LandscapeEdMode->LandscapeBrushSets[LandscapeEdMode->CurrentBrushSetIndex].BrushSetName;
			return CurrentBrushSetName == BrushSetName;
		}

		return false;
	}
}

FEdModeLandscape* FLandscapeEditorDetailCustomization_Base::GetEditorMode()
{
	return LandscapeCustomization::GetEditorMode();
}

bool FLandscapeEditorDetailCustomization_Base::IsToolActive(FName ToolName)
{
	return LandscapeCustomization::IsToolActive(ToolName);
}

bool FLandscapeEditorDetailCustomization_Base::IsBrushSetActive(FName BrushSetName)
{
	return LandscapeCustomization::IsBrushSetActive(BrushSetName);
}

//////////////////////////////////////////////////////////////////////////

FEdModeLandscape* FLandscapeEditorStructCustomization_Base::GetEditorMode()
{
	return LandscapeCustomization::GetEditorMode();
}

bool FLandscapeEditorStructCustomization_Base::IsToolActive(FName ToolName)
{
	return LandscapeCustomization::IsToolActive(ToolName);
}


