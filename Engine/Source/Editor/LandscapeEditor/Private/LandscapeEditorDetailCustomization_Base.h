// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/Casts.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

class FEdModeLandscape;
class IDetailLayoutBuilder;

/**
 * Slate widgets customizers for the Landscape Editor
 */
class FLandscapeEditorDetailCustomization_Base : public IDetailCustomization //, public TSharedFromThis<FLandscapeEditorDetailCustomization_Base>
{
public:
	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override = 0;

protected:
	static FEdModeLandscape* GetEditorMode();
	static bool IsToolActive(FName ToolName);
	static bool IsBrushSetActive(FName BrushSetName);

	template<typename type>
	static TOptional<type> OnGetValue(TSharedRef<IPropertyHandle> PropertyHandle);
	
	template<typename type>
	static void OnValueChanged(type NewValue, TSharedRef<IPropertyHandle> PropertyHandle);
	
	template<typename type>
	static void OnValueCommitted(type NewValue, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> PropertyHandle);

	template<typename type>
	static type GetPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle);

	template<typename type>
	static TOptional<type> GetOptionalPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle);

	template<typename type>
	static type* GetObjectPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle);

	static FText GetPropertyValueText(TSharedRef<IPropertyHandle> PropertyHandle);

	template<typename type>
	static void SetPropertyValue(type NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle);
};

template<typename type>
TOptional<type> FLandscapeEditorDetailCustomization_Base::OnGetValue(TSharedRef<IPropertyHandle> PropertyHandle)
{
	type Value;
	if (ensure(PropertyHandle->GetValue(Value) == FPropertyAccess::Success))
	{
		return TOptional<type>(Value);
	}

	// Value couldn't be accessed. Return an unset value
	return TOptional<type>();
}

template<typename type>
void FLandscapeEditorDetailCustomization_Base::OnValueChanged(type NewValue, TSharedRef<IPropertyHandle> PropertyHandle)
{
	const EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::InteractiveChange;
	ensure(PropertyHandle->SetValue(NewValue, Flags) == FPropertyAccess::Success);
}

template<typename type>
void FLandscapeEditorDetailCustomization_Base::OnValueCommitted(type NewValue, ETextCommit::Type CommitType, TSharedRef<IPropertyHandle> PropertyHandle)
{
	ensure(PropertyHandle->SetValue(NewValue) == FPropertyAccess::Success);
}

template<typename type>
type FLandscapeEditorDetailCustomization_Base::GetPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle)
{
	type Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		return Value;
	}

	// Couldn't get, return null / 0
	return type{};
}

template<typename type>
TOptional<type> FLandscapeEditorDetailCustomization_Base::GetOptionalPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle)
{
	type Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		return Value;
	}

	// Couldn't get, return unset optional
	return TOptional<type>();
}

template<typename type>
type* FLandscapeEditorDetailCustomization_Base::GetObjectPropertyValue(TSharedRef<IPropertyHandle> PropertyHandle)
{
	UObject* Value;
	if (PropertyHandle->GetValue(Value) == FPropertyAccess::Success)
	{
		return Cast<type>(Value);
	}

	// Couldn't get, return null
	return nullptr;
}

inline FText FLandscapeEditorDetailCustomization_Base::GetPropertyValueText(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FString Value;
	if (PropertyHandle->GetValueAsFormattedString(Value) == FPropertyAccess::Success)
	{
		return FText::FromString(Value);
	}

	return FText();
}

template<typename type>
void FLandscapeEditorDetailCustomization_Base::SetPropertyValue(type NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle)
{
	ensure(PropertyHandle->SetValue(NewValue) == FPropertyAccess::Success);
}

class FLandscapeEditorStructCustomization_Base : public IPropertyTypeCustomization
{
protected:
	static FEdModeLandscape* GetEditorMode();
	static bool IsToolActive(FName ToolName);
};
