// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_Blueprint.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditLayer.h"
#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.Blueprint"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_Blueprint::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_Blueprint);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_Blueprint::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& ToolsCategory = DetailBuilder.EditCategory("Tool Settings");

	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->GetLandscape() && LandscapeEdMode->CurrentToolMode)
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetCurrentEditLayerConst();
		check(EditLayer != nullptr);
		const bool bSupportsBPBrushes = EditLayer->SupportsBlueprintBrushes();

		const FText EnabledToolTip = LOCTEXT("LandscapeBlueprintBrush_EnabledToolTip", "Selects the blueprint brush to apply to the current edit layer. Click on the landscape to apply it.");
		const FText DisabledToolTip = FText::Format(LOCTEXT("LandscapeBlueprintBrush_DisabledToolTip", "Cannot add blueprint brush : the type of layer {0} ({1}) doesn't support blueprint brushes."),
			FText::FromName(EditLayer->GetName()), EditLayer->GetClass()->GetDisplayNameText());

		TSharedRef<IPropertyHandle> PropertyHandle_Blueprint = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BlueprintBrush));
		ToolsCategory.AddProperty(PropertyHandle_Blueprint)
			.IsEnabled(bSupportsBPBrushes)
			.ToolTip(bSupportsBPBrushes ? EnabledToolTip : DisabledToolTip);
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE