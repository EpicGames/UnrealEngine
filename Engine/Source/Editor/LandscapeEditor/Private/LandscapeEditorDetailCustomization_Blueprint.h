// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class IDetailLayoutBuilder;
class IDetailCustomization;

/**
 * Slate widgets customizer for the "Blueprint Brush" Landscape tool
 */
class FLandscapeEditorDetailCustomization_Blueprint : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

