// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_CopyPaste.h"
#include "Widgets/Text/STextBlock.h"
#include "Misc/MessageDialog.h"
#include "Misc/FileHelper.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdMode.h"
#include "LandscapeGizmoActiveActor.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailGroup.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

#include "ScopedTransaction.h"
#include "DesktopPlatformModule.h"

#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"
#include "Editor.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.CopyPaste"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_CopyPaste::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_CopyPaste);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_CopyPaste::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!IsToolActive("CopyPaste"))
	{
		return;
	}

	// Each time Ctrl-C/Ctrl-V are used the CopyPaste customization re-runs. The GuessedDimensionComboList is 
	// stored within this object so it resets automatically. When there is a valid file path, regenerate the dimensions each draw
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		if (!LandscapeEdMode->UISettings->GizmoHeightmapFilenameString.IsEmpty())
		{
			GenerateGuessDimensionList();
		}
	}

	IDetailCategoryBuilder& ToolsCategory = DetailBuilder.EditCategory("Tool Settings");

	ToolsCategory.AddCustomRow(LOCTEXT("CopyToGizmo", "Copy Data to Gizmo"))
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("CopyToGizmo.Tooltip", "Copies the data within the gizmo bounds to the gizmo taking into account any masking from selected regions."))
		.Text(LOCTEXT("CopyToGizmo", "Copy Data to Gizmo"))
		.HAlign(HAlign_Center)
		.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnCopyToGizmoButtonClicked)
	];

	ToolsCategory.AddCustomRow(LOCTEXT("FitGizmoToSelection", "Fit Gizmo to Selected Regions"))
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("FitGizmoToSelection.Tooltip", "Positions and resizes the gizmo so that it completely encompasses all region selections."))
		.Text(LOCTEXT("FitGizmoToSelection", "Fit Gizmo to Selected Regions"))
		.HAlign(HAlign_Center)
		.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnFitGizmoToSelectionButtonClicked)
	];

	ToolsCategory.AddCustomRow(LOCTEXT("FitHeightsToGizmo", "Fit Height Values to Gizmo Size"))
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("FitHeightsToGizmo.Tooltip", "Scales the data in the gizmo to fit the gizmo's Z size"))
		.Text(LOCTEXT("FitHeightsToGizmo", "Fit Height Values to Gizmo Size"))
		.HAlign(HAlign_Center)
		.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnFitHeightsToGizmoButtonClicked)
	];

	ToolsCategory.AddCustomRow(LOCTEXT("ClearGizmoData", "Clear Gizmo Data"))
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("ClearGizmoData.Tooltip", "Clears the gizmo of any copied data."))
		.Text(LOCTEXT("ClearGizmoData", "Clear Gizmo Data"))
		.HAlign(HAlign_Center)
		.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnClearGizmoDataButtonClicked)
	];

	IDetailGroup& GizmoImportExportGroup = ToolsCategory.AddGroup("Gizmo Import / Export", LOCTEXT("ImportExportTitle", "Gizmo Import / Export"), true);

	TSharedRef<IPropertyHandle> PropertyHandle_Heightmap = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, GizmoHeightmapFilenameString));
	DetailBuilder.HideProperty(PropertyHandle_Heightmap);
	GizmoImportExportGroup.AddPropertyRow(PropertyHandle_Heightmap)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_Heightmap->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			PropertyHandle_Heightmap->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		//.Padding(0,0,12,0) // Line up with the other properties due to having no reset to default button
		[
			SNew(SButton)
			.ContentPadding(FMargin(4, 0))
			.Text(NSLOCTEXT("UnrealEd", "GenericOpenDialog", "..."))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoHeightmapFilenameButtonClicked, PropertyHandle_Heightmap)
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_ImportSize = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, GizmoImportSize));
	TSharedRef<IPropertyHandle> PropertyHandle_ImportSize_X = PropertyHandle_ImportSize->GetChildHandle("X").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ImportSize_Y = PropertyHandle_ImportSize->GetChildHandle("Y").ToSharedRef();
	DetailBuilder.HideProperty(PropertyHandle_ImportSize);
	GizmoImportExportGroup.AddPropertyRow(PropertyHandle_ImportSize)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_ImportSize->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.LabelVAlign(VAlign_Center)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue(8192)
			.MinSliderValue(1)
			.MaxSliderValue(8192)
			.AllowSpin(true)
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.Value_Static(&FLandscapeEditorDetailCustomization_Base::OnGetValue<int32>, PropertyHandle_ImportSize_X)
			.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<int32>, PropertyHandle_ImportSize_X)
			.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<int32>, PropertyHandle_ImportSize_X)
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoGuessSizeButtonIsEnabled)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(FText::FromString(FString().AppendChar(0xD7))) // Multiply sign
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.LabelVAlign(VAlign_Center)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue(8192)
			.MinSliderValue(1)
			.MaxSliderValue(8192)
			.AllowSpin(true)
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.Value_Static(&FLandscapeEditorDetailCustomization_Base::OnGetValue<int32>, PropertyHandle_ImportSize_Y)
			.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<int32>, PropertyHandle_ImportSize_Y)
			.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<int32>, PropertyHandle_ImportSize_Y)
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoGuessSizeButtonIsEnabled)
		]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2, 0)
			.VAlign(VAlign_Center) 
			.HAlign(HAlign_Left)
			[
				SNew(SComboButton)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.ForegroundColor(FSlateColor::UseForeground())
				.CollapseMenuOnParentFocus(true)
				.IsEnabled(this, &FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoGuessSizeButtonIsEnabled)
				.ToolTipText(LOCTEXT("GuessSizeGizmoData.Tooltip", "Generated possible size from the specified file to import. If your size is not include, please fill the values manually."))
				.MenuContent()
				[
					SNew(SListView<TSharedPtr<FString>>)
					.ListItemsSource(&GuessedDimensionComboList)
					.SelectionMode(ESelectionMode::Type::Single)
					.OnGenerateRow_Lambda([](TSharedPtr<FString> InItem, const TSharedRef<STableViewBase>& Owner)
					{
					return	SNew(STableRow<TSharedPtr<FString>>, Owner)
							.Padding(FMargin(16, 4, 16, 4))
							[
								SNew(STextBlock).Text(FText::FromString(*InItem))
							];
					})
					.OnSelectionChanged_Lambda([this](TSharedPtr<FString> InItem, ESelectInfo::Type)
					{
						if (InItem.IsValid())
						{
							CurrentGuessedDimension = *InItem.Get();
							
							TArray<FString> SeparatedValues;
							CurrentGuessedDimension.ParseIntoArray(SeparatedValues, TEXT(" "));

							check(SeparatedValues.Num() == 3); // it should contain Number1 x Number2

							int32 FirstNumber = FCString::Atoi(*SeparatedValues[0]);
							int32 SecondNumber = FCString::Atoi(*SeparatedValues[2]);

							FEdModeLandscape* LandscapeEdMode = GetEditorMode();
							if (LandscapeEdMode != nullptr)
							{
								LandscapeEdMode->UISettings->GizmoImportSize.X = FirstNumber;
								LandscapeEdMode->UISettings->GizmoImportSize.Y = SecondNumber;
							}
						}
					})
				]
			]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_ImportTargetLayers = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, GizmoImportLayers));
	DetailBuilder.HideProperty(PropertyHandle_ImportTargetLayers);
	GizmoImportExportGroup.AddPropertyRow(PropertyHandle_ImportTargetLayers);

	GizmoImportExportGroup.AddWidgetRow()
	.FilterString(LOCTEXT("GizmoImportExport", "ImportExport"))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.Text(LOCTEXT("GizmoImport", "Import"))
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoImportButtonIsEnabled)
			.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoImportButtonClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.Text(LOCTEXT("GizmoExport", "Export"))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoExportButtonClicked)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnCopyToGizmoButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->CopyDataToGizmo();
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnFitGizmoToSelectionButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		ALandscapeGizmoActiveActor* Gizmo = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo && Gizmo->TargetLandscapeInfo.IsValid())
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeGizmo_FitToSelection", "Fit gizmo size to selection"));
			Gizmo->Modify();
			Gizmo->FitToSelection();
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnFitHeightsToGizmoButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		ALandscapeGizmoActiveActor* Gizmo = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo && Gizmo->TargetLandscapeInfo.IsValid())
		{
			FScopedTransaction Transaction(LOCTEXT("LandscapeGizmo_FitMinMaxHeight", "Set gizmo height to fix contained data"));
			Gizmo->Modify();
			Gizmo->FitMinMaxHeight();
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnClearGizmoDataButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		ALandscapeGizmoActiveActor* Gizmo = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo && Gizmo->TargetLandscapeInfo.IsValid())
		{
			Gizmo->ClearGizmoData();
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoHeightmapFilenameButtonClicked(TSharedRef<IPropertyHandle> HeightmapPropertyHandle)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		// Prompt the user for the Filenames
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
		if (DesktopPlatform != nullptr)
		{
			TArray<FString> OpenFilenames;
			bool bOpened = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				NSLOCTEXT("UnrealEd", "Import", "Import").ToString(),
				LandscapeEdMode->UISettings->LastImportPath,
				TEXT(""),
				TEXT("Raw Heightmap files (*.raw,*.r16)|*.raw;*.r16|All files (*.*)|*.*"),
				EFileDialogFlags::None,
				OpenFilenames);

			if (bOpened)
			{
				FString CurrentValue;
				HeightmapPropertyHandle->GetValue(CurrentValue);

				if (!CurrentValue.Equals(OpenFilenames[0]))
				{
					HeightmapPropertyHandle->SetValue(OpenFilenames[0]);
					LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(OpenFilenames[0]);
					LandscapeEdMode->UISettings->GizmoImportSize.X = 0;
					LandscapeEdMode->UISettings->GizmoImportSize.Y = 0;

					GenerateGuessDimensionList();
				}
			}
		}
	}

	return FReply::Handled();
}

bool FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoImportButtonIsEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		if (!LandscapeEdMode->UISettings->GizmoHeightmapFilenameString.IsEmpty() && LandscapeEdMode->UISettings->GizmoImportSize != FIntPoint(0, 0))
		{
			return true;
		}
	}

	return false;
}

bool FLandscapeEditorDetailCustomization_CopyPaste::GetGizmoGuessSizeButtonIsEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		if (!LandscapeEdMode->UISettings->GizmoHeightmapFilenameString.IsEmpty())
		{
			return true;
		}
	}

	return false;
}

void FLandscapeEditorDetailCustomization_CopyPaste::GenerateGuessDimensionList()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode != nullptr)
	{
		check(!LandscapeEdMode->UISettings->GizmoHeightmapFilenameString.IsEmpty());

		int64 GizmoFileSize = IFileManager::Get().FileSize(*LandscapeEdMode->UISettings->GizmoHeightmapFilenameString) / 2;
		TMap<uint32, int32> PrimeNumberCount; // <Prime Number, Count>

		if (GizmoFileSize != INDEX_NONE)
		{
			int64 InitialGizmoDimension = GizmoFileSize;
			int64 MaxGizmoDimension = InitialGizmoDimension;

			for (int32 i = 2; i <= MaxGizmoDimension; ++i)
			{
				int32 Count = 0;

				while (MaxGizmoDimension % i == 0)
				{
					++Count;
					MaxGizmoDimension /= i;
				}

				if (Count > 0)
				{
					PrimeNumberCount.Add(i, Count);
				}
			}

			TArray<uint32> FinalValuesLeft; 
			TArray<uint32> PreviousIterValues;
			PreviousIterValues.Add(1); // Default set

			for (TMap<uint32, int32>::TIterator It(PrimeNumberCount); It; ++It)
			{
				TArray<uint32> CurrentValues;
				for (int32 i = 0; i <= It.Value(); ++i)
				{
					CurrentValues.Add(static_cast<uint32>(FMath::Pow(static_cast<float>(It.Key()), i)));
				}				

				for (int32 i = 0; i < PreviousIterValues.Num(); ++i)
				{
					for (int32 j = 0; j < CurrentValues.Num(); ++j)
					{
						int32 Value = PreviousIterValues[i] * CurrentValues[j];

						if (Value > 0)
						{
							FinalValuesLeft.AddUnique(Value);
						}
					}
				}

				PreviousIterValues = FinalValuesLeft;
			}

			FinalValuesLeft.Sort();

			TArray<uint32> FinalValuesRight;

			for (int32 i = 0; i < FinalValuesLeft.Num(); ++i)
			{
				FinalValuesRight.Add(static_cast<uint32>(InitialGizmoDimension / FinalValuesLeft[i]));
			}

			GuessedDimensionComboList.Empty(FinalValuesLeft.Num());

			for (int32 i = 0; i < FinalValuesLeft.Num(); ++i)
			{
				float LeftValue = (float)FinalValuesLeft[i];
				float RightValue = (float)FinalValuesRight[i];

				// Remove extreme values as it's quite unlikely user use those settings
				if ((LeftValue <= RightValue && LeftValue / RightValue > 0.001f) || (LeftValue > RightValue && RightValue / LeftValue > 0.001f))
				{
					GuessedDimensionComboList.Add(MakeShared<FString>(FString::Printf(TEXT("%d x %d"), FinalValuesLeft[i], FinalValuesRight[i])));
				}
			}
		}
	}
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoImportButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		ALandscapeGizmoActiveActor* Gizmo = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo)
		{
			TArray<uint8> Data;

			FFileHelper::LoadFileToArray(Data, *LandscapeEdMode->UISettings->GizmoHeightmapFilenameString);

			if (Data.Num() <= 0
				|| Data.Num() != (LandscapeEdMode->UISettings->GizmoImportSize.X * LandscapeEdMode->UISettings->GizmoImportSize.Y * sizeof(uint16)))
			{
				const FText MessageFormat = NSLOCTEXT("UnrealEd", "LandscapeImport_BadHeightmapSize", "File size does not match.\nExpected {0} entries but file contains {1}.");
				const FText Message = FText::Format(MessageFormat, 
					FText::AsNumber(LandscapeEdMode->UISettings->GizmoImportSize.X * LandscapeEdMode->UISettings->GizmoImportSize.Y), 
					FText::AsNumber(Data.Num() / 2));
				FMessageDialog::Open(EAppMsgType::Ok, Message);
				return FReply::Handled();
			}

			TArray<ULandscapeLayerInfoObject*> LayerInfos;
			TArray<TArray<uint8> > LayerDataArrays;
			TArray<uint8*> LayerDataPtrs;

			for (int32 TargetLayerIndex = 0; TargetLayerIndex < LandscapeEdMode->UISettings->GizmoImportLayers.Num(); TargetLayerIndex++)
			{
				const FGizmoImportLayer& TargetLayer = LandscapeEdMode->UISettings->GizmoImportLayers[TargetLayerIndex];
				FString TargetLayerName = TargetLayer.LayerName.Replace(TEXT(" "), TEXT(""));
				if (TargetLayerName.IsEmpty() && !TargetLayer.bNoImport)
				{
					FMessageDialog::Open(EAppMsgType::Ok,
						FText::Format(NSLOCTEXT("UnrealEd", "LandscapeImport_BadTargetLayerName", "You must enter a name for the target layer being imported from {0}."), FText::FromString(TargetLayer.LayerFilename)));
					return FReply::Handled();
				}

				if (!TargetLayer.LayerFilename.IsEmpty() && !TargetLayer.bNoImport)
				{
					TArray<uint8>* LayerData = new(LayerDataArrays)(TArray<uint8>);
					FFileHelper::LoadFileToArray(*LayerData, *TargetLayer.LayerFilename);

					if (LayerData->Num() != (LandscapeEdMode->UISettings->GizmoImportSize.X * LandscapeEdMode->UISettings->GizmoImportSize.Y))
					{
						const FText MessageFormat = NSLOCTEXT("UnrealEd", "LandscapeImport_BadTargetLayerSize", "Target Layer {0} file size does not match.\nExpected {1} entries but file contains {2}.");
						const FText Message = FText::Format(MessageFormat, 
							FText::FromString(TargetLayer.LayerFilename),
							FText::AsNumber(LandscapeEdMode->UISettings->GizmoImportSize.X * LandscapeEdMode->UISettings->GizmoImportSize.Y), 
							FText::AsNumber(LayerData->Num()));
						FMessageDialog::Open(EAppMsgType::Ok, Message);
						return FReply::Handled();
					}

					if (ULandscapeLayerInfoObject* LayerInfoObject = LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetLayerInfoByName(FName(*TargetLayerName)))
					{
						LayerInfos.Add(LayerInfoObject);
						LayerDataPtrs.Add(&(*LayerData)[0]);
					}
				}
			}

			Gizmo->Import(LandscapeEdMode->UISettings->GizmoImportSize.X, LandscapeEdMode->UISettings->GizmoImportSize.Y, (uint16*)Data.GetData(), LayerInfos, LayerDataPtrs.Num() ? LayerDataPtrs.GetData() : nullptr);

			// Make sure gizmo actor is selected
			GEditor->SelectNone(false, true);
			GEditor->SelectActor(Gizmo, true, false, true);
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_CopyPaste::OnGizmoExportButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		ALandscapeGizmoActiveActor* Gizmo = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
		if (Gizmo && Gizmo->TargetLandscapeInfo.IsValid() && Gizmo->SelectedData.Num())
		{
			int32 TargetIndex = -1;
			ULandscapeInfo* LandscapeInfo = Gizmo->TargetLandscapeInfo.Get();
			TArray<FString> Filenames;

			// Local set for export
			TSet<ULandscapeLayerInfoObject*> LayerInfoSet;
			for (int32 i = 0; i < Gizmo->LayerInfos.Num(); i++)
			{
				if (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap && LandscapeEdMode->CurrentToolTarget.LayerInfo == Gizmo->LayerInfos[i])
				{
					TargetIndex = i;
				}
				LayerInfoSet.Add(Gizmo->LayerInfos[i]);
			}

			for (int32 i = -1; i < Gizmo->LayerInfos.Num(); i++)
			{
				if (!LandscapeEdMode->UISettings->bApplyToAllTargets && i != TargetIndex)
				{
					continue;
				}
				FString SaveDialogTitle;
				FString DefaultFilename;
				FString FileTypes;

				if (i < 0)
				{
					if (!(Gizmo->DataType & LGT_Height))
					{
						continue;
					}
					SaveDialogTitle = NSLOCTEXT("UnrealEd", "LandscapeExport_HeightmapFilename", "Choose filename for Heightmap Export").ToString();
					DefaultFilename = TEXT("Heightmap.raw");
					FileTypes = TEXT("Heightmap .raw files|*.raw|Heightmap .r16 files|*.r16|All files|*.*");
				}
				else
				{
					if (!(Gizmo->DataType & LGT_Weight))
					{
						continue;
					}

					FName LayerName = Gizmo->LayerInfos[i]->GetLayerName();
					SaveDialogTitle = FText::Format(NSLOCTEXT("UnrealEd", "LandscapeExport_LayerFilename", "Choose filename for Layer {0} Export"), FText::FromString(LayerName.ToString())).ToString();
					DefaultFilename = FString::Printf(TEXT("%s.raw"), *LayerName.ToString());
					FileTypes = TEXT("Layer .raw files|*.raw|Layer .r8 files|*.r8|All files|*.*");
				}

				TArray<FString> SaveFilenames;
				IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
				bool bSave = false;
				if (DesktopPlatform)
				{
					bSave = DesktopPlatform->SaveFileDialog(
						FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
						SaveDialogTitle,
						LandscapeEdMode->UISettings->LastImportPath,
						DefaultFilename,
						FileTypes,
						EFileDialogFlags::None,
						SaveFilenames
						);
				}

				if (!bSave)
				{
					return FReply::Handled();
				}

				Filenames.Add(SaveFilenames[0]);
				LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(SaveFilenames[0]);
			}

			Gizmo->Export(TargetIndex, Filenames);
		}
	}

	return FReply::Handled();
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IPropertyTypeCustomization> FLandscapeEditorStructCustomization_FGizmoImportLayer::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorStructCustomization_FGizmoImportLayer);
}

void FLandscapeEditorStructCustomization_FGizmoImportLayer::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FLandscapeEditorStructCustomization_FGizmoImportLayer::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedRef<IPropertyHandle> PropertyHandle_LayerFilename = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmoImportLayer, LayerFilename)).ToSharedRef();
	ChildBuilder.AddProperty(PropertyHandle_LayerFilename)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_LayerFilename->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			PropertyHandle_LayerFilename->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		//.Padding(0,0,12,0) // Line up with the other properties due to having no reset to default button
		[
			SNew(SButton)
			.ContentPadding(FMargin(4, 0))
			.Text(NSLOCTEXT("UnrealEd", "GenericOpenDialog", "..."))
			.OnClicked(this, &FLandscapeEditorStructCustomization_FGizmoImportLayer::OnGizmoImportLayerFilenameButtonClicked, PropertyHandle_LayerFilename)
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_LayerName = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGizmoImportLayer, LayerName)).ToSharedRef();
	ChildBuilder.AddProperty(PropertyHandle_LayerName);
}

FReply FLandscapeEditorStructCustomization_FGizmoImportLayer::OnGizmoImportLayerFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerFilename)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}

	// Prompt the user for the Filenames
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform != nullptr)
	{
		TArray<FString> OpenFilenames;
		bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			NSLOCTEXT("UnrealEd", "Import", "Import").ToString(),
			LandscapeEdMode->UISettings->LastImportPath,
			TEXT(""),
			TEXT("Raw Layer files (*.raw,*.r16)|*.raw;*.r16|All files (*.*)|*.*"),
			EFileDialogFlags::None,
			OpenFilenames);

		if (bOpened)
		{
			PropertyHandle_LayerFilename->SetValue(OpenFilenames[0]);
			LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(OpenFilenames[0]);
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
