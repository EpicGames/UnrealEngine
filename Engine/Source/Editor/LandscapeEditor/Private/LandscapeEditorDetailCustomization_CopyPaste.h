// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class IDetailLayoutBuilder;

/**
 * Slate widgets customizer for the "Copy/Paste" tool
 */

class FLandscapeEditorDetailCustomization_CopyPaste : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:
	FReply OnCopyToGizmoButtonClicked();
	FReply OnFitGizmoToSelectionButtonClicked();
	FReply OnFitHeightsToGizmoButtonClicked();
	FReply OnClearGizmoDataButtonClicked();
	FReply OnGizmoHeightmapFilenameButtonClicked(TSharedRef<IPropertyHandle> HeightmapPropertyHandle);

	bool GetGizmoImportButtonIsEnabled() const;
	bool GetGizmoGuessSizeButtonIsEnabled() const;

	FReply OnGizmoImportButtonClicked();
	FReply OnGizmoExportButtonClicked();

private:
	TArray<TSharedPtr<FString>> GuessedDimensionComboList;
	FString CurrentGuessedDimension;

	void GenerateGuessDimensionList();
};

class FLandscapeEditorStructCustomization_FGizmoImportLayer : public FLandscapeEditorStructCustomization_Base
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

public:
	FReply OnGizmoImportLayerFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerFilename);
};
