// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_EditLayers.h"

#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditLayerCustomization.h"
#include "LandscapeEditLayer.h"
#include "LandscapeEditorUtils.h"
#include "Misc/MessageDialog.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.EditLayersCustomization"

FEdModeLandscape* FLandscapeEditLayerCustomizationCommon::GetEditorMode()
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

bool FLandscapeEditLayerCustomizationCommon::CanDeleteLayer(int32 InLayerIndex, FText& OutReason) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (!Landscape || Landscape->GetEditLayersConst().Num() <= 1)
	{
		OutReason = LOCTEXT("DeleteLayer_CantDeleteLastLayer", "The last layer cannot be deleted");
		return false;
	}

	const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
	if (EditLayer == nullptr)
	{
		return false;
	}

	if (EditLayer->IsLocked())
	{
		OutReason = LOCTEXT("DeleteLayer_CantDeleteLocked", "Cannot delete a locked edit layer");
		return false;
	}

	OutReason = LOCTEXT("DeleteLayer_CanDelete", "Delete the edit layer");
	return true;
}

void FLandscapeEditLayerCustomizationCommon::DeleteLayer(int32 InLayerIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape && Landscape->GetEditLayersConst().Num() > 1)
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		if (EditLayer)
		{
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("Landscape_DeleteLayer_Message", "The layer {0} will be deleted.  Continue?"), FText::FromName(EditLayer->GetName())));
			if (Result == EAppReturnType::Yes)
			{
				const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Delete", "Delete Layer"));
				// Delete layer will update the selected edit layer index
				Landscape->DeleteLayer(InLayerIndex);
				LandscapeEdMode->UpdateTargetList();
				LandscapeEdMode->RefreshDetailPanel();
			}
		}
	}
}

bool FLandscapeEditLayerCustomizationCommon::CanCollapseLayer(int32 InLayerIndex, FText& OutReason) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;

	if (Landscape == nullptr)
	{
		OutReason = LOCTEXT("Landscape_CollapseLayer_Reason_InvalidLandscape", "Landscape is invalid");
		return false;
	}

	if (Landscape->GetEditLayersConst().Num() <= 1)
	{
		OutReason = LOCTEXT("Landscape_CollapseLayer_Reason_NotEnoughLayersToCollapse", "Not enough layers to do collapse");
		return false;
	}

	if (InLayerIndex < 1)
	{
		OutReason = LOCTEXT("Landscape_CollapseLayer_Reason_CantCollapseBaseLayer", "Cannot collapse the first layer");
		return false;
	}

	const FLandscapeLayer* TopLayer = Landscape->GetLayerConst(InLayerIndex);
	const FLandscapeLayer* BottomLayer = Landscape->GetLayerConst(InLayerIndex - 1);
	if ((TopLayer == nullptr) || (BottomLayer == nullptr))
	{
		return false;
	}

	const ULandscapeEditLayerBase* TopEditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
	const ULandscapeEditLayerBase* BottomEditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex - 1);

	check((TopEditLayer != nullptr) && (BottomEditLayer != nullptr));

	if (!TopEditLayer->SupportsCollapsingTo())
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_TopLayerDoesntSupportCollapsing", "Cannot collapse layer '{0}' onto layer '{1}'. The type of layer '{0}' ({2}) doesn't support collapsing to another one"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()), TopEditLayer->GetClass()->GetDisplayNameText());
		return false;
	}

	if (!BottomEditLayer->SupportsBeingCollapsedAway())
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_BottomLayerDoesntSupportCollapsing", "Cannot collapse layer '{0}' onto layer '{1}'. The type of layer '{1}' ({2}) doesn't support being collapsed away"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()), BottomEditLayer->GetClass()->GetDisplayNameText());
		return false;
	}

	if (TopEditLayer->IsLocked())
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_TopLayerIsLocked", "Cannot collapse layer '{0}' onto layer '{1}'. Layer '{0}' will be deleted in the operation but it is currently locked"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()));
		return false;
	}

	if (BottomEditLayer->IsLocked())
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_BottomLayerIsLocked", "Cannot collapse layer '{0}' onto layer '{1}'. Destination layer '{1}' is currently locked"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()));
		return false;
	}

	// Can't collapse on layer that has a Brush because result will change...
	if (TopLayer->Brushes.Num() > 0)
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_TopLayerHasBrush", "Cannot collapse layer '{0}' onto layer '{1}'. Layer '{0}' contains brush(es)"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()));
		return false;
	}

	if (BottomLayer->Brushes.Num() > 0)
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_BottomLayerHasBrush", "Cannot collapse layer '{0}' onto layer '{1}'. Layer '{1}' contains brush(es)"),
			FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()));
		return false;
	}

	OutReason = FText::Format(LOCTEXT("Landscape_CollapseLayer_Reason_Collapse", "Collapse layer '{0}' onto layer '{1}'"),
		FText::FromName(TopEditLayer->GetName()), FText::FromName(BottomEditLayer->GetName()));
	return true;
}

void FLandscapeEditLayerCustomizationCommon::CollapseLayer(int32 InLayerIndex)
{
	FText Reason;
	check(CanCollapseLayer(InLayerIndex, Reason));

	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		ALandscape* Landscape = LandscapeEdMode->GetLandscape();
		const ULandscapeEditLayerBase* Layer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		const ULandscapeEditLayerBase* BaseLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex - 1);

		if (Landscape && Layer && BaseLayer)
		{
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("Landscape_CollapseLayer_Message", "The layer '{0}' will be collapsed into layer '{1}'.  Continue?"), FText::FromName(Layer->GetName()), FText::FromName(BaseLayer->GetName())));
			if (Result == EAppReturnType::Yes)
			{
				const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Collapse", "Collapse Layer"));
				Landscape->CollapseLayer(InLayerIndex);
				OnLayerSelectionChanged(InLayerIndex - 1);
				LandscapeEdMode->RefreshDetailPanel();
			}
		}
	}
}

bool FLandscapeEditLayerCustomizationCommon::CanCollapseAllLayers(FText& OutReason) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape == nullptr)
	{
		OutReason = LOCTEXT("Landscape_CollapseLayer_Reason_InvalidLandscape", "Landscape is invalid");
		return false;
	}

	if (Landscape->GetEditLayersConst().Num() <= 1)
	{
		OutReason = LOCTEXT("Landscape_CollapseLayer_Reason_NotEnoughLayersToCollapse", "Not enough layers to do collapse");
		return false;
	}

	const ULandscapeEditLayerBase* BaseEditLayer = Landscape->GetEditLayerConst(0);
	check(BaseEditLayer);

	if (!BaseEditLayer->SupportsBeingCollapsedAway())
	{
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseAllLayers_Reason_DefaultLayerDoesntSupportCollapsing", "Cannot collapse all layers onto layer '{0}'. The type of layer '{1}' doesn't support being collapsed away"),
			FText::FromName(BaseEditLayer->GetName()), BaseEditLayer->GetClass()->GetDisplayNameText());
		return false;
	}

	TArray<FString> InvalidEditLayerNames;
	for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
	{
		check(EditLayer != nullptr);

		if (EditLayer->IsLocked())
		{
			InvalidEditLayerNames.Add(EditLayer->GetName().ToString());
		}
	}

	if (InvalidEditLayerNames.Num() > 0)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Count"), InvalidEditLayerNames.Num());
		Args.Add(TEXT("NameList"), FText::FromString(FString::Join(InvalidEditLayerNames, TEXT(", "))));
		OutReason = FText::Format(LOCTEXT("Landscape_CollapseAllLayers_Reason_LayerLocked", 
			"Cannot collapse all layers. '{NameList}' {Count}|plural(one=is,other=are) included in the collapse but {Count}|plural(one=is,other=are) locked."), Args);
		return false;
	}
	
	OutReason = FText::Format(LOCTEXT("Landscape_CollapseAllLayers_Reason_Collapse", "Collapse all layers into '{0}', preserving data while removing all Blueprint Brushes and hidden edit layer data."), 
		FText::FromName(BaseEditLayer->GetName()));
	return true;
}

void FLandscapeEditLayerCustomizationCommon::CollapseAllEditLayers()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		ALandscape* Landscape = LandscapeEdMode->GetLandscape();
		const ULandscapeEditLayerBase* BaseLayer = LandscapeEdMode->GetEditLayerConst(0);

		if (Landscape && BaseLayer)
		{
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("Landscape_CollapseAllLayer_Message", "All edit layer data will be collapsed into layer '{0}'. Blueprint brushes and hidden edit layer data will be removed. Continue?"),
				FText::FromName(BaseLayer->GetName())));

			if (Result == EAppReturnType::Yes)
			{
				const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Collapse_All", "Collapse All Layers"));
				Landscape->CollapseAllEditLayers();
				OnLayerSelectionChanged(0);
				LandscapeEdMode->RefreshDetailPanel();
			}
		}
	}
}

bool FLandscapeEditLayerCustomizationCommon::CanToggleVisibility(int32 InLayerIndex, FText& OutReason) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		if (EditLayer == nullptr)
		{
			return false;
		}

		if (EditLayer->IsLocked())
		{
			OutReason = LOCTEXT("ToggleVisibility_CantToggleLocked", "Cannot change the visibility of a locked edit layer");
			return false;
		}

		OutReason = LOCTEXT("ToggleVisibility_CanToggle", "Toggle the visibility of the edit layer");
	}

	return true;
}

FReply FLandscapeEditLayerCustomizationCommon::OnToggleVisibility(int32 InLayerIndex)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_SetVisibility", "Set Layer Visibility"));

		ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(InLayerIndex);
		check(EditLayer != nullptr);
		EditLayer->SetVisible(!EditLayer->IsVisible(), /*bInModify= */true);

		if (EditLayer->IsVisible())
		{
			OnLayerSelectionChanged(InLayerIndex);
		}
	}
	return FReply::Handled();
}

const FSlateBrush* FLandscapeEditLayerCustomizationCommon::GetVisibilityBrushForLayer(int32 InLayerIndex) const
{
	bool bIsVisible = false;

	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		check(EditLayer != nullptr);

		bIsVisible = EditLayer->IsVisible();
	}

	return bIsVisible ? FAppStyle::GetBrush("Level.VisibleIcon16x") : FAppStyle::GetBrush("Level.NotVisibleIcon16x");
}

void FLandscapeEditLayerCustomizationCommon::OnLayerSelectionChanged(int32 InLayerIndex)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->GetSelectedEditLayerIndex() != InLayerIndex)
	{
		LandscapeEdMode->SetSelectedEditLayer(InLayerIndex);
		LandscapeEdMode->UpdateTargetList();
	}
}

void FLandscapeEditLayerCustomizationCommon::ShowOnlySelectedLayer(int32 InLayerIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		const FScopedTransaction Transaction(LOCTEXT("ShowOnlySelectedLayer", "Show Only Selected Layer"));
		Landscape->ShowOnlySelectedLayer(InLayerIndex);
		OnLayerSelectionChanged(InLayerIndex);
	}
}

void FLandscapeEditLayerCustomizationCommon::ShowAllLayers()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		const FScopedTransaction Transaction(LOCTEXT("ShowAllLayers", "Show All Layers"));
		Landscape->ShowAllLayers();
	}
}


TOptional<float> FLandscapeEditLayerCustomizationCommon::GetLayerAlpha(int32 InLayerIndex) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		check(EditLayer != nullptr);

		return EditLayer->GetAlphaForTargetType(LandscapeEdMode->GetLandscapeToolTargetType());
	}

	return 1.0f;
}

float FLandscapeEditLayerCustomizationCommon::GetLayerAlphaMinValue() const
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->GetLandscapeToolTargetType() == ELandscapeToolTargetType::Heightmap)
	{
		return -1.0f;
	}
	else
	{
		return 0.0f;
	}
}

bool FLandscapeEditLayerCustomizationCommon::CanSetLayerAlpha(int32 InLayerIndex, FText& OutReason) const
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode == nullptr)
	{
		return false;
	}

	const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
	if (EditLayer == nullptr)
	{
		return false;
	}

	if (!EditLayer->SupportsAlphaForTargetType(LandscapeEdMode->GetLandscapeToolTargetType()))
	{
		OutReason = FText::Format(LOCTEXT("SetLayerAlpha_LayerDoesntSupportAlpha", "Cannot change alpha : the type of layer {0} ({1}) doesn't support alpha"),
			FText::FromName(EditLayer->GetName()), EditLayer->GetClass()->GetDisplayNameText());
		return false;
	}


	if (EditLayer->IsLocked())
	{
		OutReason = LOCTEXT("SetLayerAlpha_LayerIsLocked", "Cannot change the alpha of a locked edit layer");
		return false;
	}

	OutReason = LOCTEXT("SetLayerAlpha_CanSet", "Set the edit layer's alpha");
	return true;
}

void FLandscapeEditLayerCustomizationCommon::SetLayerAlpha(float InAlpha, int32 InLayerIndex, bool bCommit, int32 InSliderIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(InLayerIndex);
		check(EditLayer != nullptr);

		FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_SetAlpha", "Set Layer Alpha"), InSliderIndex == INDEX_NONE && bCommit);
		// Set Value when using slider or when committing text
		EditLayer->SetAlphaForTargetType(LandscapeEdMode->GetLandscapeToolTargetType(), InAlpha, /*bInModify = */true, bCommit ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive);
	}
}

TSharedRef<IEditLayerCustomization> FLandscapeEditLayerContextMenuCustomization_Base::MakeInstance()
{
	return MakeShareable(new FLandscapeEditLayerContextMenuCustomization_Base);
}

void FLandscapeEditLayerContextMenuCustomization_Base::CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;

	if (Landscape && LandscapeEdMode && InEditLayer)
	{
		const int32 EditLayerIndex = Landscape->GetLayerIndex(InEditLayer->GetGuid());

		const FName EditLayerCategory = FName("Edit Layers");
		OutContextMenuEntryMap.FindOrAdd(EditLayerCategory).SectionLabel = LOCTEXT("LandscapeEditLayerContextMenuCommon.Heading", "Edit Layers");

		// Delete Layer
		FMenuEntryParams DeleteLayerEntry;
		DeleteLayerEntry.DirectActions = FUIAction(
			FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Base::DeleteLayer, EditLayerIndex),
			FCanExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { FText Reason; return CanDeleteLayer(EditLayerIndex, Reason); }));
		DeleteLayerEntry.ToolTipOverride = MakeAttributeLambda([this, EditLayerIndex] {
			FText Reason;
			CanDeleteLayer(EditLayerIndex, Reason);
			return Reason; });
		DeleteLayerEntry.LabelOverride = LOCTEXT("DeleteLayer", "Delete");

		OutContextMenuEntryMap.FindOrAdd(EditLayerCategory).Entries.Add(DeleteLayerEntry);

		// Collapse Layer
		FMenuEntryParams CollapseLayerEntry;
		CollapseLayerEntry.DirectActions = FUIAction(
			FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Base::CollapseLayer, EditLayerIndex),
			FCanExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { FText Reason; return CanCollapseLayer(EditLayerIndex, Reason); }));
		CollapseLayerEntry.ToolTipOverride = MakeAttributeSPLambda(this, [this, EditLayerIndex] { FText Reason; CanCollapseLayer(EditLayerIndex, Reason); return Reason; });
		CollapseLayerEntry.LabelOverride = LOCTEXT("CollapseLayer", "Collapse");

		OutContextMenuEntryMap.FindOrAdd(EditLayerCategory).Entries.Add(CollapseLayerEntry);

		// Collapse All layers
		FMenuEntryParams CollapseAllLayersEntry;
		CollapseAllLayersEntry.DirectActions = FUIAction(
			FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Base::CollapseAllEditLayers),
			FCanExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { FText Reason; return CanCollapseAllLayers(Reason); }));
		CollapseAllLayersEntry.ToolTipOverride = MakeAttributeSPLambda(this, [this, EditLayerIndex] { FText Reason; CanCollapseAllLayers(Reason); return Reason; });
		CollapseAllLayersEntry.LabelOverride = LOCTEXT("CollapseAllLayers", "Collapse All Layers");

		OutContextMenuEntryMap.FindOrAdd(EditLayerCategory).Entries.Add(CollapseAllLayersEntry);
		
		// Visibility Section (new category)
		const FName VisibilityCategory = "Visibility";
		OutContextMenuEntryMap.FindOrAdd(VisibilityCategory).SectionLabel = LOCTEXT("LandscapeEditLayerContextMenuVisibility.Heading", "Visibility");

		// Show/Hide Selected Layer
		FText VisibilityText = LOCTEXT("ShowSelected", "Show Selected");
		if (InEditLayer->IsVisible())
		{
			// Hide Selected Layer
			VisibilityText = LOCTEXT("HideSelected", "Hide Selected");
		}
		FMenuEntryParams VisibilityEntry;
		VisibilityEntry.LabelOverride = VisibilityText;
		VisibilityEntry.ToolTipOverride =  MakeAttributeSPLambda(this, [this, EditLayerIndex] { FText Reason; CanToggleVisibility(EditLayerIndex, Reason); return Reason; });
		VisibilityEntry.DirectActions = FUIAction(FExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { OnToggleVisibility(EditLayerIndex); }),
			FCanExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { FText Reason; return CanToggleVisibility(EditLayerIndex, Reason); }));

		OutContextMenuEntryMap.FindOrAdd(VisibilityCategory).Entries.Add(VisibilityEntry);

		// Show Only Selected Layer
		FUIAction ShowOnlySelectedLayerAction = FUIAction(FExecuteAction::CreateSPLambda(this, [this, EditLayerIndex] { ShowOnlySelectedLayer(EditLayerIndex); }));
		FMenuEntryParams ShowOnlyEntry;
		ShowOnlyEntry.LabelOverride = LOCTEXT("ShowOnlySelected", "Show Only Selected");
		ShowOnlyEntry.ToolTipOverride = LOCTEXT("ShowOnlySelectedLayerTooltip", "Show Only Selected Layer");
		ShowOnlyEntry.DirectActions = ShowOnlySelectedLayerAction;

		OutContextMenuEntryMap.FindOrAdd(VisibilityCategory).Entries.Add(ShowOnlyEntry);

		// Show All Layers
		FUIAction ShowAllLayersAction = FUIAction(FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Base::ShowAllLayers));
		FMenuEntryParams ShowAllEntry;
		ShowAllEntry.LabelOverride = LOCTEXT("ShowAllLayers", "Show All Layers");
		ShowAllEntry.ToolTipOverride = LOCTEXT("ShowAllLayersTooltip", "Show All Layers");
		ShowAllEntry.DirectActions = ShowAllLayersAction;

		OutContextMenuEntryMap.FindOrAdd(VisibilityCategory).Entries.Add(ShowAllEntry);
	}
}

TSharedRef<IEditLayerCustomization> FLandscapeEditLayerContextMenuCustomization_Layer::MakeInstance()
{
	return MakeShareable(new FLandscapeEditLayerContextMenuCustomization_Layer);
}

void FLandscapeEditLayerContextMenuCustomization_Layer::CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape && LandscapeEdMode && InEditLayer)
	{
		const int32 EditLayerIndex = Landscape->GetLayerIndex(InEditLayer->GetGuid());
		const FName BrushCategory = FName("Brushes");
		OutContextMenuEntryMap.FindOrAdd(BrushCategory).SectionLabel = LOCTEXT("LandscapeEditLayerContextMenuBrushes.Heading", "Brushes");

		// Add unassigned brush to layer
		const TArray<ALandscapeBlueprintBrushBase*>& Brushes = LandscapeEdMode->GetBrushList();
		TArray<ALandscapeBlueprintBrushBase*> FilteredBrushes = Brushes.FilterByPredicate([](ALandscapeBlueprintBrushBase* Brush) { return Brush->GetOwningLandscape() == nullptr; });

		// If there are no unassigned brushes or the edit layer does not support brushes, show a disabled state instead of hiding the entire section
		if (FilteredBrushes.Num() > 0 && InEditLayer->SupportsBlueprintBrushes())
		{
			FMenuEntryParams AssignBrushEntry;
			AssignBrushEntry.LabelOverride = LOCTEXT("LandscapeEditorBrushAddSubMenu", "Assign Existing Brush");
			AssignBrushEntry.ToolTipOverride = LOCTEXT("LandscapeEditorBrushAddSubMenuToolTip", "To modify the terrain, brushes need to be assigned to a landscape actor. Add the brush to this edit layer");
			AssignBrushEntry.bIsSubMenu = true;
			AssignBrushEntry.EntryBuilder = FNewMenuDelegate::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Layer::FillUnassignedBrushMenu, FilteredBrushes, EditLayerIndex);

			OutContextMenuEntryMap.FindOrAdd(BrushCategory).Entries.Add(AssignBrushEntry);
		}
		else
		{
			FMenuEntryParams NoBrushEntry;
			NoBrushEntry.LabelOverride = LOCTEXT("LandscapeEditorBrushNone", "None"),
				NoBrushEntry.ToolTipOverride = InEditLayer->SupportsBlueprintBrushes() ? LOCTEXT("LandscapeEditorBrushAllBrushActorsAssigned", "All Blueprint Brush actors are assigned to a landscape edit layer")
				: FText::Format(LOCTEXT("LandscapeEditorBrushUnsupported", "This layer's type ({0}) doesn't support blueprint brushes."), InEditLayer->GetClass()->GetDisplayNameText());
			NoBrushEntry.DirectActions = FUIAction(FExecuteAction::CreateLambda([]() {}),
				FCanExecuteAction::CreateLambda([]() { return false; }));

			OutContextMenuEntryMap.FindOrAdd(BrushCategory).Entries.Add(NoBrushEntry);
		}
	}
}

void FLandscapeEditLayerContextMenuCustomization_Layer::FillUnassignedBrushMenu(FMenuBuilder& MenuBuilder, TArray<ALandscapeBlueprintBrushBase*> Brushes, int32 InLayerIndex)
{
	for (ALandscapeBlueprintBrushBase* Brush : Brushes)
	{
		FUIAction AddAction = FUIAction(FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Layer::AssignBrushToEditLayer, Brush, InLayerIndex));
		MenuBuilder.AddMenuEntry(FText::FromString(Brush->GetActorLabel()), FText(), FSlateIcon(), AddAction);
	}
}

void FLandscapeEditLayerContextMenuCustomization_Layer::AssignBrushToEditLayer(ALandscapeBlueprintBrushBase* Brush, const int32 InLayerIndex)
{
	const FScopedTransaction Transaction(LOCTEXT("LandscapeBrushAddToCurrentLayerTransaction", "Add brush to edit layer"));
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		if (ALandscape* Landscape = LandscapeEdMode->GetLandscape())
		{
			Landscape->AddBrushToLayer(InLayerIndex, Brush);
		}
	}
}

TSharedRef<IEditLayerCustomization> FLandscapeEditLayerContextMenuCustomization_Persistent::MakeInstance()
{
	return MakeShareable(new FLandscapeEditLayerContextMenuCustomization_Persistent);
}

void FLandscapeEditLayerContextMenuCustomization_Persistent::CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape && LandscapeEdMode && InEditLayer)
	{
		const int32 EditLayerIndex = Landscape->GetLayerIndex(InEditLayer->GetGuid());

		// Put Persistent edit layer actions (Clearing weightmap/heightmap) in a separate category
		const FName DataCategory = FName("Data");
		OutContextMenuEntryMap.FindOrAdd(DataCategory).SectionLabel = LOCTEXT("LandscapeEditLayerContextMenuData.Heading", "Data");

		if (LandscapeEdMode->DoesCurrentToolAffectEditLayers())
		{
			const ELandscapeToolTargetType CurrentTargetType = LandscapeEdMode->CurrentToolTarget.TargetType;
			// Use ELandscapeToolTargetType::Weightmap as default mode: used for Target Layers and ELandscapeToolTargetType::Invalid (Paint tool with no target layers)
			FText ClearEditLayerDataText = LOCTEXT("ClearAllTargetLayers", "Clear All Target Layers");

			if (CurrentTargetType == ELandscapeToolTargetType::Heightmap)
			{
				ClearEditLayerDataText = LOCTEXT("ClearSculptLayer", "Clear Sculpt Layer");
			}
			else if (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility)
			{
				ClearEditLayerDataText = LOCTEXT("ClearVisibilityLayer", "Clear Visibility Layer");
			}

			FMenuEntryParams ClearEditLayerEntry;
			ClearEditLayerEntry.LabelOverride = ClearEditLayerDataText;
			ClearEditLayerEntry.ToolTipOverride =  MakeAttributeSPLambda(this, [this, EditLayerIndex, CurrentTargetType] { FText Reason; CanClearEditLayerData(EditLayerIndex, CurrentTargetType, Reason); return Reason; });
			ClearEditLayerEntry.DirectActions = FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditLayerContextMenuCustomization_Persistent::ClearEditLayerData, EditLayerIndex, CurrentTargetType),
				FCanExecuteAction::CreateSPLambda(this, [this, EditLayerIndex, CurrentTargetType] { FText Reason; return CanClearEditLayerData(EditLayerIndex, CurrentTargetType, Reason); }));

			OutContextMenuEntryMap.FindOrAdd(DataCategory).Entries.Add(ClearEditLayerEntry);
		}
	}
}

void FLandscapeEditLayerContextMenuCustomization_Persistent::ClearEditLayerData(int32 InEditLayerIndex, ELandscapeToolTargetType InClearMode)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		ALandscape* Landscape = LandscapeEdMode->GetLandscape();
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InEditLayerIndex);
		if (Landscape != nullptr && EditLayer != nullptr)
		{
			const FText ClearType = UEnum::GetDisplayValueAsText(InClearMode);

			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, FText::Format(LOCTEXT("Landscape_ClearEditLayer_Dialog", "Edit layer {0}'s {1} data will be completely cleared.  Continue?"), FText::FromName(EditLayer->GetName()), ClearType));
			if (Result == EAppReturnType::Yes)
			{
				const FScopedTransaction Transaction(FText::Format(LOCTEXT("Landscape_Edit_Layers_Clear", "Clear Edit Layer {0} Data"), ClearType));
				Landscape->ClearEditLayer(InEditLayerIndex, /*InComponents = */ nullptr, UE::Landscape::GetLandscapeToolTargetTypeAsFlags(InClearMode));
				OnLayerSelectionChanged(InEditLayerIndex);

				if (InClearMode == ELandscapeToolTargetType::Weightmap || InClearMode == ELandscapeToolTargetType::Visibility)
				{
					LandscapeEdMode->RequestUpdateLayerUsageInformation();
				}
			}
		}
	}
}

bool FLandscapeEditLayerContextMenuCustomization_Persistent::CanClearEditLayerData(int32 InEditLayerIndex, ELandscapeToolTargetType InClearMode, FText& OutToolTip) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		switch (InClearMode)
		{
		case ELandscapeToolTargetType::Invalid:
			// In paint mode with no target layers 
			OutToolTip = LOCTEXT("EditLayerContextMenu.Clear_WeightmapNoneToolTip", "No target layer data to clear for this edit layer.");
			return false;
		case ELandscapeToolTargetType::Heightmap:
			OutToolTip = LOCTEXT("EditLayerContextMenu.Clear_HeightmapTooltip", "Clears all sculpting heightmap data for this edit layer.");
			break;
		case ELandscapeToolTargetType::Visibility:
			OutToolTip = LOCTEXT("EditLayerContextMenu.Clear_VisibilityLayerTooltip", "Removes all holes painted in visibility for this edit layer.");
			break;
		case ELandscapeToolTargetType::Weightmap:
			OutToolTip = LOCTEXT("EditLayerContextMenu.Clear_AllTargetLayerTooltip", "Clears all target layer weightmap data for this edit layer.");
			break;
		default:
			check(false);
			return false;
		}

		// InClearMode should always set the tooltip when valid
		check(OutToolTip.ToString() != FString());
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InEditLayerIndex);
		return EditLayer && LandscapeEdMode->GetLandscape() && LandscapeEdMode->CanEditLayer(&OutToolTip, EditLayer);
	}
	return false;
}

TSharedRef<IEditLayerCustomization> FLandscapeEditLayerContextMenuCustomization_Splines::MakeInstance()
{
	return MakeShareable(new FLandscapeEditLayerContextMenuCustomization_Splines);
}

void FLandscapeEditLayerContextMenuCustomization_Splines::CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && InEditLayer)
	{
		const FName SplinesCategory = FName("Splines");
		OutContextMenuEntryMap.FindOrAdd(SplinesCategory).SectionLabel = LOCTEXT("LandscapeEditLayerContextMenuSplines.Heading", "Splines");

		FMenuEntryParams ToggleSplineEntry;
		ToggleSplineEntry.LabelOverride = LOCTEXT("LandscapeEditorToggleSplinesTool", "Toggle Splines Tool"),
		ToggleSplineEntry.ToolTipOverride = LOCTEXT("LandscapeEditorToggleSplinesToolTip", "Toggles between Landscape Splines and the last Landscape Edit mode");
		ToggleSplineEntry.InputBindingOverride = LOCTEXT("LandscapeEditorToggleSplinesToolTipInputBinding", "Dbl-Click");
		ToggleSplineEntry.DirectActions = FUIAction(FExecuteAction::CreateLambda([LandscapeEdMode, InEditLayer]()
		{
			if (LandscapeEdMode && InEditLayer)
			{
				LandscapeEdMode->ToggleSplinesTool(InEditLayer);
			}
		}));

		OutContextMenuEntryMap.FindOrAdd(SplinesCategory).Entries.Add(ToggleSplineEntry);
	}
}

#undef LOCTEXT_NAMESPACE