// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "LandscapeEditLayerCustomization.h"

class FMenuBuilder;
class ALandscapeBlueprintBrushBase; 
class ULandscapeEditLayerBase;
class FEdModeLandscape;
class FReply;
struct FSlateBrush;
struct FMenuEntryParams;
enum class ELandscapeToolTargetType : uint8;

/** Edit layer common editor functionality */
class FLandscapeEditLayerCustomizationCommon
{
protected:
	static FEdModeLandscape* GetEditorMode();

	bool CanDeleteLayer(int32 InLayerIndex, FText& OutReason) const;
	void DeleteLayer(int32 InLayerIndex);

	bool CanCollapseLayer(int32 InLayerIndex, FText& OutReason) const;
	void CollapseLayer(int32 InLayerIndex);

	bool CanCollapseAllLayers(FText& OutReason) const;
	void CollapseAllEditLayers();

	bool CanToggleVisibility(int32 InLayerIndex, FText& OutReason) const;
	FReply OnToggleVisibility(int32 InLayerIndex);
	const FSlateBrush* GetVisibilityBrushForLayer(int32 InLayerIndex) const;

	void OnLayerSelectionChanged(int32 LayerIndex);
	void ShowOnlySelectedLayer(int32 InLayerIndex);
	void ShowAllLayers();

	TOptional<float> GetLayerAlpha(int32 InLayerIndex) const;
	float GetLayerAlphaMinValue() const;
	bool CanSetLayerAlpha(int32 InLayerIndex, FText& OutReason) const;
	void SetLayerAlpha(float InAlpha, int32 InLayerIndex, bool bCommit, int32 InSliderIndex);
};

/* Edit layer customizations for ULandscapeEditLayerBase */
class FLandscapeEditLayerContextMenuCustomization_Base : public IEditLayerCustomization, public FLandscapeEditLayerCustomizationCommon
{
public:
	static TSharedRef<IEditLayerCustomization> MakeInstance();

	/** IEditLayerCustomization interface */
	virtual void CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) override;
};

/* Edit layer customizations for ULandscapeEditLayer */
class FLandscapeEditLayerContextMenuCustomization_Layer : public IEditLayerCustomization, public FLandscapeEditLayerCustomizationCommon
{
public:
	static TSharedRef<IEditLayerCustomization> MakeInstance();

	/** IEditLayerCustomization interface */
	virtual void CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) override;

private:
	// Add Existing BP Brush (Brushes with no landscape actor assigned)
	void FillUnassignedBrushMenu(FMenuBuilder& MenuBuilder, TArray<ALandscapeBlueprintBrushBase*> Brushes, int32 InLayerIndex);
	void AssignBrushToEditLayer(ALandscapeBlueprintBrushBase* Brush, int32 InLayerIndex);
};

/* Edit layer customizations for ULandscapeEditLayerPersistent */
class FLandscapeEditLayerContextMenuCustomization_Persistent : public IEditLayerCustomization, public FLandscapeEditLayerCustomizationCommon
{
public:
	static TSharedRef<IEditLayerCustomization> MakeInstance();

	/** IEditLayerCustomization interface */
	virtual void CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) override;

private:
	// Weightmap/Heightmap related functions
	void ClearEditLayerData(int32 InEditLayerIndex, ELandscapeToolTargetType InClearMode);
	bool CanClearEditLayerData(int32 InEditLayerIndex, ELandscapeToolTargetType InClearMode, FText& OutToolTip) const;
};

/* Edit layer customizations for ULandscapeEditLayerSplines */
class FLandscapeEditLayerContextMenuCustomization_Splines : public IEditLayerCustomization, public FLandscapeEditLayerCustomizationCommon
{
public:
	static TSharedRef<IEditLayerCustomization> MakeInstance();

	/** IEditLayerCustomization interface */
	virtual void CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) override;
};