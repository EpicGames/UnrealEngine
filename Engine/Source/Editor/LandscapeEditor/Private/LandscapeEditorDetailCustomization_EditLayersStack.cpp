// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_EditLayersStack.h"
#include "IDetailChildrenBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Brushes/SlateColorBrush.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorClassUtils.h"
#include "LandscapeEditorDetailCustomization_LayersBrushStack.h" // FLandscapeBrushDragDropOp
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEditorUtils.h"
#include "Landscape.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Styling/SlateIconFinder.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Kismet2/SClassPickerDialog.h"

#include "SLandscapeEditor.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "ObjectTools.h"
#include "ScopedTransaction.h"
#include "DesktopPlatformModule.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "LandscapeRender.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "LandscapeEdit.h"
#include "IDetailGroup.h"
#include "Widgets/SBoxPanel.h"
#include "LandscapeEditorDetailCustomization_TargetLayers.h"
#include "Widgets/Input/SEditableText.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "LandscapeEditorCommands.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.Layers"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_EditLayersStack::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_EditLayersStack);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_EditLayersStack::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& LayerCategory = DetailBuilder.EditCategory("Edit Layers");

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode 
		&& LandscapeEdMode->GetLandscape() 
		&& (LandscapeEdMode->CurrentToolMode != nullptr)
		&& (FName(LandscapeEdMode->CurrentTool->GetToolName()) != TEXT("Mask")))
	{
		LayerCategory.AddCustomBuilder(MakeShareable(new FLandscapeEditorCustomNodeBuilder_Layers()));

		LayerCategory.AddCustomRow(FText())
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([]() { return ShouldShowLayersErrorMessageTip() ? EVisibility::Visible : EVisibility::Collapsed; })))
			[
				SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.Font(DetailBuilder.GetDetailFontBold())
				.BackgroundColor(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateLambda([]() { return FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"); })))
				.Text(TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&FLandscapeEditorDetailCustomization_EditLayersStack::GetLayersErrorMessageText)))
				.AutoWrapText(true)
			];
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FLandscapeEditorDetailCustomization_EditLayersStack::ShouldShowLayersErrorMessageTip()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->DoesCurrentToolAffectEditLayers())
	{
		return !LandscapeEdMode->CanEditLayer();
	}
	return false;
}

FText FLandscapeEditorDetailCustomization_EditLayersStack::GetLayersErrorMessageText()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	FText Reason;
	if (LandscapeEdMode && !LandscapeEdMode->CanEditLayer(&Reason))
	{
		return Reason;
	}
	return FText::GetEmpty();
}

//////////////////////////////////////////////////////////////////////////

FEdModeLandscape* FLandscapeEditorCustomNodeBuilder_Layers::GetEditorMode()
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

FLandscapeEditorCustomNodeBuilder_Layers::FLandscapeEditorCustomNodeBuilder_Layers()
	: CurrentSlider(INDEX_NONE)
{
}

FLandscapeEditorCustomNodeBuilder_Layers::~FLandscapeEditorCustomNodeBuilder_Layers()
{
}

void FLandscapeEditorCustomNodeBuilder_Layers::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
}

void FLandscapeEditorCustomNodeBuilder_Layers::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	NodeRow.NameWidget
		[
			SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("LayersLabel", "Layers"))
		];


	TSharedRef<SWidget> AddButton = PropertyCustomizationHelpers::MakeAddButton(
		FSimpleDelegate::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::CreateLayer),
		MakeAttributeSPLambda(this, [this] { FText Reason; CanCreateLayer(Reason); return Reason; }),
		MakeAttributeSPLambda(this, [this] { FText Reason; return CanCreateLayer(Reason); }));

	NodeRow.ValueWidget
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1) // Fill the entire width if possible
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.MinDesiredWidth(125.0f)
					.Text(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetNumLayersText)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(4.0f, 1.0f, 0.0f, 1.0f)
				[
					AddButton
				]
		];
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorCustomNodeBuilder_Layers::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		TSharedPtr<SDragAndDropVerticalBox> LayerList = SNew(SDragAndDropVerticalBox)
			.OnCanAcceptDropAdvanced(this, &FLandscapeEditorCustomNodeBuilder_Layers::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &FLandscapeEditorCustomNodeBuilder_Layers::HandleAcceptDrop)
			.OnDragDetected(this, &FLandscapeEditorCustomNodeBuilder_Layers::HandleDragDetected);

		LayerList->SetDropIndicator_Above(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Above"));
		LayerList->SetDropIndicator_Below(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Below"));

		ChildrenBuilder.AddCustomRow(FText::FromString(FString(TEXT("Edit Layers"))))
			.Visibility(EVisibility::Visible)
			[
				LayerList.ToSharedRef()
			];

		InlineTextBlocks.Empty();
		const int32 NumLayers = LandscapeEdMode->GetLayerCount();
		InlineTextBlocks.AddDefaulted(NumLayers);
		// Slots are displayed in the opposite order of LandscapeEditLayers
		for (int32 i = NumLayers - 1; i >= 0 ; --i)
		{
			TSharedPtr<SWidget> GeneratedRowWidget = GenerateRow(i);

			if (GeneratedRowWidget.IsValid())
			{
				LayerList->AddSlot()
					.AutoHeight()
					[
						GeneratedRowWidget.ToSharedRef()
					];
			}
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_Layers::GenerateRow(int32 InLayerIndex)
{
	TSharedRef<SWidget> DeleteButton = PropertyCustomizationHelpers::MakeDeleteButton(
		FSimpleDelegate::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::DeleteLayer, InLayerIndex),
		MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanDeleteLayer(InLayerIndex, Reason); return Reason; }),
		MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; return CanDeleteLayer(InLayerIndex, Reason); }));
	
	TSharedRef<SWidget> InspectObjectButton = PropertyCustomizationHelpers::MakeCustomButton(
		FAppStyle::GetBrush(TEXT("LandscapeEditor.InspectedObjects.ShowDetails")),
		FSimpleDelegate::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnSetInspectedDetailsToEditLayer, InLayerIndex),
		LOCTEXT("LandscapeEditLayerInspect", "Inspect the edit layer in the Landscape Details panel"));

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	TSharedPtr<SWidget> RowWidget = SNew(SLandscapeEditorSelectableBorder)
		.Padding(0.f)
		.VAlign(VAlign_Center)
		.OnContextMenuOpening(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnLayerContextMenuOpening, InLayerIndex)
		.OnSelected(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnLayerSelectionChanged, InLayerIndex)
		.IsSelected(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::IsLayerSelected, InLayerIndex)))
		.OnDoubleClick(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnLayerDoubleClick, InLayerIndex)
		.Visibility(EVisibility::Visible)
		[
			SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
						[
							SNew(SImage)
								.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
						.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
						[
							SNew(SImage)
								.Image(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetEditLayerIconBrush, InLayerIndex)
								.ToolTipText(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetEditLayerShortTooltipText, InLayerIndex)
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.OnClicked(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnToggleLock, InLayerIndex)
						.ToolTipText(LOCTEXT("LandscapeLayerLock", "Locks the current layer"))
						[
							SNew(SImage)
								.Image(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLockBrushForLayer, InLayerIndex)
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
						.ContentPadding(0.0f)
						.ButtonStyle(FAppStyle::Get(), "NoBorder")
						.IsEnabled(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; return CanToggleVisibility(InLayerIndex, Reason); }))
						.OnClicked(this, &FLandscapeEditorCustomNodeBuilder_Layers::OnToggleVisibility, InLayerIndex)
						.ToolTipText(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanToggleVisibility(InLayerIndex, Reason); return Reason; }))
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Content()
						[
							SNew(SImage)
								.Image(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetVisibilityBrushForLayer, InLayerIndex)
						]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					SNew(SHorizontalBox)
						.Clipping(EWidgetClipping::OnDemand)

						+ SHorizontalBox::Slot()
						.Padding(0)
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SAssignNew(InlineTextBlocks[InLayerIndex], SInlineEditableTextBlock)
								.IsEnabled(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; return CanRenameLayer(InLayerIndex, Reason); }))
								.Text(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerDisplayName, InLayerIndex)
								.ColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerTextColor, InLayerIndex)))
								.ToolTipText(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanRenameLayer(InLayerIndex, Reason); return Reason; }))
								.OnVerifyTextChanged(FOnVerifyTextChanged::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::CanRenameLayerTo, InLayerIndex))
								.OnTextCommitted(FOnTextCommitted::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::SetLayerName, InLayerIndex))
						]
				]

				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(0, 2)
				.HAlign(HAlign_Right)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.Padding(0)
						.FillWidth(1.0f)
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
								.IsEnabled(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; return CanSetLayerAlpha(InLayerIndex, Reason); }))
								.Visibility(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerAlphaVisibility, InLayerIndex)
								.Text(LOCTEXT("LandscapeLayerAlpha", "Alpha"))
								.ToolTipText(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanSetLayerAlpha(InLayerIndex, Reason); return Reason; }))
								.ColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerTextColor, InLayerIndex)))
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(0, 2)
						.HAlign(HAlign_Left)
						.FillWidth(1.0f)
						[
							SNew(SNumericEntryBox<float>)
								.AllowSpin(true)
								.MinValue_Lambda([this]
								{
									return GetLayerAlphaMinValue();
								})
								.MaxValue(1.0f)
								.MinSliderValue_Lambda([this]
								{
									return GetLayerAlphaMinValue();
								})
								.MaxSliderValue(1.0f)
								.Delta(0.01f)
								.MinDesiredValueWidth(60.0f)
								.IsEnabled(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; return CanSetLayerAlpha(InLayerIndex, Reason); }))
								.Visibility(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerAlphaVisibility, InLayerIndex)
								.ToolTipText(MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanSetLayerAlpha(InLayerIndex, Reason); return Reason; }))
								.Value(this, &FLandscapeEditorCustomNodeBuilder_Layers::GetLayerAlpha, InLayerIndex)
								.OnValueChanged_Lambda([this, InLayerIndex](float InValue) { SetLayerAlpha(InValue, InLayerIndex, false, CurrentSlider); })
								.OnValueCommitted_Lambda([this, InLayerIndex](float InValue, ETextCommit::Type InCommitType) { SetLayerAlpha(InValue, InLayerIndex, true, CurrentSlider); })
								.OnBeginSliderMovement_Lambda([this, InLayerIndex]()
								{
									CurrentSlider = InLayerIndex;
									GEditor->BeginTransaction(LOCTEXT("Landscape_Layers_SetAlpha", "Set Layer Alpha"));
								})
								.OnEndSliderMovement_Lambda([this](double)
								{
									GEditor->EndTransaction();
									CurrentSlider = INDEX_NONE;
								})
						]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					InspectObjectButton
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 0.0f)
				[
					DeleteButton
				]
		];

	return RowWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText FLandscapeEditorCustomNodeBuilder_Layers::GetLayerDisplayName(int32 InLayerIndex) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (LandscapeEdMode && Landscape)
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		check(EditLayer != nullptr);

		return FText::FromName(EditLayer->GetName());
	}

	return FText::FromString(TEXT("None"));
}

bool FLandscapeEditorCustomNodeBuilder_Layers::IsLayerSelected(int32 InLayerIndex) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		return LandscapeEdMode->GetSelectedEditLayerIndex() == InLayerIndex;
	}

	return false;
}

bool FLandscapeEditorCustomNodeBuilder_Layers::CanRenameLayerTo(const FText& InNewText, FText& OutErrorMessage, int32 InLayerIndex)
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		if (!LandscapeEdMode->CanRenameLayerTo(InLayerIndex, *InNewText.ToString()))
		{
			OutErrorMessage = LOCTEXT("Landscape_Layers_RenameFailed_AlreadyExists", "This edit layer name already exists");
			return false;
		}
	}
	return true;
}

void FLandscapeEditorCustomNodeBuilder_Layers::SetLayerName(const FText& InText, ETextCommit::Type InCommitType, int32 InLayerIndex)
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Rename", "Rename Edit Layer"));

		ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(InLayerIndex);
		EditLayer->SetName(*InText.ToString(), /*bInModify =*/ true);

		OnLayerSelectionChanged(InLayerIndex);
	}
}

FSlateColor FLandscapeEditorCustomNodeBuilder_Layers::GetLayerTextColor(int32 InLayerIndex) const
{
	return IsLayerSelected(InLayerIndex) ? FStyleColors::ForegroundHover : FSlateColor::UseForeground();
}

TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_Layers::OnLayerContextMenuOpening(int32 InLayerIndex)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		if (ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(InLayerIndex))
		{
			// Instead of building the MenuBuilder directly, gather a map of entries so multiple customization classes 
			// can place entries within the same category
			FEditLayerCategoryToEntryMap ContextMenuEntries;

			// Some context menu actions require slate components generated by this edit layer stack
			{
				FName EditLayerCategory = FName("Edit Layers");

				// Rename Layer
				FMenuEntryParams RenameLayerEntry;
				RenameLayerEntry.DirectActions = FUIAction(
					FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_Layers::RenameLayer, InLayerIndex),
					FCanExecuteAction::CreateSPLambda(this, [this, InLayerIndex] { FText Reason; return CanRenameLayer(InLayerIndex, Reason); }));
				RenameLayerEntry.ToolTipOverride = MakeAttributeSPLambda(this, [this, InLayerIndex] { FText Reason; CanRenameLayer(InLayerIndex, Reason); return Reason; });
				RenameLayerEntry.LabelOverride = LOCTEXT("RenameLayer", "Rename");

				ContextMenuEntries.FindOrAdd(EditLayerCategory).Entries.Add(RenameLayerEntry);
			}

			// Context menu actions that do not require slate state are defined in edit layer customization classes
			ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
			EditLayerCustomizationClassInstances = LandscapeEditorModule.QueryCustomEditLayerLayoutRecursive(EditLayer->GetClass());
			LandscapeEditorModule.ApplyEditLayerContextMenuCustomizations(EditLayer, EditLayerCustomizationClassInstances, ContextMenuEntries);
			LandscapeEditorUtils::BuildContextMenuFromCategoryEntryMap(ContextMenuEntries, MenuBuilder);
		}
	}
	return MenuBuilder.MakeWidget();
}

bool FLandscapeEditorCustomNodeBuilder_Layers::CanRenameLayer(int32 InLayerIndex, FText& OutReason) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);

	if (EditLayer == nullptr)
	{
		return false;
	}

	if (EditLayer->IsLocked())
	{
		OutReason = LOCTEXT("RenameLayer_CantRenameLocked", "Cannot rename a locked edit layer");
		return false;
	}

	OutReason = LOCTEXT("RenameLayer_CanRename", "Rename the edit layer");
	return true;
}

void FLandscapeEditorCustomNodeBuilder_Layers::RenameLayer(int32 InLayerIndex)
{
	if (InlineTextBlocks.IsValidIndex(InLayerIndex) && InlineTextBlocks[InLayerIndex].IsValid())
	{
		InlineTextBlocks[InLayerIndex]->EnterEditingMode();
	}
}

TSubclassOf<ULandscapeEditLayerBase> FLandscapeEditorCustomNodeBuilder_Layers::PickEditLayerClass() const
{
	class FLandscapeEditLayerClassFilter : public IClassViewerFilter
	{
	public:
		FLandscapeEditLayerClassFilter()
		{
			AllowedChildrenOfClasses.Add(ULandscapeEditLayerBase::StaticClass());
			DisallowedClassFlags = CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists;
		};

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
		{
			bool bIsCorrectClass = InFilterFuncs->IfInChildOfClassesSet(AllowedChildrenOfClasses, InClass) != EFilterReturn::Failed;
			bool bValidClassFlags = !InClass->HasAnyClassFlags(DisallowedClassFlags);

			return (bIsCorrectClass && bValidClassFlags);
		};

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef< const class IUnloadedBlueprintData > InUnloadedClassData, TSharedRef< class FClassViewerFilterFuncs > InFilterFuncs) override
		{
			return false;
		};

	private:
		/** All children of these classes will be included unless filtered out by another setting. */
		TSet<const UClass*> AllowedChildrenOfClasses;

		/** Disallowed class flags. */
		EClassFlags DisallowedClassFlags;
	};

	// Load the classviewer module to display a class picker
	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");

	// Fill in options
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.NameTypeToDisplay = EClassViewerNameTypeToDisplay::DisplayName;

	TSharedPtr<FLandscapeEditLayerClassFilter> Filter = MakeShareable(new FLandscapeEditLayerClassFilter());
	Options.ClassFilters.Add(Filter.ToSharedRef());

	const FText TitleText = LOCTEXT("PickEditLayerClass", "Pick Landscape Edit Layer Class");
	UClass* ChosenClass = nullptr;
	SClassPickerDialog::PickClass(TitleText, Options, ChosenClass, ULandscapeEditLayerBase::StaticClass());
	return TSubclassOf<ULandscapeEditLayerBase>(ChosenClass);
}

void FLandscapeEditorCustomNodeBuilder_Layers::CreateLayer()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		if (TSubclassOf<ULandscapeEditLayerBase> EditLayerClass = PickEditLayerClass())
		{
			// Disallow multiple layers of certain types : 
			if (!EditLayerClass.GetDefaultObject()->SupportsMultiple())
			{
				if (int32 NumLayersOfThisType = Landscape->GetLayersOfTypeConst(EditLayerClass).Num(); NumLayersOfThisType > 0)
				{
					FMessageDialog::Open(EAppMsgType::Ok, FText::Format(LOCTEXT("Landscape_CreateLayer_CannotCreateMultiple", "Cannot create layer of type {0} : {1} {1}|plural(one=layer, other=layers) of this type already {1}|plural(one=exists, other=exist) and only 1 is allowed"), 
						EditLayerClass->GetDisplayNameText(), NumLayersOfThisType));
					return;
				}
			}

			const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Create", "Create Layer"));
			Landscape->CreateLayer(FName(EditLayerClass.GetDefaultObject()->GetDefaultName()), EditLayerClass);
			OnLayerSelectionChanged(Landscape->GetEditLayersConst().Num() - 1);

			LandscapeEdMode->RefreshDetailPanel();
		}
	}
}

FText FLandscapeEditorCustomNodeBuilder_Layers::GetNumLayersText() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		return FText::Format(LOCTEXT("NumEditLayersText", "{0} Edit {0}|plural(one=Layer, other=Layers)"), Landscape->GetEditLayersConst().Num());
	}

	return FText();
}

bool FLandscapeEditorCustomNodeBuilder_Layers::CanCreateLayer(FText& OutReason) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		if (Landscape->IsMaxLayersReached())
		{
			OutReason = LOCTEXT("CreateLayerTooltip_MaxLayersReached", "Creates a new edit layer.\nCurrently disabled as the max number of layers has been reached. This can be adjusted in the landscape project settings : MaxNumberOfLayers)");
			return false;
		}
	}

	OutReason = LOCTEXT("CreateLayerTooltip", "Creates a new edit layer");
	return true;
}

void FLandscapeEditorCustomNodeBuilder_Layers::OnLayerDoubleClick(int32 InLayerIndex) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->CurrentToolMode)
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		LandscapeEdMode->ToggleSplinesTool(EditLayer);
	}
}

void FLandscapeEditorCustomNodeBuilder_Layers::OnSetInspectedDetailsToEditLayer(int32 InLayerIndex) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);

		// Clear out all previously selected objects, this may change in the future
		TArray<TWeakObjectPtr<UObject>> InspectedObjects;
		InspectedObjects.Add(const_cast<ULandscapeEditLayerBase*>(EditLayer));
		LandscapeEdMode->SetInspectedObjects(InspectedObjects);
	}
}

FReply FLandscapeEditorCustomNodeBuilder_Layers::OnToggleLock(int32 InLayerIndex)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		ULandscapeEditLayerBase* EditLayer =  LandscapeEdMode->GetEditLayer(InLayerIndex);
		check(EditLayer != nullptr);
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Locked", "Set Layer Locked"));
		EditLayer->SetLocked(!EditLayer->IsLocked(), /*bInModify = */true);
	}
	return FReply::Handled();
}

EVisibility FLandscapeEditorCustomNodeBuilder_Layers::GetLayerAlphaVisibility(int32 InLayerIndex) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	bool bIsVisible = LandscapeEdMode && LandscapeEdMode->IsLayerAlphaVisible(InLayerIndex);
	return bIsVisible ? EVisibility::Visible : EVisibility::Hidden;
}

FText FLandscapeEditorCustomNodeBuilder_Layers::GetEditLayerShortTooltipText(int32 InEditLayerIndex) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InEditLayerIndex);

		// Edit layer tooltips defined in ShortToolTip field
		return EditLayer ? EditLayer->GetClass()->GetToolTipText(/*bShortToolTip =*/ true) : FText();
	}

	return FText();
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_Layers::GetLockBrushForLayer(int32 InLayerIndex) const
{
	bool bIsLocked = false;
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(InLayerIndex);
		check(EditLayer != nullptr);
		bIsLocked = EditLayer->IsLocked();
	}
	return bIsLocked ? FAppStyle::GetBrush(TEXT("PropertyWindow.Locked")) : FAppStyle::GetBrush(TEXT("PropertyWindow.Unlocked"));
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_Layers::GetEditLayerIconBrush(int32 InLayerIndex) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	const ULandscapeEditLayerBase* EditLayer = (LandscapeEdMode != nullptr) ? LandscapeEdMode->GetEditLayerConst(InLayerIndex) : nullptr;
	return (EditLayer != nullptr) ? FSlateIconFinder::FindIconBrushForClass(EditLayer->GetClass()) : nullptr;
}

int32 FLandscapeEditorCustomNodeBuilder_Layers::SlotIndexToLayerIndex(int32 SlotIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (!Landscape)
	{
		return INDEX_NONE;
	}
	
	check(Landscape->GetEditLayersConst().IsValidIndex(SlotIndex));
	return Landscape->GetEditLayersConst().Num() - SlotIndex - 1;
}

FReply FLandscapeEditorCustomNodeBuilder_Layers::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->DoesCurrentToolAffectEditLayers())
	{
		int32 LayerIndex = SlotIndexToLayerIndex(SlotIndex);
		if (const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(LayerIndex))
		{
			if (!EditLayer->IsLocked())
			{
				TSharedPtr<SWidget> Row = GenerateRow(LayerIndex);
				if (Row.IsValid())
				{
					return FReply::Handled().BeginDragDrop(FLandscapeListElementDragDropOp::New(SlotIndex, Slot, Row));
				}
			}
		}
	}
	return FReply::Unhandled();
}

TOptional<SDragAndDropVerticalBox::EItemDropZone> FLandscapeEditorCustomNodeBuilder_Layers::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FLandscapeListElementDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FLandscapeListElementDragDropOp>();
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode && DragDropOperation.IsValid() && DragDropOperation->IsOfType<FLandscapeBrushDragDropOp>())
	{
		const int32 DestinationLayerIndex = SlotIndexToLayerIndex(SlotIndex);
		
		if (const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayerConst(DestinationLayerIndex))
		{
			if (EditLayer->SupportsBlueprintBrushes())
			{
				return DropZone;
			}
			else 
			{
				return TOptional<SDragAndDropVerticalBox::EItemDropZone>();
			}
		}
	}

	return DropZone;
}

FReply FLandscapeEditorCustomNodeBuilder_Layers::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FLandscapeListElementDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FLandscapeListElementDragDropOp>();

	if (!DragDropOperation.IsValid())
	{
		return FReply::Unhandled();
	}

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (!Landscape)
	{
		return FReply::Unhandled();
	}

	// See if we're actually getting a drag from the blueprint brush list, rather than
	// from the edit layer list
	if (DragDropOperation->IsOfType<FLandscapeBrushDragDropOp>())
	{
		int32 StartingBrushIndex = DragDropOperation->SlotIndexBeingDragged;
		int32 StartingLayerIndex = LandscapeEdMode->GetSelectedEditLayerIndex();
		int32 DestinationLayerIndex = SlotIndexToLayerIndex(SlotIndex);

		if (StartingLayerIndex == DestinationLayerIndex)
		{
			// See comment further below about not returning Handled()
			return FReply::Unhandled();
		}

		ALandscapeBlueprintBrushBase* Brush = Landscape->GetBrushForLayer(StartingLayerIndex, StartingBrushIndex);
		if (!ensure(Brush))
		{
			return FReply::Unhandled();
		}

		const FScopedTransaction Transaction(LOCTEXT("Landscape_LayerBrushes_MoveLayers", "Move Brush to Layer"));
		Landscape->RemoveBrushFromLayer(StartingLayerIndex, StartingBrushIndex);
		Landscape->AddBrushToLayer(DestinationLayerIndex, Brush);

		LandscapeEdMode->SetSelectedEditLayer(DestinationLayerIndex);

		// HACK: We don't return FReply::Handled() here because otherwise, SDragAndDropVerticalBox::OnDrop
		// will apply UI slot reordering after we return. Properly speaking, we should have a way to signal 
		// that the operation was handled yet that it is not one that SDragAndDropVerticalBox should deal with.
		// For now, however, just make sure to return Unhandled.
		return FReply::Unhandled();
	}

	// This must be a drag from our own list.
	int32 StartingLayerIndex = SlotIndexToLayerIndex(DragDropOperation->SlotIndexBeingDragged);
	int32 DestinationLayerIndex = SlotIndexToLayerIndex(SlotIndex);
	const FScopedTransaction Transaction(LOCTEXT("Landscape_Layers_Reorder", "Reorder Layer"));
	if (Landscape->ReorderLayer(StartingLayerIndex, DestinationLayerIndex))
	{
		LandscapeEdMode->SetSelectedEditLayer(DestinationLayerIndex);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<FLandscapeListElementDragDropOp> FLandscapeListElementDragDropOp::New(int32 InSlotIndexBeingDragged, SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> WidgetToShow)
{
	TSharedRef<FLandscapeListElementDragDropOp> Operation = MakeShareable(new FLandscapeListElementDragDropOp);

	Operation->MouseCursor = EMouseCursor::GrabHandClosed;
	Operation->SlotIndexBeingDragged = InSlotIndexBeingDragged;
	Operation->SlotBeingDragged = InSlotBeingDragged;
	Operation->WidgetToShow = WidgetToShow;

	Operation->Construct();

	return Operation;
}

TSharedPtr<SWidget> FLandscapeListElementDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetDragDropTooltipBackground"))
		.Content()
		[
			WidgetToShow.ToSharedRef()
		];
}

#undef LOCTEXT_NAMESPACE