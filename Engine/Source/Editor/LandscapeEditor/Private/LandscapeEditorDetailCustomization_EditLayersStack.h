// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "LandscapeEditorDetailCustomization_Base.h"
#include "LandscapeEditorDetailCustomization_EditLayers.h" // FLandscapeEditLayerCustomizationCommon
#include "LandscapeEdMode.h"
#include "LandscapeEditLayer.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateBrush.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "AssetThumbnail.h"
#include "Framework/SlateDelegates.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class SDragAndDropVerticalBox;
class SInlineEditableTextBlock;
class ALandscapeBlueprintBrushBase;
class ULandscapeLayerInfoObject;
class FMenuBuilder;

/**
 * Slate widgets customizer for the edit layer stack (Edit Layer ordered list) in the Landscape Editor
 */

class FLandscapeEditorDetailCustomization_EditLayersStack : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	static bool ShouldShowLayersErrorMessageTip();
	static FText GetLayersErrorMessageText();
};

class FLandscapeEditorCustomNodeBuilder_Layers : public IDetailCustomNodeBuilder, public TSharedFromThis<FLandscapeEditorCustomNodeBuilder_Layers>, public FLandscapeEditLayerCustomizationCommon
{
public:
	FLandscapeEditorCustomNodeBuilder_Layers();
	~FLandscapeEditorCustomNodeBuilder_Layers();

	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override { return "Layers"; }

protected:
	static class FEdModeLandscape* GetEditorMode();

	TSharedPtr<SWidget> GenerateRow(int32 InLayerIndex);

	// Drag/Drop handling
	int32 SlotIndexToLayerIndex(int32 SlotIndex);
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	TOptional<SDragAndDropVerticalBox::EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);

	bool IsLayerSelected(int32 LayerIndex) const;
	void OnLayerDoubleClick(int32 LayerIndex) const;
	TSharedPtr<SWidget> OnLayerContextMenuOpening(int32 InLayerIndex);

	FText GetNumLayersText() const;

	bool CanCreateLayer(FText& OutReason) const;
	void CreateLayer();

	const FSlateBrush* GetEditLayerIconBrush(int32 InLayerIndex) const;

	bool CanRenameLayerTo(const FText& NewText, FText& OutErrorMessage, int32 InLayerIndex);
	bool CanRenameLayer(int32 InLayerIndex, FText& OutReason) const;
	void RenameLayer(int32 InLayerIndex);

	void SetLayerName(const FText& InText, ETextCommit::Type InCommitType, int32 InLayerIndex);
	FText GetLayerText(int32 InLayerIndex) const;
	FSlateColor GetLayerTextColor(int32 InLayerIndex) const;
	FText GetLayerDisplayName(int32 InLayerIndex) const;
	EVisibility GetLayerAlphaVisibility(int32 InLayerIndex) const;
	FText GetEditLayerShortTooltipText(int32 InEditLayerIndex) const;
	FReply OnEditLayerIconClicked(int32 InEditLayerIndex) const;

	TSubclassOf<ULandscapeEditLayerBase> PickEditLayerClass() const;
	
	void OnSetInspectedDetailsToEditLayer(int32 InLayerIndex) const;

	FReply OnToggleLock(int32 InLayerIndex);
	const FSlateBrush* GetLockBrushForLayer(int32 InLayerIndex) const;

private:

	/** Widgets for displaying and editing the layer name */
	TArray< TSharedPtr< SInlineEditableTextBlock > > InlineTextBlocks;

	/** The edit layer customization class instances active in this view */
	TArray<TSharedRef<IEditLayerCustomization>> EditLayerCustomizationClassInstances;

	int32 CurrentSlider;
};

class FLandscapeListElementDragDropOp : public FDragAndDropVerticalBoxOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLandscapeListElementDragDropOp, FDragAndDropVerticalBoxOp)

	TSharedPtr<SWidget> WidgetToShow;

	static TSharedRef<FLandscapeListElementDragDropOp> New(int32 InSlotIndexBeingDragged, SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> InWidgetToShow);

public:
	virtual ~FLandscapeListElementDragDropOp() = default;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
};
