// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_ImportExport.h"
#include "SlateOptMacros.h"
#include "IDetailCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "DetailWidgetRow.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SToolTip.h"

#include "Misc/ScopedSlowTask.h"
#include "Misc/MessageDialog.h"

#include "LocationVolume.h"
#include "SourceControlHelpers.h"
#include "UObject/SavePackage.h"
#include "WorldPartition/WorldPartition.h"
#include "Internationalization/Regex.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"

#include "ScopedTransaction.h"
#include "DesktopPlatformModule.h"

#include "LandscapeEditorObject.h"
#include "LandscapeEditorModule.h"
#include "LandscapeImportHelper.h"
#include "LandscapeSubsystem.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeImageFileCache.h"
#include "LandscapeRegionUtils.h"
#include "LandscapeEditorUtils.h"


#include "LandscapeEditorPrivate.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.ImportExport"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_ImportExport::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_ImportExport);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_ImportExport::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!IsToolActive("ImportExport"))
	{
		return;
	}

	DetailBuilder.HideCategory("New Landscape");
	IDetailCategoryBuilder& ImportExportCategory = DetailBuilder.EditCategory("Import / Export");

	ImportExportCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SUniformGridPanel)
		.SlotPadding(FMargin(10, 2))
		+ SUniformGridPanel::Slot(0, 0)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked_Static(&FLandscapeEditorDetailCustomization_ImportExport::ModeIsChecked, EImportExportMode::Import)
			.OnCheckStateChanged_Static(&FLandscapeEditorDetailCustomization_ImportExport::OnModeChanged, EImportExportMode::Import)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Import", "Import"))
			]
		]
		+ SUniformGridPanel::Slot(1, 0)
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "RadioButton")
			.IsChecked_Static(&FLandscapeEditorDetailCustomization_ImportExport::ModeIsChecked, EImportExportMode::Export)
			.OnCheckStateChanged_Static(&FLandscapeEditorDetailCustomization_ImportExport::OnModeChanged, EImportExportMode::Export)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Export", "Export"))
			]
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_ImportHeightmapFilename = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapFilename));
	TSharedRef<IPropertyHandle> PropertyHandle_ExportHeightmapFilename = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, HeightmapExportFilename));
	
	PropertyHandle_ImportHeightmapFilename->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle_ImportHeightmapFilename]()
	{
		FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_ImportHeightmapFilename, /*bForExport = */false);
		FLandscapeEditorDetailCustomization_ImportExport::OnImportHeightmapFilenameChanged();
	}));

	PropertyHandle_ExportHeightmapFilename->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle_ExportHeightmapFilename]()
	{
		FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_ExportHeightmapFilename, /*bForExport = */true);
	}));
	
	TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapImportResult));
	TSharedRef<IPropertyHandle> PropertyHandle_HeightmapErrorMessage = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapErrorMessage));
	DetailBuilder.HideProperty(PropertyHandle_HeightmapImportResult);
	DetailBuilder.HideProperty(PropertyHandle_HeightmapErrorMessage);
		
	TSharedRef<IPropertyHandle> PropertyHandle_ExportEditLayer = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bExportEditLayer));
	TSharedRef<IPropertyHandle> PropertyHandle_ExportSingleFile = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bExportSingleFile));
	DetailBuilder.HideProperty(PropertyHandle_ExportEditLayer);
	DetailBuilder.HideProperty(PropertyHandle_ExportSingleFile);

	auto AddHeightmapFileName = [&](TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename, TAttribute<EVisibility> PropertyVisibility)
	{
		return ImportExportCategory.AddProperty(PropertyHandle_HeightmapFilename)
			.Visibility(PropertyVisibility)
			.CustomWidget()
			.NameContent()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 2, 0)
				[
					SNew(SCheckBox)
					.IsChecked_Static(&FLandscapeEditorDetailCustomization_ImportExport::GetHeightmapSelectedCheckState)
					.OnCheckStateChanged_Static(&FLandscapeEditorDetailCustomization_ImportExport::OnHeightmapSelectedCheckStateChanged)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					PropertyHandle_HeightmapFilename->CreatePropertyNameWidget()
				]
			]
			.ValueContent()
			.MinDesiredWidth(250.0f)
			.MaxDesiredWidth(0)
			[
				SNew(SHorizontalBox)
				.IsEnabled_Static(&FLandscapeEditorDetailCustomization_ImportExport::IsHeightmapEnabled)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 2, 0)
				[
					SNew(SErrorText)
					.Visibility_Static(&GetImportResultErrorVisibility, PropertyHandle_HeightmapImportResult)
					.BackgroundColor_Static(&GetImportResultErrorColor, PropertyHandle_HeightmapImportResult)
					.ErrorText(NSLOCTEXT("UnrealEd", "Error", "!"))
					.ToolTip(
						SNew(SToolTip)
						.Text_Static(&GetPropertyValue<FText>, PropertyHandle_HeightmapErrorMessage)
					)
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				[
					SNew(SBox)
					// This path is edited via a SEditableTextBox (rather than using CreatePropertyValueWidget) so we have to specify a tooltip in order to display the full path if 
					//  the text box is to small to display the entire thing
					.ToolTipText_Static(&GetPropertyValueText, PropertyHandle_HeightmapFilename)
					[
					    SNew(SEditableTextBox)
					    .Font(DetailBuilder.GetDetailFont())
					    .Text_Static(&GetPropertyValueText, PropertyHandle_HeightmapFilename)
					    .OnTextCommitted_Static(&SetFilename, PropertyHandle_HeightmapFilename)
						.HintText(LOCTEXT("TargetLayerAssetPath_Hint", "(Please specify a heightmap)"))
					]
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(1, 0, 0, 0)
				[
					SNew(SButton)
					.ContentPadding(FMargin(4, 0))
					.Text(NSLOCTEXT("UnrealEd", "GenericOpenDialog", "..."))
					.OnClicked_Static(&OnBrowseFilenameButtonClicked, PropertyHandle_HeightmapFilename)
				]
			]
			// Override the ResetToDefault so we can clear the file path and the check box on reset
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateLambda([this, PropertyHandle_HeightmapFilename](TSharedPtr<IPropertyHandle> InProperty)
				{
					FString FilePathValue;
					PropertyHandle_HeightmapFilename->GetValue(FilePathValue);

					FEdModeLandscape* LandscapeEdMode = GetEditorMode();
					if (LandscapeEdMode && LandscapeEdMode->UISettings)
					{
						// If the checkbox is selected or the file path is set show the reset button
						return !FilePathValue.IsEmpty() || LandscapeEdMode->UISettings->bHeightmapSelected;
					}

					return false;
				}),
				FResetToDefaultHandler::CreateLambda([this, PropertyHandle_HeightmapFilename](TSharedPtr<IPropertyHandle> InProperty)
				{
					// Reset the file path 
					PropertyHandle_HeightmapFilename->SetValue(FString());

					// Reset the selected check box
					FEdModeLandscape* LandscapeEdMode = GetEditorMode();
					if (LandscapeEdMode && LandscapeEdMode->UISettings)
					{
						LandscapeEdMode->UISettings->bHeightmapSelected = false;
					}
				})
			));
	};

	AddHeightmapFileName(PropertyHandle_ExportHeightmapFilename, MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(false); }));
	AddHeightmapFileName(PropertyHandle_ImportHeightmapFilename, MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(true); }));

	TSharedRef<IPropertyHandle> PropertyHandle_ImportType = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportType));
	ImportExportCategory.AddProperty(PropertyHandle_ImportType).Visibility(MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(true); }));

	TSharedRef<IPropertyHandle> PropertyHandle_FlipYAxis = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bFlipYAxis));
	PropertyHandle_FlipYAxis->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([]() { FLandscapeEditorDetailCustomization_ImportExport::OnImportHeightmapFilenameChanged(); }));
	ImportExportCategory.AddProperty(PropertyHandle_FlipYAxis).Visibility(MakeAttributeLambda([]() { return (!GetEditorMode()->UseSingleFileImport()) ? EVisibility::Visible : EVisibility::Collapsed; }));

	TSharedPtr<SToolTip> ExportSingleFileTooltip = SNew(SToolTip)
		.Text(PropertyHandle_ExportSingleFile->GetToolTipText());

	TSharedRef<IPropertyHandle> PropertyHandle_ImportExportMode = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportExportMode));
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	IDetailPropertyRow& ImportExportModeRow = ImportExportCategory.AddProperty(PropertyHandle_ImportExportMode);
	ImportExportModeRow.Visibility(MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(true); }));
	ImportExportModeRow.GetDefaultWidgets(NameWidget, ValueWidget);
	ImportExportModeRow.CustomWidget()
		.NameContent()
		[
			SNew(SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(0, 0, 2, 0))
				[
					SNew(STextBlock)
						.Text(MakeAttributeLambda([]() { return IsImporting() ? LOCTEXT("ImportMode", "Import Mode") : LOCTEXT("ExportMode", "Export Mode"); }))
						.Font(DetailBuilder.GetDetailFont())
						.ToolTipText(MakeAttributeLambda([]() { return IsImporting() 
							? LOCTEXT("ImportModeTooltip", "Specifies whether all or only loaded landscape regions should be imported") 
							: LOCTEXT("ExportModeToolTip", "Specifies whether all or only loaded landscape regions should be imported"); }))
				]
		]
		.ValueContent()
		[
			ValueWidget.ToSharedRef()
		];


	ImportExportCategory.AddProperty(PropertyHandle_ExportSingleFile)
		.Visibility(MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(false); }))
		.IsEnabled(MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetExportSingleFileIsEnabled(); }))
		.CustomWidget()
		.NameContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 2, 0))
			[
				SNew(STextBlock)
				.Text(PropertyHandle_ExportSingleFile->GetPropertyDisplayName())
				.Font(DetailBuilder.GetDetailFont())
				.ToolTip(ExportSingleFileTooltip)
			]
		]
		.ValueContent()
		[
			SNew(SCheckBox)
			.IsChecked_Static(&FLandscapeEditorDetailCustomization_ImportExport::GetExportSingleFileCheckState)
			.OnCheckStateChanged_Lambda([this, PropertyHandle_ExportHeightmapFilename](ECheckBoxState NewCheckState) 
				{ 
					FLandscapeEditorDetailCustomization_ImportExport::OnExportSingleFileCheckStateChanged(NewCheckState);
					FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_ExportHeightmapFilename, /*bForExport = */true);
				})
		];

	ImportExportCategory.AddProperty(PropertyHandle_ExportEditLayer).Visibility(MakeAttributeLambda([this]()
	{
		return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(false);
	}));

	ImportExportCategory.AddCustomRow(LOCTEXT("ImportResolution", "Import Resolution"))
		.Visibility(MakeAttributeLambda([]() { return FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(true); }))
		.NameContent()
		[
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 2, 0))
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("ImportResolution", "Import Resolution"))
				.ToolTipText(LOCTEXT("ImportResolutionTooltip", "Possible resolutions for the selected image"))
			]
		]
		.ValueContent()
		[
			SNew(SBox)
			.Padding(FMargin(0, 0, 12, 0)) // Line up with the other properties due to having no reset to default button
			[
				SNew(SComboButton)
				.OnGetMenuContent_Static(&FLandscapeEditorDetailCustomization_ImportExport::GetImportLandscapeResolutionMenu)
				.ContentPadding(2.0f)
				.ButtonContent()
				[
					SNew(STextBlock)
					.Font(DetailBuilder.GetDetailFont())
					.Text_Static(&FLandscapeEditorDetailCustomization_ImportExport::GetImportLandscapeResolution)
				]
			]
		];

	TSharedRef<IPropertyHandle> PropertyHandle_Layers = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_Layers));
	ImportExportCategory.AddProperty(PropertyHandle_Layers);

	ImportExportCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.Text_Static(&GetImportExportButtonText)
			.OnClicked_Static(&OnImportExportButtonClicked)
			.IsEnabled_Static(&GetImportExportButtonIsEnabled)
		]
	];

	ImportExportCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SWarningOrErrorBox)
		.Message(this, &FLandscapeEditorDetailCustomization_ImportExport::GetImportExportLandscapeErrorText)
	]
	.Visibility(TAttribute<EVisibility>(this, &FLandscapeEditorDetailCustomization_ImportExport::GetImportExportLandscapeErrorVisibility));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FLandscapeEditorDetailCustomization_ImportExport::GetExportSingleFileIsEnabled()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	// The single file option only makes sense for landscapes where multiple file import is an option : 
	return LandscapeEdMode && !LandscapeEdMode->UseSingleFileImport();
}

ECheckBoxState FLandscapeEditorDetailCustomization_ImportExport::GetExportSingleFileCheckState()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	return (LandscapeEdMode && (LandscapeEdMode->UseSingleFileImport() || (LandscapeEdMode->UISettings &&  LandscapeEdMode->UISettings->bExportSingleFile))) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLandscapeEditorDetailCustomization_ImportExport::OnExportSingleFileCheckStateChanged(ECheckBoxState NewCheckState)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		if (NewCheckState != ECheckBoxState::Undetermined)
		{
			LandscapeEdMode->UISettings->bExportSingleFile = (NewCheckState == ECheckBoxState::Checked);
		}
	}
}

EVisibility FLandscapeEditorDetailCustomization_ImportExport::GetImportExportVisibility(bool bImport)
{
	return IsImporting() == bImport ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FLandscapeEditorDetailCustomization_ImportExport::IsHeightmapEnabled()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		return LandscapeEdMode->UISettings->bHeightmapSelected;
	}

	return false;
}

ECheckBoxState FLandscapeEditorDetailCustomization_ImportExport::GetHeightmapSelectedCheckState()
{
	return IsHeightmapEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLandscapeEditorDetailCustomization_ImportExport::OnHeightmapSelectedCheckStateChanged(ECheckBoxState NewCheckedState)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		if (NewCheckedState != ECheckBoxState::Undetermined)
		{
			LandscapeEdMode->UISettings->bHeightmapSelected = (NewCheckedState == ECheckBoxState::Checked);
		}
	}
}

ECheckBoxState FLandscapeEditorDetailCustomization_ImportExport::ModeIsChecked(EImportExportMode Value)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	return (LandscapeEdMode && (LandscapeEdMode->ImportExportMode == Value)) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLandscapeEditorDetailCustomization_ImportExport::OnModeChanged(ECheckBoxState NewCheckedState, EImportExportMode Value)
{
	if (NewCheckedState == ECheckBoxState::Checked)
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (!LandscapeEdMode)
		{
			return;
		}

		LandscapeEdMode->ImportExportMode = Value;

		if (Value == EImportExportMode::Import)
		{
			if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
			{
				FVector LocalPosition(LandscapeEdMode->UISettings->ImportLandscape_GizmoLocalPosition.X, LandscapeEdMode->UISettings->ImportLandscape_GizmoLocalPosition.Y, 0.0f);
				FVector GizmoPosition = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().TransformPosition(LocalPosition);
				if (ALandscapeGizmoActiveActor* GizmoActor = LandscapeEdMode->UISettings->CurrentGizmoActor.Get())
				{
					GizmoActor->SetActorLocation(GizmoPosition);
				}
			}
		}
	}
}

EVisibility FLandscapeEditorDetailCustomization_ImportExport::GetImportResultErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_Result)
{
	// Not relevant when exporting : 
	if (!IsImporting())
	{
		return EVisibility::Collapsed;
	}

	ELandscapeImportResult ImportResult;
	FPropertyAccess::Result Result = PropertyHandle_Result->GetValue((uint8&)ImportResult);

	if (Result == FPropertyAccess::Fail)
	{
		return EVisibility::Collapsed;
	}

	if (Result == FPropertyAccess::MultipleValues)
	{
		return EVisibility::Visible;
	}

	if (ImportResult != ELandscapeImportResult::Success)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FSlateColor FLandscapeEditorDetailCustomization_ImportExport::GetImportResultErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_Result)
{
	ELandscapeImportResult ImportResult;
	FPropertyAccess::Result Result = PropertyHandle_Result->GetValue((uint8&)ImportResult);

	if (Result == FPropertyAccess::Fail ||
		Result == FPropertyAccess::MultipleValues)
	{
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	}

	switch (ImportResult)
	{
	case ELandscapeImportResult::Success:
		return FCoreStyle::Get().GetColor("InfoReporting.BackgroundColor");
	case ELandscapeImportResult::Warning:
		return FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor");
	case ELandscapeImportResult::Error:
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	default:
		check(0);
		return FSlateColor();
	}
}

void FLandscapeEditorDetailCustomization_ImportExport::SetFilename(const FText& NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle_Filename)
{
	FString Filename = NewValue.ToString();
	ensure(PropertyHandle_Filename->SetValue(Filename) == FPropertyAccess::Success);
}

FReply FLandscapeEditorDetailCustomization_ImportExport::OnBrowseFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_Filename)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const bool bIsImporting = IsImporting();
	const FString DialogType = bIsImporting ? LandscapeEditorModule.GetHeightmapImportDialogTypeString() : LandscapeEditorModule.GetHeightmapExportDialogTypeString();
	const FString DialogTitle = bIsImporting ? LOCTEXT("ImportHeightmap", "Import Heightmap").ToString() : LOCTEXT("ExportHeightmap", "Export Heightmap").ToString();

	TOptional<FString> OptionalFilename = LandscapeEditorUtils::GetImportExportFilename(DialogTitle, LandscapeEdMode->UISettings->LastImportPath, DialogType, bIsImporting);
	// If the user canceled the dialog (Filename is empty), don't clear the path
	if (OptionalFilename.IsSet() && !OptionalFilename->IsEmpty())
	{
		const FString& Filename = OptionalFilename.GetValue();

		ensure(PropertyHandle_Filename->SetValue(Filename) == FPropertyAccess::Success);
		LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(Filename);
	}

	return FReply::Handled();
}

void FLandscapeEditorDetailCustomization_ImportExport::OnImportHeightmapFilenameChanged()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		LandscapeEdMode->UISettings->OnImportHeightmapFilenameChanged();
	}
}

void FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(TSharedRef<IPropertyHandle> PropertyHandle_Filename, bool bForExport)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	// No need to format the file name if single file import/export is enabled
	if (LandscapeEdMode && !LandscapeEdMode->UseSingleFileImport() && (!bForExport || !LandscapeEdMode->UISettings->bExportSingleFile))
	{
		// If selected file as the _xN_yM suffix remove it
		FString	FilePath;
		if (PropertyHandle_Filename->GetValue(FilePath) == FPropertyAccess::Success)
		{
			FIntPoint OutCoord;
			FString OutBaseFilePattern;
			if (FLandscapeImportHelper::ExtractCoordinates(FPaths::GetBaseFilename(FilePath), OutCoord, OutBaseFilePattern))
			{
				PropertyHandle_Filename->SetValue(FString::Format(TEXT("{0}/{1}{2}"), { FPaths::GetPath(FilePath), OutBaseFilePattern, FPaths::GetExtension(FilePath, true) }));
			}
		}
	}
}

bool FLandscapeEditorDetailCustomization_ImportExport::IsImporting()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	return LandscapeEdMode && (LandscapeEdMode->ImportExportMode == EImportExportMode::Import);
}

FText FLandscapeEditorDetailCustomization_ImportExport::GetImportExportButtonText()
{
	if (IsImporting())
	{
		return LOCTEXT("Import", "Import");
	}

	return LOCTEXT("Export", "Export");
}

FReply FLandscapeEditorDetailCustomization_ImportExport::OnImportExportButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();	
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}

	UWorld* World = LandscapeEdMode->GetWorld();

	if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
	{
		FIntRect LandscapeExtent = LandscapeInfo->GetCompleteLandscapeExtent();
		LandscapeExtent.Max.X += 1;
		LandscapeExtent.Max.Y += 1;

		if (IsImporting())
		{
			if (LandscapeEdMode->UISettings->CurrentGizmoActor.IsValid())
			{
				FGuid CurrentLayerGuid = LandscapeEdMode->GetCurrentLayerGuid();
						
				const ELandscapeLayerPaintingRestriction PaintRestriction = ELandscapeLayerPaintingRestriction::None;
						
				ELandscapeImportTransformType TransformType = LandscapeEdMode->UISettings->ImportType;
									
				FVector LocalGizmoPosition = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld().InverseTransformPosition(LandscapeEdMode->UISettings->CurrentGizmoActor->GetActorLocation());
				FIntPoint LocalGizmoPoint = FIntPoint(FMath::FloorToInt32(LocalGizmoPosition.X), FMath::FloorToInt32(LocalGizmoPosition.Y));
			
				// Update Gizmo Position if we exit and comeback into tool
				LandscapeEdMode->UISettings->ImportLandscape_GizmoLocalPosition = LocalGizmoPoint;

				FIntRect ImportRegion = LandscapeExtent;
				FIntPoint ImportOffset(0, 0);
				if (TransformType == ELandscapeImportTransformType::ExpandOffset)
				{
					ImportOffset = LocalGizmoPoint - FIntPoint(LandscapeExtent.Min.X, LandscapeExtent.Min.Y);
				}
				else if (TransformType == ELandscapeImportTransformType::None ||
						 TransformType == ELandscapeImportTransformType::Subregion)
				{
					// We only know the image size here if a heightmap is being imported (which requires that all weightmaps match when imported at the same 
					// time).  Pass in the correct offset consistently and let ImportHeightData/ImportWeightData figure out what that means for the import region.
					ImportOffset = { LocalGizmoPoint.X, LocalGizmoPoint.Y };
				}
				// Note:  Resample mode is always aligned to the landscape and uses no offset.
			
				check(LandscapeEdMode->UISettings->ImportLandscape_HeightmapImportResult != ELandscapeImportResult::Error);
				const bool bIsWorldPartition = LandscapeEdMode->GetWorld()->GetSubsystem<ULandscapeSubsystem>()->IsGridBased();

				if (bIsWorldPartition && LandscapeEdMode->UISettings->ImportExportMode != ELandscapeImportExportMode::LoadedOnly && LandscapeRegionUtils::NumLandscapeRegions(LandscapeInfo) > 0) // If in world partition import region by region
				{
					TArray<ALocationVolume*> LandscapeRegions;
					TArray<AActor*> Children;
					LandscapeInfo->LandscapeActor->GetAttachedActors(Children);
					for (AActor* Child : Children)
					{
						if (Child->IsA<ALocationVolume>())
						{
							LandscapeRegions.Add(Cast<ALocationVolume>(Child));
						}
					}

					int32 NumRegions = LandscapeRegions.Num();
					
					FScopedSlowTask Progress(static_cast<float>(NumRegions), LOCTEXT("ImportingLandscapeRegions", "Importing Landscape Regions"));
					Progress.MakeDialog(/*bShowCancelButton = */ true);

					auto RegionImporter = [&Progress, LandscapeInfo, CurrentLayerGuid, LandscapeEdMode, ImportRegion, TransformType, ImportOffset, PaintRestriction](const FBox& RegionBounds, const TArray<ALandscapeProxy*>& Proxies)
					{
						// todo need a better way of working out the landscape bounds of the StreamingProxies in this region.
						// What if some of the landscape is loaded?
						FIntRect LandscapeLoadedExtent;
						LandscapeInfo->GetLandscapeExtent(LandscapeLoadedExtent);
						LandscapeLoadedExtent.Max.X += 1;
						LandscapeLoadedExtent.Max.Y += 1;

						Progress.EnterProgressFrame(1.0f, LOCTEXT("ImportingLandscapeRegions", "Importing Landscape Regions"));

						if (LandscapeEdMode->UISettings->bHeightmapSelected)
						{
							LandscapeEdMode->ImportHeightData(LandscapeInfo, CurrentLayerGuid, LandscapeEdMode->UISettings->ImportLandscape_HeightmapFilename, LandscapeLoadedExtent, TransformType, ImportOffset, PaintRestriction, LandscapeEdMode->UISettings->bFlipYAxis);
						}

						for (const FLandscapeImportLayer& ImportLayer : LandscapeEdMode->UISettings->ImportLandscape_Layers)
						{
							if (ImportLayer.bSelected)
							{
								check(ImportLayer.ImportResult != ELandscapeImportResult::Error);
								LandscapeEdMode->ImportWeightData(LandscapeInfo, CurrentLayerGuid, ImportLayer.LayerInfo, ImportLayer.SourceFilePath, LandscapeLoadedExtent, TransformType, ImportOffset, PaintRestriction, LandscapeEdMode->UISettings->bFlipYAxis);
							}
						}

						return !Progress.ShouldCancel();
					};

					LandscapeRegionUtils::ForEachRegion_LoadProcessUnload(LandscapeInfo, World, RegionImporter);

					FLandscapeImageFileCache& LandscapeImageFileCache = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor").GetImageFileCache();
					LandscapeImageFileCache.Clear();
				}
				else
				{
					FScopedSlowTask Progress(static_cast<float>(1 + LandscapeEdMode->UISettings->ImportLandscape_Layers.Num()), LOCTEXT("ImportingLandscape", "Importing Landscape"));
					Progress.MakeDialog(/*bShowCancelButton = */ true);

					if (LandscapeEdMode->UISettings->bHeightmapSelected)
					{
						Progress.EnterProgressFrame(1.0f, LOCTEXT("ImportingLandscapeHeight", "Importing Landscape Height"));
						LandscapeEdMode->ImportHeightData(LandscapeInfo, CurrentLayerGuid, LandscapeEdMode->UISettings->ImportLandscape_HeightmapFilename, ImportRegion, TransformType, ImportOffset, PaintRestriction, LandscapeEdMode->UISettings->bFlipYAxis);
					}

					for (const FLandscapeImportLayer& ImportLayer : LandscapeEdMode->UISettings->ImportLandscape_Layers)
					{
						if (Progress.ShouldCancel())
						{
							continue;
						}

						if (ImportLayer.bSelected)
						{
							Progress.EnterProgressFrame(1.0f, LOCTEXT("ImportingLandscapeWeight", "Importing Landscape Weight"));
							check(ImportLayer.ImportResult != ELandscapeImportResult::Error);
							LandscapeEdMode->ImportWeightData(LandscapeInfo, CurrentLayerGuid, ImportLayer.LayerInfo, ImportLayer.SourceFilePath, ImportRegion, TransformType, ImportOffset, PaintRestriction, LandscapeEdMode->UISettings->bFlipYAxis);
						}
					}

					LandscapeInfo->ForceLayersFullUpdate();
				}
			}
		}
		else // !IsImporting()
		{
			auto BuildExportFileName = [](const FString& Filename, const FIntPoint& FileOffset, bool bUseOffset)
			{
				if (bUseOffset)
				{
					FString Extension = FPaths::GetExtension(Filename, true);
					FString BaseFilename = FPaths::GetBaseFilename(Filename, false);
					return FString::Format(TEXT("{0}_x{1}_y{2}{3}"), { BaseFilename, FString::FromInt(FileOffset.X), FString::FromInt(FileOffset.Y), Extension });
				}
				
				return Filename;
			};

			auto PerformExport = [BuildExportFileName](ULandscapeEditorObject* InLandscapeEditorSettings, ULandscapeInfo* InLandscapeInfo, const FIntRect& InExportRegion, const FIntPoint& InFileOffset, bool bInUseOffset)
			{
				if (InLandscapeEditorSettings->bHeightmapSelected)
				{
					FString ExportFilename = BuildExportFileName(InLandscapeEditorSettings->HeightmapExportFilename, InFileOffset, bInUseOffset);
					InLandscapeInfo->ExportHeightmap(ExportFilename, InExportRegion);
				}

				for (const FLandscapeImportLayer& ImportLayer : InLandscapeEditorSettings->ImportLandscape_Layers)
				{
					if (ImportLayer.bSelected)
					{
						FString ExportFilename = BuildExportFileName(ImportLayer.ExportFilePath, InFileOffset, bInUseOffset);
						InLandscapeInfo->ExportLayer(ImportLayer.LayerInfo, ExportFilename, InExportRegion);
					}
				}
			};

			FScopedSetLandscapeEditingLayer Scope(LandscapeInfo->LandscapeActor.Get(), LandscapeEdMode->UISettings->bExportEditLayer ? LandscapeEdMode->GetCurrentLayerGuid() : FGuid());

			ALandscape* Landscape = LandscapeInfo->LandscapeActor.Get();
			check(Landscape);
			if (LandscapeEdMode->UseSingleFileImport() || LandscapeEdMode->UISettings->bExportSingleFile)
			{
				// For single file export, export the whole landscape (main actor) : 
				FIntRect ExportRegion;
				if (LandscapeInfo->GetLandscapeExtent(ExportRegion))
				{
					PerformExport(LandscapeEdMode->UISettings, LandscapeInfo, ExportRegion, FIntPoint(), /*bInUseOffset = */false);
				}
			}
			else
			{
				FScopedSlowTask Progress(0, LOCTEXT("ExportingLandscapeRegions", "Exporting Landscape Regions"));
				Progress.MakeDialog(/*bShowCancelButton = */ true);

				auto Exporter = [&Progress, LandscapeInfo, Landscape, LandscapeEdMode, LandscapeExtent, PerformExport, World](const FBox& RegionBounds, const TArray<ALandscapeProxy*>& Proxies)
				{
					Progress.EnterProgressFrame(0.0f, LOCTEXT("ExportingLandscapeRegions", "Exporting Landscape Regions"));
					for (ALandscapeProxy* LandscapeProxy : Proxies)
					{
						FIntRect ExportRegion;

						if (LandscapeInfo->GetLandscapeExtent(LandscapeProxy, ExportRegion))
						{
							const int32 YCoord = LandscapeEdMode->UISettings->bFlipYAxis ? LandscapeExtent.Max.Y - ExportRegion.Max.Y : ExportRegion.Min.Y - LandscapeExtent.Min.Y;
							FIntPoint FileOffset = FIntPoint((ExportRegion.Min.X - LandscapeExtent.Min.X) / Landscape->GetGridSize(),
								YCoord / Landscape->GetGridSize());

							// Remove the shared line/column that this proxy has with its neighbors because it 
							// will be included by the neighbor or lost if there is none (that could become an option to avoid that loss)
							ExportRegion.Max.X -= 1;
							ExportRegion.Max.Y -= 1;

							PerformExport(LandscapeEdMode->UISettings, LandscapeInfo, ExportRegion, FileOffset, /*bInUseOffset = */true);
						}
					}
					return !Progress.ShouldCancel();
					
				};

				LandscapeRegionUtils::ForEachRegion_LoadProcessUnload(LandscapeInfo, World, Exporter);
			

				// For multiple file export, export each landscape proxy individually :
				LandscapeInfo->ForEachLandscapeProxy([LandscapeInfo, Landscape, LandscapeEdMode, LandscapeExtent, BuildExportFileName, PerformExport](ALandscapeProxy* LandscapeProxy)
				{
					FIntRect ExportRegion;
					if (LandscapeInfo->GetLandscapeExtent(LandscapeProxy, ExportRegion))
					{
						const int32 YCoord = LandscapeEdMode->UISettings->bFlipYAxis ? LandscapeExtent.Max.Y - ExportRegion.Max.Y : ExportRegion.Min.Y - LandscapeExtent.Min.Y;
						FIntPoint FileOffset = FIntPoint((ExportRegion.Min.X - LandscapeExtent.Min.X) / Landscape->GetGridSize(),
							YCoord / Landscape->GetGridSize());

						// Remove the shared line/column that this proxy has with its neighbors because it 
						// will be included by the neighbor or lost if there is none (that could become an option to avoid that loss)
						ExportRegion.Max.X -= 1;
						ExportRegion.Max.Y -= 1;

						PerformExport(LandscapeEdMode->UISettings, LandscapeInfo, ExportRegion, FileOffset, /*bInUseOffset = */true);
					}
					return true;
				});
			}

			// Reset the import gizmo location to landscape minx/miny
			LandscapeEdMode->UISettings->ImportLandscape_GizmoLocalPosition = LandscapeExtent.Min;
		}
	}

	return FReply::Handled();
}

bool FLandscapeEditorDetailCustomization_ImportExport::GetImportExportButtonIsEnabled()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return false;
	}

	bool bHasOneSelection = false;
	if (LandscapeEdMode->UISettings->bHeightmapSelected)
	{
		bHasOneSelection = true;
		if (IsImporting())
		{
			if (LandscapeEdMode->UISettings->ImportLandscape_HeightmapImportResult == ELandscapeImportResult::Error || LandscapeEdMode->UISettings->ImportLandscape_HeightmapFilename.IsEmpty())
			{
				return false;
			}
		}
		else
		{
			if (LandscapeEdMode->UISettings->HeightmapExportFilename.IsEmpty())
			{
				return false;
			}
		}
	}

	for (const FLandscapeImportLayer& ImportLayer : LandscapeEdMode->UISettings->ImportLandscape_Layers)
	{
		if (ImportLayer.bSelected)
		{
			bHasOneSelection = true;
			
			if (IsImporting())
			{
				if (ImportLayer.ImportResult == ELandscapeImportResult::Error || ImportLayer.SourceFilePath.IsEmpty())
				{
					return false;
				}
			}
			else
			{
				if (ImportLayer.ExportFilePath.IsEmpty())
				{
					return false;
				}
			}
		}
	}

	return bHasOneSelection && LandscapeEdMode->IsLandscapeResolutionCompliant();
}

EVisibility FLandscapeEditorDetailCustomization_ImportExport::GetImportingVisibility() const
{
	return IsImporting() ? EVisibility::Visible : EVisibility::Hidden;
}

EVisibility FLandscapeEditorDetailCustomization_ImportExport::GetImportExportLandscapeErrorVisibility() const
{
	FEdModeLandscape* EdMode = GetEditorMode();

	if (EdMode != nullptr)
	{
		return EdMode->IsLandscapeResolutionCompliant() ? EVisibility::Hidden : EVisibility::Visible;
	}

	return EVisibility::Hidden;
}

FText FLandscapeEditorDetailCustomization_ImportExport::GetImportExportLandscapeErrorText() const
{
	FEdModeLandscape* EdMode = GetEditorMode();

	if (EdMode != nullptr)
	{
		return EdMode->GetLandscapeResolutionErrorText();
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_ImportExport::GetImportLandscapeResolutionMenu()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	FMenuBuilder MenuBuilder(true, nullptr);
	if (LandscapeEdMode)
	{
		for (int32 i = 0; i < LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions.Num(); i++)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Width"), LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions[i].Width);
			Args.Add(TEXT("Height"), LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions[i].Height);
			MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("ImportResolution_Format", "{Width}\u00D7{Height}"), Args), FText(), FSlateIcon(), FExecuteAction::CreateStatic(&FLandscapeEditorDetailCustomization_ImportExport::OnChangeImportLandscapeResolution, i));
		}
	}
	
	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_ImportExport::OnChangeImportLandscapeResolution(int32 Index)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		LandscapeEdMode->UISettings->OnChangeImportLandscapeResolution(Index);
	}
}

FText FLandscapeEditorDetailCustomization_ImportExport::GetImportLandscapeResolution()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FText::GetEmpty();
	}

	const int32	Width = LandscapeEdMode->UISettings->ImportLandscape_Width;
	const int32	Height = LandscapeEdMode->UISettings->ImportLandscape_Height;
		
	if (Width != 0 && Height != 0)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Width"), Width);
		Args.Add(TEXT("Height"), Height);
		return FText::Format(LOCTEXT("ImportResolution_Format", "{Width}\u00D7{Height}"), Args);
	}

	return LOCTEXT("ImportResolution_Invalid", "(invalid)");
}



#undef LOCTEXT_NAMESPACE
