// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "LandscapeEdMode.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class IDetailLayoutBuilder;

/**
 * Slate widgets customizer for the "Import Export" tool
 */

namespace LandscapeTestUtils
{
	struct LandscapeTestCommands;
} //namespace LandscapeTestUtils

class FLandscapeEditorDetailCustomization_ImportExport : public FLandscapeEditorDetailCustomization_Base
{
	// So tests have access to private members
	friend struct LandscapeTestUtils::LandscapeTestCommands;
public:
	FLandscapeEditorDetailCustomization_ImportExport()
	{}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

	static void FormatFilename(TSharedRef<IPropertyHandle> PropertyHandle_Filename, bool bForExport);
private:
	static bool GetExportSingleFileIsEnabled();
	static ECheckBoxState GetExportSingleFileCheckState();
	static void OnExportSingleFileCheckStateChanged(ECheckBoxState NewCheckState);
	static EVisibility GetImportExportVisibility(bool bImport);
	static bool IsHeightmapEnabled();
	static ECheckBoxState GetHeightmapSelectedCheckState();
	static void OnHeightmapSelectedCheckStateChanged(ECheckBoxState CheckState);
	static ECheckBoxState ModeIsChecked(EImportExportMode Value);
	static void OnModeChanged(ECheckBoxState NewCheckedState, EImportExportMode Value);

	static EVisibility GetImportResultErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult);
	static FSlateColor GetImportResultErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult);
	static void SetFilename(const FText& NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle_Filename);
	static FReply OnBrowseFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_Filename);

	static FText GetImportExportButtonText();
	static FReply OnImportExportButtonClicked();
	static bool GetImportExportButtonIsEnabled();

	EVisibility GetImportExportLandscapeErrorVisibility() const;
	EVisibility GetImportingVisibility() const;
	FText GetImportExportLandscapeErrorText() const;

	static TSharedRef<SWidget> GetImportLandscapeResolutionMenu();
	static void OnChangeImportLandscapeResolution(int32 DescriptorIndex);
	static FText GetImportLandscapeResolution();

	static void OnImportHeightmapFilenameChanged();
	
	static bool IsImporting();
};
