// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_ImportLayers.h"
#include "LandscapeEditorDetailCustomization_ImportExport.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEditorUtils.h"
#include "LandscapeSettings.h"
#include "LandscapeUtils.h"
#include "SLandscapeEditor.h"

#include "IDetailChildrenBuilder.h"
#include "DetailWidgetRow.h"

#include "PropertyCustomizationHelpers.h"

#include "Dialogs/DlgPickAssetPath.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SToolTip.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "ObjectTools.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.ImportLayers"

TSharedRef<IPropertyTypeCustomization> FLandscapeEditorStructCustomization_FLandscapeImportLayer::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorStructCustomization_FLandscapeImportLayer);
}

bool FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsImporting()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
	if (IsToolActive("ImportExport"))
	{
		return LandscapeEdMode->ImportExportMode == EImportExportMode::Import;
	}

		if (IsToolActive("NewLandscape"))
		{
			if (LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
			{
	return true;
}
		}
	}

	return false;
}

void FLandscapeEditorStructCustomization_FLandscapeImportLayer::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FLandscapeEditorStructCustomization_FLandscapeImportLayer::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedRef<IPropertyHandle> PropertyHandle_LayerName = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, LayerName)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, LayerInfo)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_SourceFilePath = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, SourceFilePath)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ExportFilePath = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, ExportFilePath)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ThumbnailMIC = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, ThumbnailMIC)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ImportResult = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, ImportResult)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ErrorMessage = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, ErrorMessage)).ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Selected = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeImportLayer, bSelected)).ToSharedRef();
	
	PropertyHandle_SourceFilePath->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle_SourceFilePath]()
		{
			FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_SourceFilePath, /*bForExport = */false);
			FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportWeightmapFilenameChanged();
		}));

	PropertyHandle_ExportFilePath->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([PropertyHandle_ExportFilePath]()
		{
			FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_ExportFilePath, /*bForExport = */true);
		}));

	FName LayerName;
	FText LayerNameText;
	FPropertyAccess::Result Result = PropertyHandle_LayerName->GetValue(LayerName);
	check(Result != FPropertyAccess::Fail);
	LayerNameText = FText::FromName(LayerName);
	if (Result == FPropertyAccess::MultipleValues)
	{
		LayerName = NAME_None;
		LayerNameText = NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	UObject* ThumbnailObject = nullptr;

	UObject* ThumbnailMIC = nullptr;
	Result = PropertyHandle_ThumbnailMIC->GetValue(ThumbnailMIC);
	checkSlow(Result == FPropertyAccess::Success);

	UObject* LayerInfo = nullptr;
	Result = PropertyHandle_LayerInfo->GetValue(LayerInfo);
	checkSlow(Result == FPropertyAccess::Success);

	// If the layer info is valid and has a custom thumbnail, it's considered as an override, so use this as the thumbnail object : 
	ThumbnailObject = ((LayerInfo != nullptr) && ThumbnailTools::AssetHasCustomThumbnail(LayerInfo->GetFullName())) ? LayerInfo : ThumbnailMIC;

	auto CreateFilePathNameContentWidget = [&](bool bImport)
		{
			return SNew(SHorizontalBox)
				.Visibility_Static(&GetImportExportVisibility, bImport)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Left)
				.Padding(FMargin(2))
				[
					SNew(SCheckBox)
						.Visibility_Static(GetImportLayerSelectionVisibility)
						.IsEnabled_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsValidLayerInfo, PropertyHandle_LayerInfo)
						.IsChecked_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerSelectedCheckState, PropertyHandle_Selected, PropertyHandle_LayerInfo)
						.OnCheckStateChanged_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportLayerSelectedCheckStateChanged, PropertyHandle_Selected)
						.ToolTipText_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerSelectedToolTip, PropertyHandle_Selected, PropertyHandle_LayerInfo)
				]
			+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(2))
				[
					SNew(SLandscapeAssetThumbnail, ThumbnailObject, StructCustomizationUtils.GetThumbnailPool().ToSharedRef(), FName("LandscapeEditor.Target_Weightmap"))
						.ThumbnailSize(FIntPoint(48, 48))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Center)
				.Padding(FMargin(2))
				[
					SNew(STextBlock)
						.Font(StructCustomizationUtils.GetRegularFont())
						.Text(LayerNameText)
				];
		};

	auto CreateFilePathContentWidget = [&](TSharedRef<IPropertyHandle> PropertyHandle_Filepath, bool bImport)
		{
			return SNew(SBox)
				.VAlign(VAlign_Center)
				.Visibility_Static(&GetImportExportVisibility, bImport)
				.IsEnabled_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsImportLayerSelected, PropertyHandle_Selected, PropertyHandle_LayerInfo)
				[
					SNew(SVerticalBox)
						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SHorizontalBox)
								.Visibility_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetLayerInfoAssignVisibility)
								+ SHorizontalBox::Slot()
								[
									SNew(SObjectPropertyEntryBox)
										.AllowedClass(ULandscapeLayerInfoObject::StaticClass())
										.PropertyHandle(PropertyHandle_LayerInfo)
										.OnShouldFilterAsset_Static(&ShouldFilterLayerInfo, LayerName)
										.AllowCreate(false)
								]
							+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SButton)
										.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
										.ContentPadding(4.0f)
										.ForegroundColor(FSlateColor::UseForeground())
										.IsFocusable(false)
										.ToolTipText(LOCTEXT("Target_Create", "Create Layer Info"))
										.Visibility_Static(&GetImportLayerCreateVisibility, PropertyHandle_LayerInfo)
										.OnClicked_Lambda([PropertyHandle_LayerInfo, LayerName]() { return FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportLayerCreateClicked(PropertyHandle_LayerInfo, LayerName); })
										[
											SNew(SImage)
												.Image(FAppStyle::GetBrush("Icons.Plus"))
												.ColorAndOpacity(FSlateColor::UseForeground())
										]
								]
						]
					+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SVerticalBox)
								.Visibility_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerVisibility)
								.IsEnabled_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsImportLayerSelected, PropertyHandle_Selected, PropertyHandle_LayerInfo)
								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SHorizontalBox)
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(0, 0, 2, 0)
										[
											SNew(SErrorText)
												.Visibility_Static(&GetErrorVisibility, PropertyHandle_ImportResult)
												.BackgroundColor_Static(&GetErrorColor, PropertyHandle_ImportResult)
												.ErrorText(NSLOCTEXT("UnrealEd", "Error", "!"))
												.ToolTip(
													SNew(SToolTip)
													.Text_Static(&GetErrorText, PropertyHandle_ErrorMessage)
												)
										]
									+ SHorizontalBox::Slot()
										[
											PropertyHandle_Filepath->CreatePropertyValueWidget()
										]
										+ SHorizontalBox::Slot()
										.AutoWidth()
										.Padding(1, 0, 0, 0)
										[
											SNew(SButton)
												.ContentPadding(FMargin(4, 0))
												.Text(NSLOCTEXT("UnrealEd", "GenericOpenDialog", "..."))
												.OnClicked_Static(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnLayerFilenameButtonClicked, PropertyHandle_Filepath)
										]
								]
						]
				];
		};

	// Here we add both the Source (Import) and export file paths, one is always hidden by the visibility lambda
	// Each property handle needs to be registered to a custom row for the Reset To Default arrows to be shown
	IDetailPropertyRow& ImportFilePathRow = ChildBuilder.AddProperty(PropertyHandle_SourceFilePath)
		.Visibility(TAttribute<EVisibility>::CreateLambda([PropertyHandle_Selected]()
			{
				return GetImportExportVisibility(/*bImport=*/true);
			}));

	ImportFilePathRow.CustomWidget()
		.NameContent()
		[
			CreateFilePathNameContentWidget(/*bImport=*/true)
		]
		.ValueContent()
		.MinDesiredWidth(250.0f) // copied from SPropertyEditorAsset::GetDesiredWidth
		.MaxDesiredWidth(0)
		[
			CreateFilePathContentWidget(PropertyHandle_SourceFilePath, /*bImport=*/true)
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateLambda([this, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath](TSharedPtr<IPropertyHandle> InProperty)
				{
					return IsResetFilePathButtonShown(PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath);
				}),
			FResetToDefaultHandler::CreateLambda([this, PropertyHandle_LayerInfo, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath](TSharedPtr<IPropertyHandle> InProperty)
				{
					OnResetFilePathButtonClicked(PropertyHandle_LayerInfo, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath);
				})
		));

	// Export file path (row hidden while in import mode)
	IDetailPropertyRow& ExportFilePathRow = ChildBuilder.AddProperty(PropertyHandle_ExportFilePath)
		.Visibility(TAttribute<EVisibility>::CreateLambda([PropertyHandle_Selected]()
			{
				return GetImportExportVisibility(/*bImport=*/false);
			}));

	ExportFilePathRow.CustomWidget()
		.NameContent()
		[
			CreateFilePathNameContentWidget(/*bImport=*/false)
		]
		.ValueContent()
		.MinDesiredWidth(250.0f) // copied from SPropertyEditorAsset::GetDesiredWidth
		.MaxDesiredWidth(0)
		[
			CreateFilePathContentWidget(PropertyHandle_ExportFilePath, /*bImport=*/false)
		]
		.OverrideResetToDefault(FResetToDefaultOverride::Create(
			FIsResetToDefaultVisible::CreateLambda([this, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath](TSharedPtr<IPropertyHandle> InProperty)
				{
					return IsResetFilePathButtonShown(PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath);
				}),
			FResetToDefaultHandler::CreateLambda([this, PropertyHandle_LayerInfo, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath](TSharedPtr<IPropertyHandle> InProperty)
				{
					OnResetFilePathButtonClicked(PropertyHandle_LayerInfo, PropertyHandle_Selected, PropertyHandle_SourceFilePath, PropertyHandle_ExportFilePath);
				})
		));
}

void FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnResetFilePathButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo, TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_SourceFilePath, TSharedRef<IPropertyHandle> PropertyHandle_ExportFilePath)
{
	// Reset the file path 
	IsImporting() ? PropertyHandle_SourceFilePath->SetValue(FString()) : PropertyHandle_ExportFilePath->SetValue(FString());

	// Reset the selected check box
	PropertyHandle_Selected->SetValue(false);

	if (IsValidLayerInfo(PropertyHandle_LayerInfo))
	{
		// In New Landscape Import mode, each row has an optional layer info selector that should be reset
		ULandscapeLayerInfoObject* EmptyLayerInfo = nullptr;
		PropertyHandle_LayerInfo->SetValue(EmptyLayerInfo);
	}
}

bool FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsResetFilePathButtonShown(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_SourceFilePath, TSharedRef<IPropertyHandle> PropertyHandle_ExportFilePath)
{
	TSharedRef<IPropertyHandle> CurrentFilePathHandle = IsImporting() ? PropertyHandle_SourceFilePath : PropertyHandle_ExportFilePath;

	FString FilePathValue;
	CurrentFilePathHandle->GetValue(FilePathValue);

	bool bIsSelected = false;
	PropertyHandle_Selected->GetValue(bIsSelected);

	// If the checkbox is selected or the file path is set show the reset button
	return bIsSelected || !FilePathValue.IsEmpty();
}

void FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportWeightmapFilenameChanged()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		LandscapeEdMode->UISettings->OnImportWeightmapFilenameChanged();
	}
}

FReply FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnLayerFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerFilename)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const bool bIsImporting = IsImporting();
	const FString DialogTypeString = bIsImporting  ? LandscapeEditorModule.GetWeightmapImportDialogTypeString() : LandscapeEditorModule.GetWeightmapExportDialogTypeString();
	const FString DialogTitle = bIsImporting ? LOCTEXT("ImportLayer", "Import Layer").ToString() : LOCTEXT("ExportLayer", "Export Layer").ToString();

	TOptional<FString> OptionalExportImportPath = LandscapeEditorUtils::GetImportExportFilename(DialogTitle, LandscapeEdMode->UISettings->LastImportPath, DialogTypeString, bIsImporting);
	// If the user canceled the dialog (Filename is empty), don't clear the path
	if (OptionalExportImportPath.IsSet() && !OptionalExportImportPath->IsEmpty())
	{
		const FString& Filename = OptionalExportImportPath.GetValue();

		ensure(PropertyHandle_LayerFilename->SetValue(Filename) == FPropertyAccess::Success);
		LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(Filename);
	}

	return FReply::Handled();
}

bool FLandscapeEditorStructCustomization_FLandscapeImportLayer::ShouldFilterLayerInfo(const FAssetData& AssetData, FName LayerName)
{
	const FName LayerNameMetaData = AssetData.GetTagValueRef<FName>("LayerName");
	if (!LayerNameMetaData.IsNone())
	{
		return LayerNameMetaData != LayerName;
	}

	ULandscapeLayerInfoObject* LayerInfo = CastChecked<ULandscapeLayerInfoObject>(AssetData.GetAsset());
	return LayerInfo->GetLayerName() != LayerName;
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerSelectionVisibility()
{
	return IsToolActive("ImportExport") ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerCreateVisibility(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo)
{
	if (IsToolActive("ImportExport"))
	{
		return EVisibility::Collapsed;
	}

	
	return IsValidLayerInfo(PropertyHandle_LayerInfo) ? EVisibility::Collapsed : EVisibility::Visible;
}

bool FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsValidLayerInfo(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo)
{
	UObject* LayerInfoAsUObject = nullptr;
	return (PropertyHandle_LayerInfo->GetValue(LayerInfoAsUObject) != FPropertyAccess::Fail && LayerInfoAsUObject != nullptr);
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportExportVisibility(bool bImport)
{
	return IsImporting() == bImport ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportLayerCreateClicked(const TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo, FName LayerName)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}
	
	// Build default layer object name and package name
	FName FileName;
	FString PackageName = UE::Landscape::GetLayerInfoObjectPackageName(LayerName, LandscapeEdMode->GetTargetLayerAssetPackagePath(), FileName);

	TSharedRef<SDlgPickAssetPath> NewLayerDlg =
		SNew(SDlgPickAssetPath)
		.Title(LOCTEXT("CreateNewLayerInfo", "Create New Landscape Layer Info Object"))
		.DefaultAssetPath(FText::FromString(PackageName));

	if (NewLayerDlg->ShowModal() != EAppReturnType::Cancel)
	{
		PackageName = NewLayerDlg->GetAssetPath().ToString();
		FileName = FName(*NewLayerDlg->GetAssetName().ToString());

		// Creates a new layer info object asset, using the default if available, or a new empty one
		ULandscapeLayerInfoObject* LayerInfo = UE::Landscape::CreateTargetLayerInfo(LayerName, PackageName, *FileName.ToString());
		LayerInfo->SetBlendMethod(GetDefault<ULandscapeSettings>()->GetTargetLayerDefaultBlendMethod(), /*bInModify = */false);

		const UObject* LayerInfoAsUObject = LayerInfo; // HACK: If SetValue took a reference to a const ptr (T* const &) or a non-reference (T*) then this cast wouldn't be necessary
		ensure(PropertyHandle_LayerInfo->SetValue(LayerInfoAsUObject) == FPropertyAccess::Success);

		// Show in the content browser
		TArray<UObject*> Objects;
		Objects.Add(LayerInfo);
		GEditor->SyncBrowserToObjects(Objects);
	}
	
	return FReply::Handled();
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetLayerInfoAssignVisibility()
{
	return IsToolActive("ImportExport") ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerVisibility()
{
	if (IsToolActive("ImportExport"))
	{
		return EVisibility::Visible;
	}

	if (IsToolActive("NewLandscape"))
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (LandscapeEdMode != nullptr)
		{
			if (LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult)
{
	ELandscapeImportResult WeightmapImportResult;
	FPropertyAccess::Result Result = PropertyHandle_ImportResult->GetValue((uint8&)WeightmapImportResult);

	if (Result == FPropertyAccess::Fail ||
		Result == FPropertyAccess::MultipleValues)
	{
		return EVisibility::Visible;
	}

	if (WeightmapImportResult != ELandscapeImportResult::Success)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Collapsed;
}

FSlateColor FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult)
{
	ELandscapeImportResult WeightmapImportResult;
	FPropertyAccess::Result Result = PropertyHandle_ImportResult->GetValue((uint8&)WeightmapImportResult);

	if (Result == FPropertyAccess::Fail ||
		Result == FPropertyAccess::MultipleValues)
	{
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	}

	switch (WeightmapImportResult)
	{
	case ELandscapeImportResult::Success:
		return FCoreStyle::Get().GetColor("InfoReporting.BackgroundColor");
	case ELandscapeImportResult::Warning:
		return FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor");
	case ELandscapeImportResult::Error:
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	default:
		check(0);
		return FSlateColor();
	}
}

FText FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetErrorText(TSharedRef<IPropertyHandle> PropertyHandle_ErrorMessage)
{
	FText ErrorMessage;
	FPropertyAccess::Result Result = PropertyHandle_ErrorMessage->GetValue(ErrorMessage);
	if (Result == FPropertyAccess::Fail)
	{
		return LOCTEXT("Import_LayerUnknownError", "Unknown Error");
	}
	else if (Result == FPropertyAccess::MultipleValues)
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return ErrorMessage;
}

bool FLandscapeEditorStructCustomization_FLandscapeImportLayer::IsImportLayerSelected(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo)
{
	if (IsToolActive("ImportExport"))
	{
		// Need a valid layer info to import/export
		if (!IsValidLayerInfo(PropertyHandle_LayerInfo))
		{
			return false;
		}

		bool bSelected;
		FPropertyAccess::Result Result = PropertyHandle_Selected->GetValue(bSelected);
		return Result == FPropertyAccess::Success ? bSelected : false;
	}

	return true;
}

ECheckBoxState FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerSelectedCheckState(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo)
{
	return IsImportLayerSelected(PropertyHandle_Selected, PropertyHandle_LayerInfo) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText FLandscapeEditorStructCustomization_FLandscapeImportLayer::GetImportLayerSelectedToolTip(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo)
{
	if (!IsValidLayerInfo(PropertyHandle_LayerInfo))
	{
		return LOCTEXT("InvalidLayerInfo", "This layer doesn't have a valid LayerInfo object assigned.");
	}

	return FText();
}

void FLandscapeEditorStructCustomization_FLandscapeImportLayer::OnImportLayerSelectedCheckStateChanged(ECheckBoxState CheckState, TSharedRef<IPropertyHandle> PropertyHandle_Selected)
{
	ensure(PropertyHandle_Selected->SetValue(CheckState == ECheckBoxState::Checked, EPropertyValueSetFlags::NotTransactable) == FPropertyAccess::Success);
}

#undef LOCTEXT_NAMESPACE
