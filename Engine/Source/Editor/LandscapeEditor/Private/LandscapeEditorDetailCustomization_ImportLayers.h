// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "LandscapeEdMode.h"
#include "PropertyHandle.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class FLandscapeEditorStructCustomization_FLandscapeImportLayer : public FLandscapeEditorStructCustomization_Base
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	static void OnImportWeightmapFilenameChanged();
	static FReply OnLayerFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerFilename);
	static bool ShouldFilterLayerInfo(const struct FAssetData& AssetData, FName LayerName);

	static EVisibility GetImportLayerCreateVisibility(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo);
	static TSharedRef<SWidget> OnGetImportLayerCreateMenu(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo, FName LayerName);
	static FReply OnImportLayerCreateClicked(const TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo, FName LayerName);
	static EVisibility GetImportLayerSelectionVisibility();
	static EVisibility GetImportExportVisibility(bool bImport);

	static EVisibility GetLayerInfoAssignVisibility();
	static EVisibility GetImportLayerVisibility();
	static EVisibility GetErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult);
	static FSlateColor GetErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_ImportResult);
	static FText GetErrorText(TSharedRef<IPropertyHandle> PropertyHandle_ErrorMessage);

	static void OnResetFilePathButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo, TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_SourceFilePath, TSharedRef<IPropertyHandle> PropertyHandle_ExportFilePath);
	static bool IsResetFilePathButtonShown(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_SourceFilePath, TSharedRef<IPropertyHandle> PropertyHandle_ExportFilePath);
	static bool IsImportLayerSelected(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo);
	static ECheckBoxState GetImportLayerSelectedCheckState(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo);
	static void OnImportLayerSelectedCheckStateChanged(ECheckBoxState CheckState, TSharedRef<IPropertyHandle> PropertyHandle_Selected);
	static FText GetImportLayerSelectedToolTip(TSharedRef<IPropertyHandle> PropertyHandle_Selected, TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo);
	static bool IsValidLayerInfo(TSharedRef<IPropertyHandle> PropertyHandle_LayerInfo);
	static bool IsImporting();
};
