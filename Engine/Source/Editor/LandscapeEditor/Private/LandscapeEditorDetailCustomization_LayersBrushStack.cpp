// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_LayersBrushStack.h"
#include "IDetailChildrenBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Brushes/SlateColorBrush.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditLayer.h"
#include "Styling/StyleColors.h"

#include "ScopedTransaction.h"

#include "LandscapeEditorDetailCustomization_TargetLayers.h"
#include "Widgets/Input/SEditableText.h"
#include "LandscapeBlueprintBrushBase.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.Layers"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_LayersBrushStack::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_LayersBrushStack);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_LayersBrushStack::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode
		&& LandscapeEdMode->GetLandscape()
		&& (LandscapeEdMode->CurrentToolMode != nullptr)
		&& (FName(LandscapeEdMode->CurrentTool->GetToolName()) != TEXT("Mask")))
	{
		IDetailCategoryBuilder& LayerCategory = DetailBuilder.EditCategory(FName("Edit Layer Blueprint Brushes"));

		LayerCategory.AddCustomBuilder(MakeShareable(new FLandscapeEditorCustomNodeBuilder_LayersBrushStack(DetailBuilder.GetThumbnailPool().ToSharedRef())));
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

//////////////////////////////////////////////////////////////////////////

FEdModeLandscape* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetEditorMode()
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

FLandscapeEditorCustomNodeBuilder_LayersBrushStack::FLandscapeEditorCustomNodeBuilder_LayersBrushStack(TSharedRef<FAssetThumbnailPool> InThumbnailPool)
	: ThumbnailPool(InThumbnailPool)
{
}

FLandscapeEditorCustomNodeBuilder_LayersBrushStack::~FLandscapeEditorCustomNodeBuilder_LayersBrushStack()
{
	
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->CurrentToolMode)
	{
		const int32 BrushCount = LandscapeEdMode->GetBrushesForCurrentLayer().Num();
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetCurrentEditLayerConst();
		check(EditLayer != nullptr);

		TSharedPtr<SDragAndDropVerticalBox> BrushesList = SNew(SDragAndDropVerticalBox)
			.OnCanAcceptDropAdvanced(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleAcceptDrop)
			.OnDragDetected(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleDragDetected);

		BrushesList->SetDropIndicator_Above(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Above"));
		BrushesList->SetDropIndicator_Below(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Below"));

		const TSharedPtr<SWidget> BrushesListWidget = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(VAlign_Center)
			.Padding(0, 2)
			[
				BrushesList.ToSharedRef()
			];

		// Display a disabled message when the current edit layer does not support BP brushes
		const FText DisabledToolTip = FText::Format(LOCTEXT("LandscapeEditLayerBlueprintBrushes_Disabled", "The type of layer {0} ({1}) doesn't support blueprint brushes."),
			FText::FromName(EditLayer->GetName()), EditLayer->GetClass()->GetDisplayNameText());

		const TSharedPtr<SWidget> ErrorMessage = SNew(SMultiLineEditableTextBox)
			.IsReadOnly(true)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
			.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
			.Text(DisabledToolTip)
			.AutoWrapText(true);

		ChildrenBuilder.AddCustomRow(LOCTEXT("LandscapeEditLayerBlueprintBrushes", "Edit Layer Blueprint Brushes"))
			.WholeRowContent()
			[
				EditLayer->SupportsBlueprintBrushes() ? BrushesListWidget.ToSharedRef() : ErrorMessage.ToSharedRef()
			];

		for (int32 i = 0; i < BrushCount; ++i)
		{
			TSharedPtr<SWidget> GeneratedRowWidget = GenerateRow(i);
			if (GeneratedRowWidget.IsValid())
			{
				BrushesList->AddSlot()
					.AutoHeight()
					[
						GeneratedRowWidget.ToSharedRef()
					];
			}
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GenerateRow(int32 InBrushIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	TSharedPtr<SWidget> RowWidget = SNew(SLandscapeEditorSelectableBorder)
		.Padding(FMargin(8.0f, 0.f))
		.VAlign(VAlign_Center)
		.OnContextMenuOpening(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnBrushContextMenuOpening, InBrushIndex)
		.OnSelected(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnBrushSelectionChanged, InBrushIndex)
		.IsSelected(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsBrushSelected, InBrushIndex)))
		[	
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SButton)
				.ContentPadding(0.0f)
				.ButtonStyle(FAppStyle::Get(), "NoBorder")
				.OnClicked(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleVisibility, InBrushIndex)
				.ToolTipText(LOCTEXT("LandscapeBrushVisibility", "Toggle Brush Visibility"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Content()
				[
					SNew(SImage)
					.Image(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetVisibilityBrush, InBrushIndex)
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4, 0)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(0, 2)
				[
					SNew(STextBlock)
					.ColorAndOpacity(TAttribute<FSlateColor>::Create(TAttribute<FSlateColor>::FGetter::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrushTextColor, InBrushIndex)))
					.Text(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrushText, InBrushIndex)
					.IsEnabled(true)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked_Lambda([InBrushIndex, this]() { return this->IsAffectingHeightmap(InBrushIndex) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Raw(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsHeightmap, InBrushIndex)
				.ToolTipText(LOCTEXT("LandscapeBrushAffectsHeightmap", "Toggle Affects Heightmap"))
				.HAlign(HAlign_Center)
				.Padding(FMargin(5.f, 3.f))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsHeightmapBrush, InBrushIndex)
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 4, 0)
			.VAlign(VAlign_Center)
            [
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked_Lambda([InBrushIndex, this]() { return this->IsAffectingWeightmap(InBrushIndex) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Raw(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsWeightmap, InBrushIndex)
				.ToolTipText(LOCTEXT("LandscapeBrushAffectsWeightmap", "Toggle Affects Weightmap"))
				.HAlign(HAlign_Center)
				.Padding(FMargin(5.f, 3.f))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsWeightmapBrush, InBrushIndex)
				]
            ]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
				.IsChecked_Lambda([InBrushIndex, this]() { return this->IsAffectingVisibilityLayer(InBrushIndex) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Raw(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsVisibilityLayer, InBrushIndex)
				.ToolTipText(LOCTEXT("LandscapeBrushAffectsVisibilityLayer", "Toggle Affects Visibility Layer"))
				.HAlign(HAlign_Center)
				.Padding(FMargin(5.f, 3.f))
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(this, &FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsVisibilityLayerBrush, InBrushIndex)
				]
			]
		];
	
	return RowWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnBrushContextMenuOpening(int32 InBrushIndex)
{
	// Don't use GetBrush here as it will return nullptr if the layer contains a nullptr brush, and we still want to allow the Remove context menu when this happens : 
	TArray<ALandscapeBlueprintBrushBase*> CurrentBrushes = GetBrushes();
	if (CurrentBrushes.IsValidIndex(InBrushIndex))
	{
		ALandscapeBlueprintBrushBase* CurrentBrush = CurrentBrushes[InBrushIndex];

		FMenuBuilder MenuBuilder(true, NULL);
		MenuBuilder.BeginSection("LandscapeEditorBrushActions", LOCTEXT("LandscapeEditorBrushActions.Heading", "Brushes"));
		{
			TSharedRef<FLandscapeEditorCustomNodeBuilder_LayersBrushStack> SharedThis = AsShared();

			if (CurrentBrush != nullptr)
			{
				// Show only selected
				FUIAction ShowOnlySelectedAction = FUIAction(FExecuteAction::CreateLambda([SharedThis, CurrentBrush]
				{
					const FScopedTransaction Transaction(LOCTEXT("LandscapeBrushShowOnlySelectedTransaction", "Show Only Selected Brush"));
					GetEditorMode()->ShowOnlySelectedBrush(CurrentBrush);
				}));
				MenuBuilder.AddMenuEntry(LOCTEXT("LandscapeBrushShowOnlySelected", "Show Only Selected"), LOCTEXT("LandscapeBrushShowOnlySelectedToolTip", "Hides all other brushes from the same layer"), FSlateIcon(), ShowOnlySelectedAction);

				// Hide selected
				FUIAction ShowHideSelectedAction = FUIAction(FExecuteAction::CreateLambda([SharedThis, CurrentBrush]
				{
					SharedThis->OnToggleVisibility(CurrentBrush);
				}));

				FText MenuText = CurrentBrush->IsVisible() ? LOCTEXT("LandscapeBrushHideSelected", "Hide Selected") : LOCTEXT("LandscapeBrushShowSelected", "Show Selected");
				MenuBuilder.AddMenuEntry(MenuText, LOCTEXT("LandscapeBrushToggleVisiblityToolTip", "Toggle brush visiblity"), FSlateIcon(), ShowHideSelectedAction);

				MenuBuilder.AddMenuSeparator();

				// Duplicate Brush
				FUIAction DuplicateAction = FUIAction(FExecuteAction::CreateLambda([SharedThis, CurrentBrush]
				{
					const FScopedTransaction Transaction(LOCTEXT("LandscapeBrushDuplicateTransaction", "Duplicate Brush"));
					GetEditorMode()->DuplicateBrush(CurrentBrush);
				}));
				MenuBuilder.AddMenuEntry(LOCTEXT("LandscapeBrushDuplicate", "Duplicate"), LOCTEXT("LandscapeBrushDuplicateToolTip", "Duplicate brush"), FSlateIcon(), DuplicateAction);

				// Remove Brush
				FUIAction RemoveAction = FUIAction(FExecuteAction::CreateLambda([SharedThis, InBrushIndex]
					{
						const FScopedTransaction Transaction(LOCTEXT("LandscapeBrushRemoveTransaction", "Remove Brush"));
						GetEditorMode()->RemoveBrushFromCurrentLayer(InBrushIndex);
					}));
				MenuBuilder.AddMenuEntry(LOCTEXT("LandscapeBrushRemove", "Remove"), LOCTEXT("LandscapeBrushRemoveToolTip", "Remove brush from selected edit layer"), FSlateIcon(), RemoveAction);
			}
		}
		MenuBuilder.EndSection();
		return MenuBuilder.MakeWidget();
	}

	return nullptr;
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsHeightmap(ECheckBoxState InCheckBoxState, int32 InBrushIndex)
{
	if ((InCheckBoxState != ECheckBoxState::Checked) && (InCheckBoxState != ECheckBoxState::Unchecked))
	{
		return;
	}

	if (ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Brush_AffectsHeightmap", "Set Brush Can Affect Heightmap"));
		Brush->SetCanAffectHeightmap(InCheckBoxState == ECheckBoxState::Checked);
	}
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsWeightmap(ECheckBoxState InCheckBoxState, int32 InBrushIndex)
{
	if ((InCheckBoxState != ECheckBoxState::Checked) && (InCheckBoxState != ECheckBoxState::Unchecked))
	{
		return;
	}

	if (ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Brush_AffectsWeightmap", "Set Brush Can Affect Weightmap"));
		Brush->SetCanAffectWeightmap(InCheckBoxState == ECheckBoxState::Checked);
	}
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleAffectsVisibilityLayer(ECheckBoxState InCheckBoxState, int32 InBrushIndex)
{
	if ((InCheckBoxState != ECheckBoxState::Checked) && (InCheckBoxState != ECheckBoxState::Unchecked))
	{
		return;
	}

	if (ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex))
	{
		const FScopedTransaction Transaction(LOCTEXT("Landscape_Brush_AffectsVisibilityLayer", "Set Brush Can Affect Visibility Layer"));
		Brush->SetCanAffectVisibilityLayer(InCheckBoxState == ECheckBoxState::Checked);
	}
}

FReply FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleVisibility(int32 InBrushIndex)
{
	if (ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex))
	{
		OnToggleVisibility(Brush);
	}
	return FReply::Handled();
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnToggleVisibility(ALandscapeBlueprintBrushBase* Brush)
{
	const FScopedTransaction Transaction(LOCTEXT("Landscape_Brush_SetVisibility", "Set Brush Visibility"));
	bool bVisible = Brush->IsVisible();
	Brush->SetIsVisible(!bVisible);
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsWeightmapBrush(int32 InBrushIndex) const
{
	return IsAffectingWeightmap(InBrushIndex) ? FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsWeightmap") : FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsWeightmap.Disabled");
}

bool FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsAffectingWeightmap(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);

	return Brush && Brush->CanAffectWeightmap();
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsHeightmapBrush(int32 InBrushIndex) const
{
	return IsAffectingHeightmap(InBrushIndex) ? FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsHeightmap") : FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsHeightmap.Disabled");
}

bool FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsAffectingHeightmap(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);

	return Brush && Brush->CanAffectHeightmap();
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetAffectsVisibilityLayerBrush(int32 InBrushIndex) const
{
	return IsAffectingVisibilityLayer(InBrushIndex) ? FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsVisibilityLayer.Enabled") : FAppStyle::GetBrush("LandscapeEditor.Brush.AffectsVisibilityLayer.Disabled");
}

bool FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsAffectingVisibilityLayer(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);

	return Brush && Brush->CanAffectVisibilityLayer();
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetVisibilityBrush(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);
	bool bIsVisible = Brush && Brush->IsVisible();
	return bIsVisible ? FAppStyle::GetBrush("Level.VisibleIcon16x") : FAppStyle::GetBrush("Level.NotVisibleIcon16x");
}

bool FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsBrushEnabled(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	return Brush 
		&& ((Brush->CanAffectHeightmap() && LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap) 
		|| (Brush->CanAffectWeightmap() && LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap) 
		|| (Brush->CanAffectVisibilityLayer() && LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility));
}

bool FLandscapeEditorCustomNodeBuilder_LayersBrushStack::IsBrushSelected(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);
	return Brush && Brush->IsSelected();
}

void FLandscapeEditorCustomNodeBuilder_LayersBrushStack::OnBrushSelectionChanged(int32 InBrushIndex)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode || !LandscapeEdMode->CurrentToolMode)
	{
		return;
	}

	const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();
	const FName BlueprintBrushTool = TEXT("BlueprintBrush");
	bool bSwitchTool = CurrentToolName != BlueprintBrushTool;
	
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);
	if (Brush != nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("LandscapeBrushSelect", "Brush selection"));
		if (bSwitchTool)
		{
			LandscapeEdMode->SetCurrentTool(BlueprintBrushTool);
		}
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(Brush, true, true);
	}
}

FText FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrushText(int32 InBrushIndex) const
{
	if (ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex))
	{
		return FText::FromString(Brush->GetActorLabel());
	}

	return FText::FromName(NAME_None);
}

FSlateColor FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrushTextColor(int32 InBrushIndex) const
{
	ALandscapeBlueprintBrushBase* Brush = GetBrush(InBrushIndex);

	if (Brush == nullptr)
	{
		return FSlateColor::UseSubduedForeground();
	}

	return IsBrushSelected(InBrushIndex) ? FStyleColors::ForegroundHover : FSlateColor::UseForeground();
}

ALandscapeBlueprintBrushBase* FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrush(int32 InBrushIndex) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		return LandscapeEdMode->GetBrushForCurrentLayer(InBrushIndex);
	}

	return nullptr;
}

TArray<ALandscapeBlueprintBrushBase*> FLandscapeEditorCustomNodeBuilder_LayersBrushStack::GetBrushes() const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		return LandscapeEdMode->GetBrushesForCurrentLayer();
	}

	return TArray<ALandscapeBlueprintBrushBase*>();
}

FReply FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		if (const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetCurrentEditLayerConst())
		{
			const TArray<ALandscapeBlueprintBrushBase*>& BrushStack = LandscapeEdMode->GetBrushesForCurrentLayer();
			if (!EditLayer->IsLocked() && BrushStack.IsValidIndex(SlotIndex))
			{
				TSharedPtr<SWidget> Row = GenerateRow(SlotIndex);
				if (Row.IsValid())
				{
					return FReply::Handled().BeginDragDrop(FLandscapeBrushDragDropOp::New(SlotIndex, Slot, Row));
				}
			}
		}
	}
	return FReply::Unhandled();
}

TOptional<SDragAndDropVerticalBox::EItemDropZone> FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FLandscapeBrushDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FLandscapeBrushDragDropOp>();

	if (DragDropOperation.IsValid())
	{
		return DropZone;
	}

	return TOptional<SDragAndDropVerticalBox::EItemDropZone>();
}

FReply FLandscapeEditorCustomNodeBuilder_LayersBrushStack::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FLandscapeBrushDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FLandscapeBrushDragDropOp>();

	if (DragDropOperation.IsValid())
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
		if (Landscape)
		{
			int32 StartingBrushIndex = DragDropOperation->SlotIndexBeingDragged;
			int32 DestinationBrushIndex = SlotIndex;
			const FScopedTransaction Transaction(LOCTEXT("Landscape_LayerBrushes_Reorder", "Reorder Layer Brush"));
			if (Landscape->ReorderLayerBrush(LandscapeEdMode->GetSelectedEditLayerIndex(), StartingBrushIndex, DestinationBrushIndex))
			{
				LandscapeEdMode->RefreshDetailPanel();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

TSharedRef<FLandscapeBrushDragDropOp> FLandscapeBrushDragDropOp::New(int32 InSlotIndexBeingDragged, 
	SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> InWidgetToShow)
{
	// Same code as FLandscapeListElementDragDropOp, just returns a different type.

	TSharedRef<FLandscapeBrushDragDropOp> Operation = MakeShared<FLandscapeBrushDragDropOp>();

	Operation->MouseCursor = EMouseCursor::GrabHandClosed;
	Operation->SlotIndexBeingDragged = InSlotIndexBeingDragged;
	Operation->SlotBeingDragged = InSlotBeingDragged;
	Operation->WidgetToShow = InWidgetToShow;

	Operation->Construct();

	return Operation;
}

#undef LOCTEXT_NAMESPACE