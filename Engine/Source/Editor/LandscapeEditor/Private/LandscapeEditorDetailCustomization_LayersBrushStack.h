// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "LandscapeEditorDetailCustomization_EditLayersStack.h" // FLandscapeListElementDragDropOp
#include "LandscapeEdMode.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "AssetThumbnail.h"
#include "Framework/SlateDelegates.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class SDragAndDropVerticalBox;
class FMenuBuilder;

/**
 * Slate widgets customizer for the layers list in the Landscape Editor
 */

class FLandscapeEditorDetailCustomization_LayersBrushStack : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

class FLandscapeEditorCustomNodeBuilder_LayersBrushStack : public IDetailCustomNodeBuilder, public TSharedFromThis<FLandscapeEditorCustomNodeBuilder_LayersBrushStack>
{
public:
	FLandscapeEditorCustomNodeBuilder_LayersBrushStack(TSharedRef<FAssetThumbnailPool> ThumbnailPool);
	~FLandscapeEditorCustomNodeBuilder_LayersBrushStack();

	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick( float DeltaTime ) override {}
	virtual bool RequiresTick() const override { return false; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override { return "Brush Stack"; }

protected:
	TSharedRef<FAssetThumbnailPool> ThumbnailPool;

	static class FEdModeLandscape* GetEditorMode();

	TSharedPtr<SWidget> GenerateRow(int32 InBrushIndex);
	TSharedPtr<SWidget> OnBrushContextMenuOpening(int32 InBrushIndex);

	FReply OnToggleVisibility(int32 InBrushIndex);
	void OnToggleAffectsHeightmap(ECheckBoxState InCheckBoxState, int32 InBrushIndex);
	void OnToggleAffectsWeightmap(ECheckBoxState InCheckBoxState, int32 InBrushIndex);
	void OnToggleAffectsVisibilityLayer(ECheckBoxState InCheckBoxState, int32 InBrushIndex);

	void OnToggleVisibility(ALandscapeBlueprintBrushBase* Brush);

	const FSlateBrush* GetAffectsHeightmapBrush(int32 InBrushIndex) const;
	bool IsAffectingHeightmap(int32 InBrushIndex) const;

	const FSlateBrush* GetAffectsWeightmapBrush(int32 InBrushIndex) const;
	bool IsAffectingWeightmap(int32 InBrushIndex) const;

	const FSlateBrush* GetAffectsVisibilityLayerBrush(int32 InBrushIndex) const;
	bool IsAffectingVisibilityLayer(int32 InBrushIndex) const;

	const FSlateBrush* GetVisibilityBrush(int32 InBrushIndex) const;
	
	bool IsBrushSelected(int32 InBrushIndex) const;
	bool IsBrushEnabled(int32 InBrushIndex) const;
	void OnBrushSelectionChanged(int32 InBrushIndex);
	FText GetBrushText(int32 InBrushIndex) const;
	FSlateColor GetBrushTextColor(int32 InBrushIndex) const;
	ALandscapeBlueprintBrushBase* GetBrush(int32 InBrushIndex) const;
	TArray<ALandscapeBlueprintBrushBase*> GetBrushes() const;

	// Drag/Drop handling
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	TOptional<SDragAndDropVerticalBox::EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);
};

class FLandscapeBrushDragDropOp : public FLandscapeListElementDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FLandscapeBrushDragDropOp, FLandscapeListElementDragDropOp)

	static TSharedRef<FLandscapeBrushDragDropOp> New(int32 InSlotIndexBeingDragged, 
		SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> InWidgetToShow);

public:
	virtual ~FLandscapeBrushDragDropOp() = default;
};
