// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_MiscTools.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Text/STextBlock.h"
#include "SlateOptMacros.h"
#include "SWarningOrErrorBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "LandscapeEditorObject.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"

#include "ScopedTransaction.h"
#include "IDetailPropertyRow.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "SFlattenHeightEyeDropperButton.h"
#include "LandscapeEdModeTools.h"
#include "LandscapeSplineSelection.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.Tools"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_MiscTools::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_MiscTools);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_MiscTools::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& ToolsCategory = DetailBuilder.EditCategory("Tool Settings");
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return;
	}

	if (IsBrushSetActive("BrushSet_Component"))
	{
		ToolsCategory.AddCustomRow(LOCTEXT("Component.ClearSelection", "Clear Component Selection"))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FLandscapeEditorDetailCustomization_MiscTools::GetClearComponentSelectionVisibility)))
		[
			SNew(SButton)
			.Text(LOCTEXT("Component.ClearSelection", "Clear Component Selection"))
			.ToolTipText(LOCTEXT("Component.ClearSelectionToolTip", "Removes all components from the current selection"))
			.HAlign(HAlign_Center)
			.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnClearComponentSelectionButtonClicked)
		];
	}

	IDetailCategoryBuilder& SelectMaskCategory = DetailBuilder.EditCategory("Select Mask");
	{
		SelectMaskCategory.AddCustomRow(LOCTEXT("Mask.ClearSelection", "Clear Region Selection"))
		.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FLandscapeEditorDetailCustomization_MiscTools::GetClearRegionSelectionVisibility)))
		[
			SNew(SButton)
			.Text(LOCTEXT("Mask.ClearSelection", "Clear Region Selection"))
			.ToolTipText(LOCTEXT("Mask.ClearSelectionToolTip", "Removes all painted regions from the current selection"))
			.HAlign(HAlign_Center)
			.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnClearRegionSelectionButtonClicked)
		];
	}

	if (IsToolActive("Flatten"))
	{
		TSharedRef<IPropertyHandle> FlattenValueProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, FlattenTarget));
		IDetailPropertyRow& FlattenValueRow = ToolsCategory.AddProperty(FlattenValueProperty);
		FlattenValueRow.CustomWidget()
			.NameContent()
			[
				FlattenValueProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.Padding(0.0f, 2.0f, 5.0f, 2.0f)
				.FillWidth(1.0f)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(SNumericEntryBox<float>)
					.AllowSpin(true)
					.Font(DetailBuilder.GetDetailFont())
					.Value(this, &FLandscapeEditorDetailCustomization_MiscTools::GetFlattenValue)
					.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<float>, FlattenValueProperty)
					.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<float>, FlattenValueProperty)
					.SliderExponentNeutralValue(this, &FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMid)
					.SliderExponent(5.0f)
					.MinSliderValue(this, &FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMin)
					.MaxSliderValue(this, &FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMax)
					.MinDesiredValueWidth(75.0f)
					.ToolTipText(LOCTEXT("FlattenToolTips", "Target height to flatten towards (in Unreal Units)"))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 2.0f, 5.0f, 2.0f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SNew(SFlattenHeightEyeDropperButton)
					.OnBegin(this, &FLandscapeEditorDetailCustomization_MiscTools::OnBeginFlattenToolEyeDrop)
					.OnComplete(this, &FLandscapeEditorDetailCustomization_MiscTools::OnCompletedFlattenToolEyeDrop)
				]		
			];

		TSharedRef<IPropertyHandle> TerraceIntervalProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TerraceInterval));
		IDetailPropertyRow& TerraceIntervalRow = ToolsCategory.AddProperty(TerraceIntervalProperty);
		TerraceIntervalRow.CustomWidget()
			.NameContent()
			[
				TerraceIntervalProperty->CreatePropertyNameWidget()
			]
			.ValueContent()
			[
				SNew(SNumericEntryBox<float>)
					.AllowSpin(true)
					.Font(DetailBuilder.GetDetailFont())
					.Value_Static(&FLandscapeEditorDetailCustomization_Base::GetOptionalPropertyValue<float>, TerraceIntervalProperty)
					.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<float>, TerraceIntervalProperty)
					.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<float>, TerraceIntervalProperty)
					.SliderExponent(5.0f)
					.MinValue(1.0f)
					.MinSliderValue(1.0f)
					.MaxSliderValue(this, &FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTerraceIntervalValueMax)
					.ToolTipText(LOCTEXT("TerraceIntervalToolTips", "Height of the terrace intervals in unreal units"))
			];
	}

	if (IsToolActive("Splines"))
	{
		ToolsCategory.AddCustomRow(LOCTEXT("SelectAllLabel", "Select all"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 6, 0, 0)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.ShadowOffset(FVector2D::UnitVector)
				.Text(LOCTEXT("Spline.SelectAll", "Select All:"))
			]
		];
		ToolsCategory.AddCustomRow(LOCTEXT("SelectAllLabel", "Select all"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("Spline.ControlPoints.All.Tooltip", "Selects all landscape spline points in the map."))
				.Text(LOCTEXT("Spline.ControlPoints", "Points"))
				.HAlign(HAlign_Center)
				.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnSelectAllControlPointsButtonClicked)
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.ToolTipText(LOCTEXT("Spline.Segments.All.Tooltip", "Selects all landscape spline segments in the map."))
				.Text(LOCTEXT("Spline.Segments", "Segments"))
				.HAlign(HAlign_Center)
				.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnSelectAllSegmentsButtonClicked)
			]
		];
		ToolsCategory.AddCustomRow(LOCTEXT("Spline.bUseAutoRotateControlPoint.Selected", "Use Auto Rotate Point"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 6, 0, 0)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FLandscapeEditorDetailCustomization_MiscTools::OnbUseAutoRotateControlPointChanged)
				.IsChecked(this, &FLandscapeEditorDetailCustomization_MiscTools::GetbUseAutoRotateControlPoint)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Spline.bUseAutoRotateControlPoint.Selected", "Use Auto Rotate Point"))
					.ToolTipText(LOCTEXT("Spline.bUseAutoRotateControlPoint.Selected.Tooltip", "If enabled, the spline point auto-rotates to maintain a smooth spline after releasing the mouse button."))
				]
			]
		];
		ToolsCategory.AddCustomRow(LOCTEXT("Spline.bAlwaysForward.Selected", "Auto-Rotate Always Forward"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 6, 0, 0)
			[
				SNew(SCheckBox)
				.OnCheckStateChanged(this, &FLandscapeEditorDetailCustomization_MiscTools::OnbAlwaysRotateForwardChanged)
				.IsChecked(this, &FLandscapeEditorDetailCustomization_MiscTools::GetbAlwaysRotateForward)
				.Content()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Spline.bAlwaysForward.Selected", "Auto-Rotate Always Forward"))
					.ToolTipText(LOCTEXT("Spline.bAlwaysForward.Selected.Tooltip", "If enabled, the spline point auto-rotates in a forward direction to maintain a consistent orientation after releasing the mouse button."))
				]
			]
		];
	}


	if (IsToolActive("Ramp"))
	{
		ToolsCategory.AddCustomRow(LOCTEXT("RampLabel", "Ramp"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0, 6, 0, 0)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.ShadowOffset(FVector2D::UnitVector)
				.Text(LOCTEXT("Ramp.Hint", "Click to add ramp points, then press \"Add Ramp\"."))
			]
		];
		ToolsCategory.AddCustomRow(LOCTEXT("ApplyRampLabel", "Apply Ramp"))
		[
			SNew(SBox)
			.Padding(FMargin(0, 0, 12, 0)) // Line up with the other properties due to having no reset to default button
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 3, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Ramp.Reset", "Reset"))
					.ToolTipText(LOCTEXT("Ramp.ResetToolTip", "Clear the added ramp points"))
					.HAlign(HAlign_Center)
					.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnResetRampButtonClicked)
				]
				+ SHorizontalBox::Slot()
				.Padding(3, 0, 0, 0)
				[
					SNew(SButton)
					.IsEnabled_Static(&FLandscapeEditorDetailCustomization_MiscTools::GetApplyRampButtonIsEnabled)
					.Text(LOCTEXT("Ramp.Apply", "Add Ramp"))
					.ToolTipText(LOCTEXT("Ramp.ApplyToolTip", "Applies the current ramp to the height map of the currently selected edit layer"))
					.HAlign(HAlign_Center)
					.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnApplyRampButtonClicked)
				]
			]
		];
	}

	if (IsToolActive("Mirror"))
	{
		ToolsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MirrorPoint));
		ToolsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MirrorOp));
		ToolsCategory.AddCustomRow(LOCTEXT("ApplyMirrorLabel", "Apply Mirror"))
		[
			SNew(SBox)
			.Padding(FMargin(0, 0, 12, 0)) // Line up with the other properties due to having no reset to default button
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(0, 0, 3, 0)
				[
					SNew(SButton)
					.Text(LOCTEXT("Mirror.Reset", "Recenter"))
					.ToolTipText(LOCTEXT("Mirror.ResetToolTip", "Center the mirror point on the current landscape"))
					.HAlign(HAlign_Center)
					.OnClicked_Lambda(&FLandscapeEditorDetailCustomization_MiscTools::OnResetMirrorPointButtonClicked)
				]
				+ SHorizontalBox::Slot()
				.Padding(3, 0, 0, 0)
				[
					SNew(SButton)
					.IsEnabled_Lambda([]() { FEdModeLandscape* LandscapeEdMode = GetEditorMode(); return LandscapeEdMode && LandscapeEdMode->CanEditLayer(); })
					.Text(LOCTEXT("Mirror.Apply", "Apply"))
					.ToolTipText(LOCTEXT("Mirror.ApplyToolTip", "Apply the mirror operation to the current landscape edit layer"))
					.HAlign(HAlign_Center)
					.OnClicked_Static(&FLandscapeEditorDetailCustomization_MiscTools::OnApplyMirrorButtonClicked)
				]
			]
		];
	}

	if (IsToolActive("AddComponent"))
	{
		ToolsCategory.AddCustomRow(FText::GetEmpty())
		[
			SNew(SWarningOrErrorBox)
			.Message(this, &FLandscapeEditorDetailCustomization_MiscTools::GetMiscLandscapeErrorText)
		]
		.Visibility(TAttribute<EVisibility>(this, &FLandscapeEditorDetailCustomization_MiscTools::GetMiscLandscapeErrorVisibility));
	}

	// Display the Current Gizmo location (Vector3) 
	// Only revealing a single ALandscapeGizmoActiveActor property - if more fields need to be exposed later, consider registering a struct customization instead
	TSharedRef<IPropertyHandle> CurrentGizmoHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, CurrentGizmoActor));
	TWeakObjectPtr<ALandscapeGizmoActiveActor> GizmoActor = LandscapeEdMode->UISettings->CurrentGizmoActor.Get();
	if (GizmoActor.IsValid())
	{
		FSlateFontInfo FontInfo = IDetailLayoutBuilder::GetDetailFont();

		ToolsCategory.AddProperty(CurrentGizmoHandle)
			.CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
					.Text(LOCTEXT("LandscapeImportGizmoLocation", "Import Gizmo Location"))
					.Font(IDetailLayoutBuilder::GetDetailFont())
			]
			.ValueContent()
			.MinDesiredWidth(125.0f * 3.0f)
			.MaxDesiredWidth(125.0f * 3.0f)
			.VAlign(VAlign_Center)
			[
				// Since FComponentTransformDetails::GenerateChildContent is not exposed, create a minimal vector component to display location
				// Drag features intentionally omitted to keep this widget minimal
				SNew(SVectorInputBox)
					.bColorAxisLabels(true)
					.Font(FontInfo)
					.X(TAttribute<TOptional<float>>::Create([GizmoActor]() { return GizmoActor.IsValid() ? GizmoActor->GetActorLocation().X : 0; }))
					.Y(TAttribute<TOptional<float>>::Create([GizmoActor]() { return GizmoActor.IsValid() ? GizmoActor->GetActorLocation().Y : 0; }))
					.Z(TAttribute<TOptional<float>>::Create([GizmoActor]() { return GizmoActor.IsValid() ? GizmoActor->GetActorLocation().Z : 0; }))
					.OnXCommitted_Lambda([GizmoActor](float NewX, ETextCommit::Type) 
					{
						if (GizmoActor.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("LandscapeUpdateGizmoX", "Landscape Update Gizmo X Location"));
							GizmoActor->Modify();
							// Apply delta from the current location
							GizmoActor->EditorApplyTranslation(FVector(NewX - GizmoActor->GetActorLocation().X, 0, 0), /*bAltDown =*/false, /*bShiftDown =*/false, /*bCtrlDown =*/false);
						}
					})
					.OnYCommitted_Lambda([GizmoActor](float NewY, ETextCommit::Type) 
					{
						if (GizmoActor.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("LandscapeUpdateGizmoY", "Landscape Update Gizmo Y Location"));
							GizmoActor->Modify();
							// Apply delta from the current location
							GizmoActor->EditorApplyTranslation(FVector(0, NewY - GizmoActor->GetActorLocation().Y, 0), /*bAltDown =*/false, /*bShiftDown =*/false, /*bCtrlDown =*/false);
						}
					})
					.OnZCommitted_Lambda([GizmoActor](float NewZ, ETextCommit::Type) 
					{
						if (GizmoActor.IsValid())
						{
							FScopedTransaction Transaction(LOCTEXT("LandscapeUpdateGizmoZ", "Landscape Update Gizmo Z Location"));
							GizmoActor->Modify();
							// Apply delta from the current location
							GizmoActor->EditorApplyTranslation(FVector(0, 0, NewZ - GizmoActor->GetActorLocation().Z), /*bAltDown =*/false, /*bShiftDown =*/false, /*bCtrlDown =*/false);
						}
					})
			]
			.OverrideResetToDefault(FResetToDefaultOverride::Create(
				FIsResetToDefaultVisible::CreateStatic(&FLandscapeEditorDetailCustomization_MiscTools::IsResetGizmoButtonShown),
				FResetToDefaultHandler::CreateStatic(&FLandscapeEditorDetailCustomization_MiscTools::ResetGizmoToOrigin)
			));
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FLandscapeEditorDetailCustomization_MiscTools::IsResetGizmoButtonShown(TSharedPtr<IPropertyHandle> InProperty)
{
	if (FEdModeLandscape * LandscapeEdMode = GetEditorMode())
	{
		TWeakObjectPtr<ULandscapeInfo> LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo;
		TWeakObjectPtr<ALandscapeGizmoActiveActor> CurrentGizmoActor = LandscapeEdMode->UISettings->CurrentGizmoActor;

		check(LandscapeInfo.Get());
		check(CurrentGizmoActor.Get());

		FIntRect LandscapeExtent;
		LandscapeInfo->GetLandscapeExtent(LandscapeExtent);
		const FTransform LandscapeToWorldTransform = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld();
		const FVector LandscapeOrigin = LandscapeToWorldTransform.TransformPosition(FVector(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, 0.0f));

		return CurrentGizmoActor->GetActorLocation() != LandscapeOrigin;
	}
	return false;
}

void FLandscapeEditorDetailCustomization_MiscTools::ResetGizmoToOrigin(TSharedPtr<IPropertyHandle> InProperty)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return;
	}

	TWeakObjectPtr<ULandscapeInfo> LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo;
	TWeakObjectPtr<ALandscapeGizmoActiveActor> CurrentGizmoActor = LandscapeEdMode->UISettings->CurrentGizmoActor;

	check(LandscapeInfo.Get());
	check(CurrentGizmoActor.Get());

	FIntRect LandscapeExtent;
	LandscapeInfo->GetLandscapeExtent(LandscapeExtent);
	const FTransform LandscapeToWorldTransform = LandscapeInfo->GetLandscapeProxy()->LandscapeActorToWorld();
	const FVector LandscapeOrigin = LandscapeToWorldTransform.TransformPosition(FVector(LandscapeExtent.Min.X, LandscapeExtent.Min.Y, 0.0f));

	CurrentGizmoActor->Modify();
	CurrentGizmoActor->EditorApplyTranslation(LandscapeOrigin - CurrentGizmoActor->GetActorLocation(), /*bAltDown =*/false, /*bShiftDown =*/false, /*bCtrlDown =*/false);
}

EVisibility FLandscapeEditorDetailCustomization_MiscTools::GetClearComponentSelectionVisibility()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();
		if (CurrentToolName == FName("Select"))
		{
			return EVisibility::Visible;
		}
		else if (LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetSelectedComponents().Num() > 0)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnClearComponentSelectionButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo)
		{
			FScopedTransaction Transaction(LOCTEXT("Component.Undo_ClearSelected", "Clearing Selection"));
			LandscapeInfo->Modify();

			TSet<ULandscapeComponent*> PreviouslySelectedComponents = LandscapeInfo->GetSelectedComponents();
			LandscapeInfo->ClearSelectedRegion(true);

			// Remove the previously selected components from the selected objects in the details view: 
			FPropertyEditorModule& PropertyModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
			TArray<UObject*> ObjectsToRemove(PreviouslySelectedComponents.Array());
			PropertyModule.RemoveDeletedObjects(ObjectsToRemove);
		}
	}

	return FReply::Handled();
}

EVisibility FLandscapeEditorDetailCustomization_MiscTools::GetClearRegionSelectionVisibility()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();
		if (CurrentToolName == FName("Mask"))
		{
			return EVisibility::Visible;
		}
		else if (LandscapeEdMode->CurrentTool && LandscapeEdMode->CurrentTool->SupportsMask() &&
			LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->SelectedRegion.Num() > 0)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnClearRegionSelectionButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (LandscapeInfo)
		{
			FScopedTransaction Transaction(LOCTEXT("Region.Undo_ClearSelected", "Clearing Region Selection"));
			LandscapeInfo->Modify();
			LandscapeInfo->ClearSelectedRegion(false);
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnSelectAllControlPointsButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		// Clears current selection before selecting all control points
		LandscapeEdMode->SelectAllSplineControlPoints(ESplineNavigationFlags::None);
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnSelectAllSegmentsButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		// Clears current selection before selecting all segments
		LandscapeEdMode->SelectAllSplineSegments(ESplineNavigationFlags::None);
	}

	return FReply::Handled();
}

void FLandscapeEditorDetailCustomization_MiscTools::OnbUseAutoRotateControlPointChanged(ECheckBoxState NewState)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->SetbUseAutoRotateOnJoin(NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState FLandscapeEditorDetailCustomization_MiscTools::GetbUseAutoRotateControlPoint() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		return LandscapeEdMode->GetbUseAutoRotateOnJoin() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

void FLandscapeEditorDetailCustomization_MiscTools::OnbAlwaysRotateForwardChanged(ECheckBoxState NewState)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->SetbAlwaysRotateForward(NewState == ECheckBoxState::Checked);
	}
}

ECheckBoxState FLandscapeEditorDetailCustomization_MiscTools::GetbAlwaysRotateForward() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		return LandscapeEdMode->GetbAlwaysRotateForward() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	return ECheckBoxState::Unchecked;
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnApplyRampButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && IsToolActive(FName("Ramp")))
	{
		LandscapeEdMode->ApplyRampTool();
	}

	return FReply::Handled();
}

bool FLandscapeEditorDetailCustomization_MiscTools::GetApplyRampButtonIsEnabled()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && IsToolActive(FName("Ramp")))
	{
		return LandscapeEdMode->CanApplyRampTool();
	}

	return false;
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnResetRampButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && IsToolActive(FName("Ramp")))
	{
		LandscapeEdMode->ResetRampTool();
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnApplyMirrorButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && IsToolActive(FName("Mirror")))
	{
		LandscapeEdMode->ApplyMirrorTool();
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_MiscTools::OnResetMirrorPointButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && IsToolActive(FName("Mirror")))
	{
		LandscapeEdMode->CenterMirrorTool();
	}

	return FReply::Handled();
}

TOptional<float> FLandscapeEditorDetailCustomization_MiscTools::GetFlattenValue() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr && IsToolActive(FName("Flatten")))
	{
		return LandscapeEdMode->UISettings->GetFlattenTarget(/* bInReturnPreviewValueIfActive = */true);
	}

	return 0.0f;
}

TOptional<float> FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMin() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if ((LandscapeEdMode != nullptr) && IsToolActive(FName("Flatten")))
	{
		if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			if (ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy())
			{
				return LandscapeProxy->ActorToWorld().TransformPosition(FVector(0.0, 0.0, LandscapeDataAccess::GetLocalHeight(0))).Z;
			}
		}		
	}

	return TOptional<float>();
}

float FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMid() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if ((LandscapeEdMode != nullptr) && IsToolActive(FName("Flatten")))
	{
		if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			if (ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy())
			{
				return LandscapeProxy->ActorToWorld().TransformPosition(FVector(0.0, 0.0, LandscapeDataAccess::GetLocalHeight(LandscapeDataAccess::MidValue))).Z;
			}
		}
	}

	return 0.0f;
}

TOptional<float> FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTargetValueMax() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if ((LandscapeEdMode != nullptr) && IsToolActive(FName("Flatten")))
	{
		if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			if (ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy())
			{
				return LandscapeProxy->ActorToWorld().TransformPosition(FVector(0.0, 0.0, LandscapeDataAccess::GetLocalHeight(LandscapeDataAccess::MaxValue))).Z;
			}
		}
	}

	return TOptional<float>();
}

TOptional<float> FLandscapeEditorDetailCustomization_MiscTools::GetFlattenTerraceIntervalValueMax() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if ((LandscapeEdMode != nullptr) && IsToolActive(FName("Flatten")))
	{
		if (ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get())
		{
			if (ALandscapeProxy* LandscapeProxy = LandscapeInfo->GetLandscapeProxy())
			{
				float WorldMax = LandscapeProxy->ActorToWorld().TransformPosition(FVector(0.0, 0.0, LandscapeDataAccess::GetLocalHeight(LandscapeDataAccess::MaxValue))).Z;
				float WorldMin = LandscapeProxy->ActorToWorld().TransformPosition(FVector(0.0, 0.0, LandscapeDataAccess::GetLocalHeight(0))).Z;
				return FMath::Max(WorldMax - WorldMin, 1.0f);
			}
		}
	}

	return TOptional<float>();
}

EVisibility FLandscapeEditorDetailCustomization_MiscTools::GetMiscLandscapeErrorVisibility() const
{
	FEdModeLandscape* EdMode = GetEditorMode();
	
	if (EdMode != nullptr)
	{
		return EdMode->IsLandscapeResolutionCompliant() ? EVisibility::Hidden : EVisibility::Visible;
	}
	
	return EVisibility::Hidden;
}

FText FLandscapeEditorDetailCustomization_MiscTools::GetMiscLandscapeErrorText() const
{
	FEdModeLandscape* EdMode = GetEditorMode();

	if (EdMode != nullptr)
	{
		return EdMode->GetLandscapeResolutionErrorText();
	}

	return FText::GetEmpty();
}

void FLandscapeEditorDetailCustomization_MiscTools::OnBeginFlattenToolEyeDrop()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr && IsToolActive(FName("Flatten")))
	{
		LandscapeEdMode->UISettings->bFlattenEyeDropperModeActivated = true;
		// Use the current target value when activating the eye drop for consistency. This will be reset when the mouse moves in the viewport anyway : 
		LandscapeEdMode->UISettings->FlattenEyeDropperModeDesiredTarget = LandscapeEdMode->UISettings->FlattenTarget;
		LandscapeEdMode->CurrentTool->SetCanToolBeActivated(false);
	}
}

void FLandscapeEditorDetailCustomization_MiscTools::OnCompletedFlattenToolEyeDrop(bool Canceled)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr && IsToolActive(FName("Flatten")))
	{
		// Consider clicks outside the viewport as a cancellation : this avoids validating the preview value when clicking outside the viewport
		if (!Canceled && LandscapeEdMode->UISettings->bFlattenEyeDropperModeActivated && LandscapeEdMode->UISettings->bFlattenEyeDropperModeMousingOverViewport)
		{
			LandscapeEdMode->UISettings->FlattenTarget = LandscapeEdMode->UISettings->FlattenEyeDropperModeDesiredTarget;
		}
		LandscapeEdMode->UISettings->bFlattenEyeDropperModeActivated = false;
		LandscapeEdMode->CurrentTool->SetCanToolBeActivated(true);
	}
}

#undef LOCTEXT_NAMESPACE
