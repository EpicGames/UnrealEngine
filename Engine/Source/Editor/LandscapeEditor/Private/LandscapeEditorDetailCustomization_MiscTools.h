// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "LandscapeEditorDetailCustomization_Base.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class IDetailLayoutBuilder;

namespace LandscapeTestUtils
{
	struct LandscapeTestCommands;
} //namespace LandscapeTestUtils

/**
 * Slate widgets customizer for the smaller tools that need little customization
 */

class FLandscapeEditorDetailCustomization_MiscTools : public FLandscapeEditorDetailCustomization_Base
{
	// Give tests access to private members
	friend struct LandscapeTestUtils::LandscapeTestCommands;

public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	// Component Selection Tool
	static EVisibility GetClearComponentSelectionVisibility();
	static FReply OnClearComponentSelectionButtonClicked();

	// Mask Tool
	static EVisibility GetClearRegionSelectionVisibility();
	static FReply OnClearRegionSelectionButtonClicked();

	// Splines Tool
	static FReply OnSelectAllControlPointsButtonClicked();
	static FReply OnSelectAllSegmentsButtonClicked();
	void OnbUseAutoRotateControlPointChanged(ECheckBoxState NewState);
	ECheckBoxState GetbUseAutoRotateControlPoint() const;
	void OnbAlwaysRotateForwardChanged(ECheckBoxState NewState);
	ECheckBoxState GetbAlwaysRotateForward() const;

	// Ramp Tool
	static FReply OnApplyRampButtonClicked();
	static bool GetApplyRampButtonIsEnabled();
	static FReply OnResetRampButtonClicked();

	// Mirror Tool
	static FReply OnApplyMirrorButtonClicked();
	static FReply OnResetMirrorPointButtonClicked();

	// Flatten Tool
	void HandleFlattenValueChanged(float NewValue);
	void OnBeginFlattenToolEyeDrop();
	void OnCompletedFlattenToolEyeDrop(bool Canceled);
	TOptional<float> GetFlattenValue() const;
	TOptional<float> GetFlattenTargetValueMin() const;
	float GetFlattenTargetValueMid() const;
	TOptional<float> GetFlattenTargetValueMax() const;
	TOptional<float> GetFlattenTerraceIntervalValueMax() const;

	// Common Error handling
	EVisibility GetMiscLandscapeErrorVisibility() const;
	FText GetMiscLandscapeErrorText() const;

	// Current Gizmo customization (Import, Copy tools)
	static bool IsResetGizmoButtonShown(TSharedPtr<IPropertyHandle> InProperty = nullptr);
	static void ResetGizmoToOrigin(TSharedPtr<IPropertyHandle> InProperty = nullptr);
};
