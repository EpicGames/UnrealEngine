// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorDetailCustomization_ImportExport.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Dialog/SCustomDialog.h"
#include "SWarningOrErrorBox.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "Landscape.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeRegionUtils.h"
#include "LandscapeEditorPrivate.h"
#include "LandscapeEditorUtils.h"

#include "LandscapeConfigHelper.h"
#include "LandscapeImportHelper.h"
#include "LandscapeSettings.h"
#include "FileHelpers.h"

#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"

#include "SLandscapeEditor.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "Widgets/Input/SRotatorInputBox.h"
#include "ScopedTransaction.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "TutorialMetaData.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "LandscapeDataAccess.h"
#include "Settings/EditorExperimentalSettings.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "LandscapeSubsystem.h"
#include "LocationVolume.h"
#include "SPrimaryButton.h"
#include "Widgets/Input/SSegmentedControl.h"

#include "UObject/SavePackage.h"
#include "Builders/CubeBuilder.h"
#include "ActorFactories/ActorFactory.h"

#include "SourceControlHelpers.h"

#include "Misc/ScopedSlowTask.h"
#include "EditorModeManager.h"
#include "EditorModes.h"

#include "Math/Box.h"
#include "LandscapeEditorUtils.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.NewLandscape"


void FLandscapeEditorArrayBuilder_NewTargetLayer::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	const TSharedPtr<IPropertyHandle> PropertyHandle = GetPropertyHandle();
	check(PropertyHandle);

	NodeRow
		.FilterString(PropertyHandle->GetPropertyDisplayName())
		.NameContent()
		[
			PropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				[
					SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "SimpleButton")
						.ToolTipText(LOCTEXT("NewLandscape_AutoFillTargetLayers_Tip", "Auto-Fill Target Layer Assets"))
						.OnClicked_Static(&FLandscapeEditorArrayBuilder_NewTargetLayer::ShowAutoFillTargetLayerDialog)
						.IsEnabled_Static(&FLandscapeEditorArrayBuilder_NewTargetLayer::HasWeightmapTargetLayers)
						[
							SNew(SImage)
								.Image(FAppStyle::GetBrush("LandscapeEditor.Layer.AutoFill"))
						]
				]
		];
}

FReply FLandscapeEditorArrayBuilder_NewTargetLayer::HandleAutoFillTargetLayers(const bool bCreateAssetForEmptyLayers)
{
	if (const FEdModeLandscape* LandscapeEdMode = static_cast<FEdModeLandscape*>(GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape)))
	{
		FScopedTransaction Transaction(LOCTEXT("NewLandscape_AutoFillTargetLayers", "Auto-Fill Target Layer Assets"));

		LandscapeEdMode->UISettings->AutoFillTargetLayers(LandscapeEdMode->GetTargetLayerAssetPackagePath(), bCreateAssetForEmptyLayers);
	}

	return FReply::Handled();
}

FReply FLandscapeEditorArrayBuilder_NewTargetLayer::ShowAutoFillTargetLayerDialog()
{
	if (const FEdModeLandscape* LandscapeEdMode = static_cast<FEdModeLandscape*>(GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape)))
	{
		const FString& DefaultTargetLayerAssetPath = LandscapeEdMode->GetTargetLayerAssetPackagePath();
		
		bool bIsCreateAssetForEmptyLayersChecked = false;

		const TSharedPtr<SCustomDialog> DialogWindow = SNew(SCustomDialog)
			.Title(LOCTEXT("TargetLayerAutoFillLayers", "Auto-Fill Target Layer Assets"))
			.HAlignButtonBox(HAlign_Center)
			.Buttons({
				SCustomDialog::FButton(
					LOCTEXT("OK", "OK"),
					/*InOnClicked = */FSimpleDelegate::CreateLambda([&bIsCreateAssetForEmptyLayersChecked]()
						{
							HandleAutoFillTargetLayers(bIsCreateAssetForEmptyLayersChecked);
						}),
					SCustomDialog::EButtonRole::Confirm)
					.SetPrimary(true),
				SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"), /*InOnClicked = */FSimpleDelegate(), SCustomDialog::EButtonRole::Cancel)
				})
			.ContentAreaPadding(10.0f)
			.Content()
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(STextBlock)
							.WrapTextAt(512.0f)
							.Text(FText::Format(LOCTEXT("TargetLayerAutoFillDialog", "This operation will assign the layer info assets found within the default asset folder {0} to the landscape's target layers."),
								FText::FromString(LandscapeEdMode->GetTargetLayerAssetPackagePath())))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 10.0f, 0, 0)
					[
						SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SCheckBox)
									.IsChecked_Lambda([&bIsCreateAssetForEmptyLayersChecked]() { return bIsCreateAssetForEmptyLayersChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
									.OnCheckStateChanged_Lambda([&bIsCreateAssetForEmptyLayersChecked](ECheckBoxState InState) { bIsCreateAssetForEmptyLayersChecked = (InState == ECheckBoxState::Checked); })
							]
							+ SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(STextBlock)
									.Text(LOCTEXT("TargetLayerAutoFillDialog_CreateNew", "Create new assets in the default folder if none are found."))
							]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0, 20.0f, 0, 0) // Create spacing between details and prompt
					[
						SNew(STextBlock)
							.Text(LOCTEXT("TargetLayerAutoFillDialog_Prompt", "Do you want to auto-fill all target layers?"))
					]
			];

		DialogWindow->ShowModal();
	}
	
	return FReply::Handled();
}

bool FLandscapeEditorArrayBuilder_NewTargetLayer::HasWeightmapTargetLayers()
{
	if (const FEdModeLandscape* LandscapeEdMode = static_cast<FEdModeLandscape*>(GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape)))
	{
		return !LandscapeEdMode->UISettings->NewLandscape_Layers.IsEmpty();
	}

	return false;
}

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_NewLandscape::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_NewLandscape);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_NewLandscape::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!IsToolActive("NewLandscape"))
	{
		return;
	}

	IDetailCategoryBuilder& NewLandscapeCategory = DetailBuilder.EditCategory("New Landscape");

	NewLandscapeCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SBox)
		.Padding(2.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SSegmentedControl<ENewLandscapePreviewMode>)
			.Value_Lambda([this]()
			{
				FEdModeLandscape* LandscapeEdMode = GetEditorMode();
				return LandscapeEdMode ? LandscapeEdMode->NewLandscapePreviewMode : ENewLandscapePreviewMode::NewLandscape;

			})
			.OnValueChanged_Lambda([this](ENewLandscapePreviewMode Mode)
			{
				FEdModeLandscape* LandscapeEdMode = GetEditorMode();
				if (LandscapeEdMode != nullptr)
				{
					LandscapeEdMode->NewLandscapePreviewMode = Mode;
				}
			})
			+SSegmentedControl<ENewLandscapePreviewMode>::Slot(ENewLandscapePreviewMode::NewLandscape)
			.Text(LOCTEXT("NewLandscape", "Create New"))
			+ SSegmentedControl<ENewLandscapePreviewMode>::Slot(ENewLandscapePreviewMode::ImportLandscape)
			.Text(LOCTEXT("ImportLandscape", "Import from File"))
		]
		
	];

	TSharedRef<IPropertyHandle> PropertyHandle_FlipYAxis = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bFlipYAxis));
	NewLandscapeCategory.AddProperty(PropertyHandle_FlipYAxis).Visibility(MakeAttributeLambda([]()
	{ 
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (LandscapeEdMode != nullptr)
		{
			if ((LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape) && !LandscapeEdMode->UseSingleFileImport())
			{
				return EVisibility::Visible;
			}
		}

		return EVisibility::Collapsed; 
	}));
	
	PropertyHandle_FlipYAxis->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this]() { OnImportHeightmapFilenameChanged(); }));

	TSharedRef<IPropertyHandle> PropertyHandle_GridSize = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WorldPartitionGridSize));
	TSharedRef<IPropertyHandle> PropertyHandle_RegionSize = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WorldPartitionRegionSize));
	
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode->GetWorld()->GetSubsystem<ULandscapeSubsystem>()->IsGridBased())
	{
		NewLandscapeCategory.AddProperty(PropertyHandle_GridSize);
		NewLandscapeCategory.AddProperty(PropertyHandle_RegionSize);
	}
	else
	{
		DetailBuilder.HideProperty(PropertyHandle_GridSize);
		DetailBuilder.HideProperty(PropertyHandle_RegionSize);
	}
	
	TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapFilename));
	TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapImportResult));
	TSharedRef<IPropertyHandle> PropertyHandle_HeightmapErrorMessage = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapErrorMessage));
	DetailBuilder.HideProperty(PropertyHandle_HeightmapImportResult);
	DetailBuilder.HideProperty(PropertyHandle_HeightmapErrorMessage);
	PropertyHandle_HeightmapFilename->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([this, PropertyHandle_HeightmapFilename]()
	{
		FLandscapeEditorDetailCustomization_ImportExport::FormatFilename(PropertyHandle_HeightmapFilename, /*bForExport = */false);
		OnImportHeightmapFilenameChanged();
	}));
	NewLandscapeCategory.AddProperty(PropertyHandle_HeightmapFilename)
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::ImportLandscape)))
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_HeightmapFilename->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(0,0,2,0)
		[
			SNew(SErrorText)
			.Visibility_Static(&GetHeightmapErrorVisibility, PropertyHandle_HeightmapImportResult)
			.BackgroundColor_Static(&GetHeightmapErrorColor, PropertyHandle_HeightmapImportResult)
			.ErrorText(NSLOCTEXT("UnrealEd", "Error", "!"))
			.ToolTip(
				SNew(SToolTip)
				.Text_Static(&GetPropertyValue<FText>, PropertyHandle_HeightmapErrorMessage)
			)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SBox)
			// This path is edited via a SEditableTextBox (rather than using CreatePropertyValueWidget) so we have to specify a tooltip in order to display the full path if 
			//  the text box is to small to display the entire thing
			.ToolTipText_Static(&GetPropertyValueText, PropertyHandle_HeightmapFilename)
			[
				SNew(SEditableTextBox)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetPropertyValueText, PropertyHandle_HeightmapFilename)
				.OnTextCommitted_Static(&SetImportHeightmapFilenameString, PropertyHandle_HeightmapFilename)
				.HintText(LOCTEXT("Import_HeightmapNotSet", "(Please specify a heightmap)"))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1,0,0,0)
		[
			SNew(SButton)
			//.Font(DetailBuilder.GetDetailFont())
			.ContentPadding(FMargin(4, 0))
			.Text(NSLOCTEXT("UnrealEd", "GenericOpenDialog", "..."))
			.OnClicked_Static(&OnImportHeightmapFilenameButtonClicked, PropertyHandle_HeightmapFilename)
		]
	];

	NewLandscapeCategory.AddCustomRow(LOCTEXT("HeightmapResolution", "Heightmap Resolution"))
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::ImportLandscape)))
	.NameContent()
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(LOCTEXT("HeightmapResolution", "Heightmap Resolution"))
			.ToolTipText(LOCTEXT("ImportResolutionTooltip", "Possible resolutions for the selected image"))
		]
	]
	.ValueContent()
	[
		SNew(SBox)
		.Padding(FMargin(0,0,12,0)) // Line up with the other properties due to having no reset to default button
		[
			SNew(SComboButton)
			.OnGetMenuContent(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetImportLandscapeResolutionMenu)
			.ContentPadding(2.0f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetImportLandscapeResolution)
			]
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_Material = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Material));
	IDetailPropertyRow& MaterialPropertyRow = NewLandscapeCategory.AddProperty(PropertyHandle_Material);


	FIsResetToDefaultVisible IsResetVisible = FIsResetToDefaultVisible::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetMaterialToDefault);
	FResetToDefaultHandler ResetHandler = FResetToDefaultHandler::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ResetMaterialToDefault);
	FResetToDefaultOverride ResetOverride = FResetToDefaultOverride::Create(IsResetVisible, ResetHandler);

	MaterialPropertyRow.OverrideResetToDefault(ResetOverride);

	TSharedRef<IPropertyHandle> PropertyHandle_AlphamapType = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_AlphamapType));
	NewLandscapeCategory.AddProperty(PropertyHandle_AlphamapType)
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::ImportLandscape)));

	TSharedRef<IPropertyHandle> PropertyHandle_TargetLayerAssetFilePath = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TargetLayerAssetFilePath));
	NewLandscapeCategory.AddProperty(PropertyHandle_TargetLayerAssetFilePath);

	TSharedRef<IPropertyHandle> PropertyHandle_Layers = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Layers));
	TSharedPtr<FLandscapeEditorArrayBuilder_NewTargetLayer> ArrayBuilder = MakeShared<FLandscapeEditorArrayBuilder_NewTargetLayer>(PropertyHandle_Layers);
	ArrayBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::GenerateLayersArrayElementWidget));

	NewLandscapeCategory.AddCustomBuilder(ArrayBuilder.ToSharedRef());

	NewLandscapeCategory.AddCustomRow(LOCTEXT("LayersLabel", "Layers"))
		.Visibility(TAttribute<EVisibility>(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetMaterialTipVisibility))
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(15, 12, 0, 12)
				[
					SNew(STextBlock)
						.Font(DetailBuilder.GetDetailFont())
						.Text(LOCTEXT("Material_Tip", "Hint: Assign a material to see the landscape layers it handles. Use Auto-Fill button to populate layer assets."))
				]
		];

	TSharedRef<IPropertyHandle> PropertyHandle_Location = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Location));
	TSharedRef<IPropertyHandle> PropertyHandle_Location_X = PropertyHandle_Location->GetChildHandle("X").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Location_Y = PropertyHandle_Location->GetChildHandle("Y").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Location_Z = PropertyHandle_Location->GetChildHandle("Z").ToSharedRef();
	
	IDetailPropertyRow& LocationPropertyRow = NewLandscapeCategory.AddProperty(PropertyHandle_Location);

	FIsResetToDefaultVisible IsResetLocationVisible = FIsResetToDefaultVisible::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetLocationlToDefault);
	FResetToDefaultHandler ResetLocationHandler = FResetToDefaultHandler::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ResetLocationToDefault);
	FResetToDefaultOverride ResetLocationOverride = FResetToDefaultOverride::Create(IsResetLocationVisible, ResetLocationHandler);

	LocationPropertyRow.OverrideResetToDefault(ResetLocationOverride);

	LocationPropertyRow
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_Location->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.0f * 3.0f) // copied from FComponentTransformDetails
	.MaxDesiredWidth(125.0f * 3.0f)
	[
		SNew(SNumericVectorInputBox<FVector::FReal>)
		.bColorAxisLabels(true)
		.Font(DetailBuilder.GetDetailFont())
		.X_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Location_X)
		.Y_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Location_Y)
		.Z_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Location_Z)
		.OnXCommitted_Static(&SetPropertyValue<FVector::FReal>, PropertyHandle_Location_X)
		.OnYCommitted_Static(&SetPropertyValue<FVector::FReal>, PropertyHandle_Location_Y)
		.OnZCommitted_Static(&SetPropertyValue<FVector::FReal>, PropertyHandle_Location_Z)
		.OnXChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Location_X->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.OnYChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Location_Y->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.OnZChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Location_Z->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.AllowSpin(true)
	];

	TSharedRef<IPropertyHandle> PropertyHandle_Rotation = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Rotation));
	TSharedRef<IPropertyHandle> PropertyHandle_Rotation_Roll  = PropertyHandle_Rotation->GetChildHandle("Roll").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Rotation_Pitch = PropertyHandle_Rotation->GetChildHandle("Pitch").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Rotation_Yaw   = PropertyHandle_Rotation->GetChildHandle("Yaw").ToSharedRef();

	IDetailPropertyRow& RotationPropertyRow = NewLandscapeCategory.AddProperty(PropertyHandle_Rotation);

	FIsResetToDefaultVisible IsResetRotationVisible = FIsResetToDefaultVisible::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetRotationToDefault);
	FResetToDefaultHandler ResetRotationHandler = FResetToDefaultHandler::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ResetRotationToDefault);
	FResetToDefaultOverride ResetRotationOverride = FResetToDefaultOverride::Create(IsResetRotationVisible, ResetRotationHandler);

	RotationPropertyRow.OverrideResetToDefault(ResetRotationOverride);

	RotationPropertyRow
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_Rotation->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.0f * 3.0f) // copied from FComponentTransformDetails
	.MaxDesiredWidth(125.0f * 3.0f)
	[
		SNew(SNumericRotatorInputBox<FRotator::FReal>)
		.bColorAxisLabels(true)
		.Font(DetailBuilder.GetDetailFont())
		.Roll_Static(&GetOptionalPropertyValue<FRotator::FReal>, PropertyHandle_Rotation_Roll)
		.Pitch_Static(&GetOptionalPropertyValue<FRotator::FReal>, PropertyHandle_Rotation_Pitch)
		.Yaw_Static(&GetOptionalPropertyValue<FRotator::FReal>, PropertyHandle_Rotation_Yaw)
		.OnYawCommitted_Static(&SetPropertyValue<FRotator::FReal>, PropertyHandle_Rotation_Yaw) // not allowed to roll or pitch landscape
		.OnYawChanged_Lambda([=](FRotator::FReal NewValue){ ensure(PropertyHandle_Rotation_Yaw->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.AllowSpin(true)
	];

	TSharedRef<IPropertyHandle> PropertyHandle_Scale = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Scale));
	TSharedRef<IPropertyHandle> PropertyHandle_Scale_X = PropertyHandle_Scale->GetChildHandle("X").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Scale_Y = PropertyHandle_Scale->GetChildHandle("Y").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_Scale_Z = PropertyHandle_Scale->GetChildHandle("Z").ToSharedRef();
	
	IDetailPropertyRow& ScalePropertyRow = NewLandscapeCategory.AddProperty(PropertyHandle_Scale);

	FIsResetToDefaultVisible IsResetScaleVisible = FIsResetToDefaultVisible::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetScaleToDefault);
	FResetToDefaultHandler ResetScaleHandler = FResetToDefaultHandler::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::ResetScaleToDefault);
	FResetToDefaultOverride ResetScaleOverride = FResetToDefaultOverride::Create(IsResetScaleVisible, ResetScaleHandler);

	ScalePropertyRow.OverrideResetToDefault(ResetScaleOverride);

	ScalePropertyRow
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_Scale->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(125.0f * 3.0f) // copied from FComponentTransformDetails
	.MaxDesiredWidth(125.0f * 3.0f)
	[
		SNew(SNumericVectorInputBox<FVector::FReal>)
		.bColorAxisLabels(true)
		.Font(DetailBuilder.GetDetailFont())
		.X_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Scale_X)
		.Y_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Scale_Y)
		.Z_Static(&GetOptionalPropertyValue<FVector::FReal>, PropertyHandle_Scale_Z)
		.OnXCommitted_Static(&SetScale, PropertyHandle_Scale_X)
		.OnYCommitted_Static(&SetScale, PropertyHandle_Scale_Y)
		.OnZCommitted_Static(&SetScale, PropertyHandle_Scale_Z)
		.OnXChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Scale_X->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.OnYChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Scale_Y->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.OnZChanged_Lambda([=](FVector::FReal NewValue) { ensure(PropertyHandle_Scale_Z->SetValue(NewValue, EPropertyValueSetFlags::InteractiveChange) == FPropertyAccess::Success); })
		.AllowSpin(true)
	];

	TSharedRef<IPropertyHandle> PropertyHandle_QuadsPerSection = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_QuadsPerSection));
	NewLandscapeCategory.AddProperty(PropertyHandle_QuadsPerSection)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_QuadsPerSection->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SComboButton)
		.OnGetMenuContent_Static(&GetSectionSizeMenu, PropertyHandle_QuadsPerSection)
		.ContentPadding(2.0f)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetSectionSize, PropertyHandle_QuadsPerSection)
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_SectionsPerComponent = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_SectionsPerComponent));
	NewLandscapeCategory.AddProperty(PropertyHandle_SectionsPerComponent)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_SectionsPerComponent->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SComboButton)
		.OnGetMenuContent_Static(&GetSectionsPerComponentMenu, PropertyHandle_SectionsPerComponent)
		.ContentPadding(2.0f)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetSectionsPerComponent, PropertyHandle_SectionsPerComponent)
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_ComponentCount));
	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_X = PropertyHandle_ComponentCount->GetChildHandle("X").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_Y = PropertyHandle_ComponentCount->GetChildHandle("Y").ToSharedRef();
	NewLandscapeCategory.AddProperty(PropertyHandle_ComponentCount)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_ComponentCount->CreatePropertyNameWidget()
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue_Lambda(LandscapeEditorUtils::GetMaxSizeInComponents)
			.MinSliderValue(1)
			.MaxSliderValue_Lambda(LandscapeEditorUtils::GetMaxSizeInComponents)
			.AllowSpin(true)
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.Value_Static(&FLandscapeEditorDetailCustomization_Base::OnGetValue<int32>, PropertyHandle_ComponentCount_X)
			.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<int32>, PropertyHandle_ComponentCount_X)
			.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<int32>, PropertyHandle_ComponentCount_X)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(FText::FromString(FString().AppendChar(0xD7))) // Multiply sign
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue_Lambda(LandscapeEditorUtils::GetMaxSizeInComponents)
			.MinSliderValue(1)
			.MaxSliderValue_Lambda(LandscapeEditorUtils::GetMaxSizeInComponents)
			.AllowSpin(true)
			.UndeterminedString(NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values"))
			.Value_Static(&FLandscapeEditorDetailCustomization_Base::OnGetValue<int32>, PropertyHandle_ComponentCount_Y)
			.OnValueChanged_Static(&FLandscapeEditorDetailCustomization_Base::OnValueChanged<int32>, PropertyHandle_ComponentCount_Y)
			.OnValueCommitted_Static(&FLandscapeEditorDetailCustomization_Base::OnValueCommitted<int32>, PropertyHandle_ComponentCount_Y)
		]
	];

	NewLandscapeCategory.AddCustomRow(LOCTEXT("Resolution", "Overall Resolution"))
	.RowTag("LandscapeEditor.OverallResolution")
	.NameContent()
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(LOCTEXT("Resolution", "Overall Resolution"))
			.ToolTipText(TAttribute<FText>(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetOverallResolutionTooltip))
		]
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue(8192)
			.MinSliderValue(1)
			.MaxSliderValue(8192)
			.AllowSpin(true)
			//.MinSliderValue(TAttribute<TOptional<int32> >(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetMinLandscapeResolution))
			//.MaxSliderValue(TAttribute<TOptional<int32> >(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetMaxLandscapeResolution))
			.Value(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetLandscapeResolutionX)
			.OnValueChanged_Lambda([this](int32 NewValue)
			{
				OnChangeLandscapeResolutionX(NewValue, false);
			})
			.OnValueCommitted_Lambda([this](int32 NewValue, ETextCommit::Type)
			{
				OnChangeLandscapeResolutionX(NewValue, true);
			})
			.OnBeginSliderMovement_Lambda([this]()
			{
				bUsingSlider = true;
				GEditor->BeginTransaction(LOCTEXT("ChangeResolutionX_Transaction", "Change Landscape Resolution X"));
			})
			.OnEndSliderMovement_Lambda([this](double)
			{
				GEditor->EndTransaction();
				bUsingSlider = false;
			})
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2, 0)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(FText::FromString(FString().AppendChar(0xD7))) // Multiply sign
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		[
			SNew(SNumericEntryBox<int32>)
			.Font(DetailBuilder.GetDetailFont())
			.MinValue(1)
			.MaxValue(8192)
			.MinSliderValue(1)
			.MaxSliderValue(8192)
			.AllowSpin(true)
			//.MinSliderValue(TAttribute<TOptional<int32> >(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetMinLandscapeResolution))
			//.MaxSliderValue(TAttribute<TOptional<int32> >(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetMaxLandscapeResolution))
			.Value(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetLandscapeResolutionY)
			.OnValueChanged_Lambda([this](int32 NewValue)
			{
				OnChangeLandscapeResolutionY(NewValue, false);
			})
			.OnValueCommitted_Lambda([this](int32 NewValue, ETextCommit::Type)
			{
				OnChangeLandscapeResolutionY(NewValue, true);
			})
			.OnBeginSliderMovement_Lambda([this]()
			{
				bUsingSlider = true;
				GEditor->BeginTransaction(LOCTEXT("ChangeResolutionY_Transaction", "Change Landscape Resolution Y"));
			})
			.OnEndSliderMovement_Lambda([this](double)
			{
				GEditor->EndTransaction();
				bUsingSlider = false;
			})
		]
	];

	NewLandscapeCategory.AddCustomRow(LOCTEXT("TotalComponents", "Total Components"))
	.RowTag("LandscapeEditor.TotalComponents")
	.NameContent()
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(LOCTEXT("TotalComponents", "Total Components"))
			.ToolTipText(LOCTEXT("NewLandscape_TotalComponents", "The total number of components that will be created for this landscape."))
		]
	]
	.ValueContent()
	.VAlign(VAlign_Center)
	[
		SNew(SEditableTextBox)
		.IsReadOnly(true)
		.Font(DetailBuilder.GetDetailFont())
		.Text(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetTotalComponentCount)
	];

	NewLandscapeCategory.AddCustomRow(FText::GetEmpty()).WholeRowContent()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Right)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(4,0)
		.AutoWidth()
		[
			SNew(SButton)
			.Visibility_Static(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::NewLandscape)
			.Text(LOCTEXT("FillWorld", "Fill World"))
			.ToolTipText(LOCTEXT("FillWorldTooltip", "Automatically set the component count such that the landscape fills the entire world (up to the maximum supported landscape resolution)"))
			.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("FillWorldButton"), TEXT("LevelEditorToolBox")))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_NewLandscape::OnFillWorldButtonClicked)
		]
		+ SHorizontalBox::Slot()
		.Padding(4, 0)
		.AutoWidth()
		[
			SNew(SButton)
			.Visibility_Static(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::ImportLandscape)
			.Text(LOCTEXT("FitToData", "Fit To Data"))
			.ToolTipText(LOCTEXT("FitToDataTooltip", "Try to find a section size and number of sections that exactly matches the dimensions of the import data"))
			.AddMetaData<FTagMetaData>(TEXT("ImportButton"))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_NewLandscape::OnFitImportDataButtonClicked)
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetImportButtonIsEnabled)
		]
		+ SHorizontalBox::Slot()
		.Padding(4, 0)
		.AutoWidth()
		[
			SNew(SPrimaryButton)
			.Visibility_Static(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::NewLandscape)
			.Text(LOCTEXT("Create", "Create"))
			.ToolTipText(LOCTEXT("CreateTooltip", "Create the landscape based on the currently selected parameters"))
			.AddMetaData<FTutorialMetaData>(FTutorialMetaData(TEXT("CreateButton"), TEXT("LevelEditorToolBox")))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_NewLandscape::OnCreateButtonClicked)
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_NewLandscape::IsCreateButtonEnabled)
		]
		+ SHorizontalBox::Slot()
		.Padding(4, 0)
		.AutoWidth()
		[
			SNew(SButton)
			.Visibility_Static(&GetVisibilityOnlyInNewLandscapeMode, ENewLandscapePreviewMode::ImportLandscape)
			.Text(LOCTEXT("Import", "Import"))
			.ToolTipText(LOCTEXT("ImportTooltip", "Create the landscape based on the currently selected parameters and applies the import data onto it"))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_NewLandscape::OnCreateButtonClicked)
			.IsEnabled(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetImportButtonIsEnabled)
		]
	];

	NewLandscapeCategory.AddCustomRow(FText::GetEmpty())
	.WholeRowContent()
	[
		SNew(SWarningOrErrorBox)
		.Message(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetNewLandscapeErrorText)
	]
	.Visibility(TAttribute<EVisibility>(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetNewLandscapeErrorVisibility));
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText FLandscapeEditorDetailCustomization_NewLandscape::GetOverallResolutionTooltip() const
{
	return (GetEditorMode() && GetEditorMode()->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
	? LOCTEXT("ImportLandscape_OverallResolution", "Overall final resolution of the imported landscape in vertices")
	: LOCTEXT("NewLandscape_OverallResolution", "Overall final resolution of the new landscape in vertices");
}

void FLandscapeEditorDetailCustomization_NewLandscape::SetScale(FVector::FReal NewValue, ETextCommit::Type, TSharedRef<IPropertyHandle> PropertyHandle)
{
	FScopedTransaction Transaction(LOCTEXT("ChangeScale_Transaction", "Change Landscape Scale"));
	FVector::FReal OldValue = 0;
	PropertyHandle->GetValue(OldValue);

	if (NewValue == 0)
	{
		if (OldValue < 0)
		{
			NewValue = -1;
		}
		else
		{
			NewValue = 1;
		}
	}

	ensure(PropertyHandle->SetValue(NewValue) == FPropertyAccess::Success);

	// Make X and Y scale match
	FName PropertyName = PropertyHandle->GetProperty()->GetFName();
	if (PropertyName == "X")
	{
		TSharedRef<IPropertyHandle> PropertyHandle_Y = PropertyHandle->GetParentHandle()->GetChildHandle("Y").ToSharedRef();
		ensure(PropertyHandle_Y->SetValue(NewValue) == FPropertyAccess::Success);
	}
	else if (PropertyName == "Y")
	{
		TSharedRef<IPropertyHandle> PropertyHandle_X = PropertyHandle->GetParentHandle()->GetChildHandle("X").ToSharedRef();
		ensure(PropertyHandle_X->SetValue(NewValue) == FPropertyAccess::Success);
	}
}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_NewLandscape::GetSectionSizeMenu(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < UE_ARRAY_COUNT(FLandscapeConfig::SubsectionSizeQuadsValues); i++)
	{
		MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("NxNQuads", "{0}\u00D7{0} Quads"), FText::AsNumber(FLandscapeConfig::SubsectionSizeQuadsValues[i])), FText::GetEmpty(),
			FSlateIcon(), FExecuteAction::CreateStatic(&OnChangeSectionSize, PropertyHandle, FLandscapeConfig::SubsectionSizeQuadsValues[i]));
	}

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnChangeSectionSize(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize)
{
	ensure(PropertyHandle->SetValue(NewSize) == FPropertyAccess::Success);
}

FText FLandscapeEditorDetailCustomization_NewLandscape::GetSectionSize(TSharedRef<IPropertyHandle> PropertyHandle)
{
	int32 QuadsPerSection = 0;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(QuadsPerSection);
	check(Result != FPropertyAccess::Fail);

	if (Result == FPropertyAccess::MultipleValues)
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return FText::Format(LOCTEXT("NxNQuads", "{0}\u00D7{0} Quads"), FText::AsNumber(QuadsPerSection));
}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_NewLandscape::GetSectionsPerComponentMenu(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < UE_ARRAY_COUNT(FLandscapeConfig::NumSectionValues); i++)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Width"), FLandscapeConfig::NumSectionValues[i]);
		Args.Add(TEXT("Height"), FLandscapeConfig::NumSectionValues[i]);
		MenuBuilder.AddMenuEntry(FText::Format(FLandscapeConfig::NumSectionValues[i] == 1 ? LOCTEXT("1x1Section", "{Width}\u00D7{Height} Section") : LOCTEXT("NxNSections", "{Width}\u00D7{Height} Sections"), Args),
			FText::GetEmpty(), FSlateIcon(), FExecuteAction::CreateStatic(&OnChangeSectionsPerComponent, PropertyHandle, FLandscapeConfig::NumSectionValues[i]));
	}

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnChangeSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize)
{
	ensure(PropertyHandle->SetValue(NewSize) == FPropertyAccess::Success);
}

FText FLandscapeEditorDetailCustomization_NewLandscape::GetSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle)
{
	int32 SectionsPerComponent = 0;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(SectionsPerComponent);
	check(Result != FPropertyAccess::Fail);

	if (Result == FPropertyAccess::MultipleValues)
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("Width"), SectionsPerComponent);
	Args.Add(TEXT("Height"), SectionsPerComponent);
	return FText::Format(SectionsPerComponent == 1 ? LOCTEXT("1x1Section", "{Width}\u00D7{Height} Section") : LOCTEXT("NxNSections", "{Width}\u00D7{Height} Sections"), Args);
}

TOptional<int32> FLandscapeEditorDetailCustomization_NewLandscape::GetLandscapeResolutionX() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->GetNewLandscapeResolutionX();
	}

	return 0;
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnChangeLandscapeResolutionX(int32 NewValue, bool bCommit)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		int32 NewComponentCountX = LandscapeEdMode->UISettings->CalcComponentsCount(NewValue);
		if (NewComponentCountX == LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X)
		{
return;
		}

		FScopedTransaction Transaction(LOCTEXT("ChangeResolutionX_Transaction", "Change Landscape Resolution X"), !bUsingSlider && bCommit);

		LandscapeEdMode->UISettings->Modify();
		LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X = NewComponentCountX;
	}
}

TOptional<int32> FLandscapeEditorDetailCustomization_NewLandscape::GetLandscapeResolutionY() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->GetNewLandscapeResolutionY();
	}

	return 0;
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnChangeLandscapeResolutionY(int32 NewValue, bool bCommit)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		int32 NewComponentCountY = LandscapeEdMode->UISettings->CalcComponentsCount(NewValue);
		if (NewComponentCountY == LandscapeEdMode->UISettings->NewLandscape_ComponentCount.Y)
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("ChangeResolutionY_Transaction", "Change Landscape Resolution Y"), !bUsingSlider && bCommit);

		LandscapeEdMode->UISettings->Modify();
		LandscapeEdMode->UISettings->NewLandscape_ComponentCount.Y = NewComponentCountY;
	}
}

TOptional<int32> FLandscapeEditorDetailCustomization_NewLandscape::GetMinLandscapeResolution() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		// Min size is one component
		return (LandscapeEdMode->UISettings->NewLandscape_SectionsPerComponent * LandscapeEdMode->UISettings->NewLandscape_QuadsPerSection + 1);
	}

	return 0;
}

TOptional<int32> FLandscapeEditorDetailCustomization_NewLandscape::GetMaxLandscapeResolution() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		// Max size is either whole components below 8192 verts, or 32 components
		const int32 QuadsPerComponent = (LandscapeEdMode->UISettings->NewLandscape_SectionsPerComponent * LandscapeEdMode->UISettings->NewLandscape_QuadsPerSection);
		//return (float)(FMath::Min(32, FMath::FloorToInt(8191 / QuadsPerComponent)) * QuadsPerComponent);
		return (8191 / QuadsPerComponent) * QuadsPerComponent + 1;
	}

	return 0;
}

FText FLandscapeEditorDetailCustomization_NewLandscape::GetTotalComponentCount() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return FText::AsNumber(LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X * LandscapeEdMode->UISettings->NewLandscape_ComponentCount.Y);
	}

	return FText::FromString(TEXT("---"));
}


EVisibility FLandscapeEditorDetailCustomization_NewLandscape::GetVisibilityOnlyInNewLandscapeMode(ENewLandscapePreviewMode value)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		if (LandscapeEdMode->NewLandscapePreviewMode == value)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void FLandscapeEditorDetailCustomization_NewLandscape::GenerateLayersArrayElementWidget(TSharedRef<IPropertyHandle> InPropertyHandle, int32 InArrayIndex, IDetailChildrenBuilder& InChildrenBuilder)
{
	InChildrenBuilder.AddProperty(InPropertyHandle)
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::GetLayerVisibility, InPropertyHandle)));
}

EVisibility FLandscapeEditorDetailCustomization_NewLandscape::GetLayerVisibility(TSharedRef<IPropertyHandle> InPropertyHandle) const
{
	const FEdModeLandscape* EdMode = GetEditorMode();
	TArray<void*> RawData;
	InPropertyHandle->AccessRawData(RawData);

	if ((EdMode != nullptr) && !RawData.IsEmpty() && (RawData[0] != nullptr))
	{
		FLandscapeImportLayer* ImportLayer = reinterpret_cast<FLandscapeImportLayer*>(RawData[0]);

		if ((EdMode->NewLandscapePreviewMode != ENewLandscapePreviewMode::ImportLandscape) && (ImportLayer->LayerName == ALandscapeProxy::VisibilityLayer->GetLayerName()))
		{
			return EVisibility::Hidden;
		}
	}

	return EVisibility::Visible;
}

bool FLandscapeEditorDetailCustomization_NewLandscape::IsCreateButtonEnabled() const
{
	const FEdModeLandscape* EdMode = GetEditorMode();
	
	if (EdMode != nullptr)
	{
		return EdMode->IsLandscapeResolutionCompliant();
	}

	return true;
}

EVisibility FLandscapeEditorDetailCustomization_NewLandscape::GetNewLandscapeErrorVisibility() const
{
	return IsCreateButtonEnabled() ? EVisibility::Hidden : EVisibility::Visible;
}

FText FLandscapeEditorDetailCustomization_NewLandscape::GetNewLandscapeErrorText() const
{
	const FEdModeLandscape* EdMode = GetEditorMode();

	if (EdMode != nullptr)
	{
		return EdMode->GetLandscapeResolutionErrorText();
	}

	return FText::GetEmpty();
}

void FLandscapeEditorDetailCustomization_NewLandscape::AddComponents(ULandscapeInfo* InLandscapeInfo, ULandscapeSubsystem* InLandscapeSubsystem, const TArray<FIntPoint>& InComponentCoordinates, TArray<ALandscapeProxy*>& OutCreatedStreamingProxies)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AddComponents);
	TArray<ULandscapeComponent*> NewComponents;
	InLandscapeInfo->Modify();
	for (const FIntPoint& ComponentCoordinate : InComponentCoordinates)
	{
		ULandscapeComponent* LandscapeComponent = InLandscapeInfo->XYtoComponentMap.FindRef(ComponentCoordinate);
		if (LandscapeComponent)
		{
			continue;
		}

		// Add New component...
		FIntPoint ComponentBase = ComponentCoordinate * InLandscapeInfo->ComponentSizeQuads;

		ALandscapeProxy* LandscapeProxy = InLandscapeSubsystem->FindOrAddLandscapeProxy(InLandscapeInfo, ComponentBase);
		if (!LandscapeProxy)
		{
			continue;
		}

		OutCreatedStreamingProxies.Add(LandscapeProxy);

		LandscapeComponent = NewObject<ULandscapeComponent>(LandscapeProxy, NAME_None, RF_Transactional);
		NewComponents.Add(LandscapeComponent);
		LandscapeComponent->Init(
			ComponentBase.X, ComponentBase.Y,
			LandscapeProxy->ComponentSizeQuads,
			LandscapeProxy->NumSubsections,
			LandscapeProxy->SubsectionSizeQuads
		);

		TArray<FColor> HeightData;
		const int32 ComponentVerts = (LandscapeComponent->SubsectionSizeQuads + 1) * LandscapeComponent->NumSubsections;
		const FColor DefaultHeightColor = LandscapeDataAccess::GetDefaultPackedHeightColor();
		HeightData.Init(DefaultHeightColor, FMath::Square(ComponentVerts));

		LandscapeComponent->InitHeightmapData(HeightData, true);
		LandscapeComponent->UpdateMaterialInstances();

		InLandscapeInfo->XYtoComponentMap.Add(ComponentCoordinate, LandscapeComponent);
		InLandscapeInfo->XYtoAddCollisionMap.Remove(ComponentCoordinate);
	}

	// Need to register to use general height data update
	for (int32 Idx = 0; Idx < NewComponents.Num(); Idx++)
	{
		NewComponents[Idx]->RegisterComponent();
	}

	ALandscape* Landscape = InLandscapeInfo->LandscapeActor.Get();

	for (ULandscapeComponent* NewComponent : NewComponents)
	{
		if (Landscape)
		{
			TArray<ULandscapeComponent*> ComponentsUsingHeightmap;
			ComponentsUsingHeightmap.Add(NewComponent);

			for (const ULandscapeEditLayerBase* EditLayer : Landscape->GetEditLayersConst())
			{
				// Since we do not share heightmap when adding new component, we will provided the required array, but they will only be used for 1 component
				TMap<UTexture2D*, UTexture2D*> CreatedHeightmapTextures;
				NewComponent->AddDefaultLayerData(EditLayer->GetGuid(), ComponentsUsingHeightmap, CreatedHeightmapTextures);
			}
		}

		// Update Collision
		NewComponent->UpdateCachedBounds();
		NewComponent->UpdateBounds();
		NewComponent->MarkRenderStateDirty();
	}

	if (Landscape)
	{
		GEngine->BroadcastOnActorMoved(Landscape);
	}
}

FReply FLandscapeEditorDetailCustomization_NewLandscape::OnCreateButtonClicked()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FLandscapeEditorDetailCustomization_NewLandscape::OnCreateButtonClicked);

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (!LandscapeEdMode)
	{
		return FReply::Handled();
	}

	const bool bIsNewLandscape = LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::NewLandscape;
	
	UWorld* World = LandscapeEdMode->GetWorld();
	
	const bool bCreateLandscape = LandscapeEdMode != nullptr && 
		World != nullptr &&
		World->GetCurrentLevel()->bIsVisible;
	
	if (!bCreateLandscape)
	{
		return FReply::Handled();
	}

	const bool bIsTempPackage = FPackageName::IsTempPackage(World->GetPackage()->GetName());
	ULandscapeEditorObject* UISettings = LandscapeEdMode->UISettings;
	
	const bool bIsWorldPartition = World->GetSubsystem<ULandscapeSubsystem>()->IsGridBased();
	const bool bLandscapeLargerThanRegion = static_cast<int32>(UISettings->WorldPartitionRegionSize) < UISettings->NewLandscape_ComponentCount.X || static_cast<int32>(UISettings->WorldPartitionRegionSize) < UISettings->NewLandscape_ComponentCount.Y;
	const bool bNeedsLandscapeRegions =  bIsWorldPartition && bLandscapeLargerThanRegion;

	// If we need to ensure the map is saved before proceeding to create a landscape with regions 
	if (bIsTempPackage && bNeedsLandscapeRegions)
	{
		FString NewMapPackageName;
		if (!FEditorFileUtils::SaveCurrentLevel())
		{
			UE_LOGF(LogLandscapeTools, Error, "Unable to save current level");
			return FReply::Handled();
		}
	}

	const int32 QuadsPerSection = UISettings->NewLandscape_QuadsPerSection;
	
	const FIntPoint TotalLandscapeComponentSize { UISettings->NewLandscape_ComponentCount.X, UISettings->NewLandscape_ComponentCount.Y };

	const int32 ComponentCountX = bNeedsLandscapeRegions ? FMath::Min(static_cast<int32>(UISettings->WorldPartitionRegionSize), TotalLandscapeComponentSize.X) : TotalLandscapeComponentSize.X;
	const int32 ComponentCountY = bNeedsLandscapeRegions ? FMath::Min(static_cast<int32>(UISettings->WorldPartitionRegionSize), TotalLandscapeComponentSize.Y) : TotalLandscapeComponentSize.Y;
	const int32 QuadsPerComponent = UISettings->NewLandscape_SectionsPerComponent * QuadsPerSection;
	const int32 SizeX = ComponentCountX * QuadsPerComponent + 1;
	const int32 SizeY = ComponentCountY * QuadsPerComponent + 1;

	TArray<FLandscapeImportLayerInfo> MaterialImportLayers;
	ELandscapeImportResult LayerImportResult = bIsNewLandscape  ? UISettings->CreateNewLayersInfo(MaterialImportLayers) : UISettings->CreateImportLayersInfo(MaterialImportLayers);

	if (LayerImportResult == ELandscapeImportResult::Error)
	{
		UE_LOGF(LogLandscapeTools, Error, "Unable to import weight maps");
		return FReply::Handled();
	}

	TMap<FGuid, TArray<uint16>> HeightDataPerLayers;
	TMap<FGuid, TArray<FLandscapeImportLayerInfo>> MaterialLayerDataPerLayers;

	TArray<uint16> OutHeightData;
	
	if (!(bIsNewLandscape || bNeedsLandscapeRegions))
	{
		UISettings->ExpandImportData(OutHeightData, MaterialImportLayers);
	}

	HeightDataPerLayers.Add(FGuid(), OutHeightData);
	// ComputeHeightData will also modify/expand material layers data, which is why we create MaterialLayerDataPerLayers after calling ComputeHeightData
	MaterialLayerDataPerLayers.Add(FGuid(), MoveTemp(MaterialImportLayers));

	TUniquePtr<FScopedTransaction> Transaction;
	// If we're going to use regions then we're going to save and won't be able to undo this operation
	if (!bNeedsLandscapeRegions)
	{
		Transaction = MakeUnique<FScopedTransaction>(bIsNewLandscape ? LOCTEXT("Undo_CreateLandscape", "Creating New Landscape") : LOCTEXT("Undo_CreateAndImportLandscape", "Creating and Importing New Landscape"));
	}

	const FVector Offset = FTransform(UISettings->NewLandscape_Rotation, FVector::ZeroVector, UISettings->NewLandscape_Scale).TransformVector(FVector(-UISettings->NewLandscape_ComponentCount.X * QuadsPerComponent / 2.0, -UISettings->NewLandscape_ComponentCount.Y * QuadsPerComponent / 2.0, 0.0));
	
	ALandscape* Landscape = LandscapeEdMode->GetWorld()->SpawnActor<ALandscape>(UISettings->NewLandscape_Location + Offset, UISettings->NewLandscape_Rotation);
	Landscape->LandscapeMaterial = UISettings->NewLandscape_Material.Get();
	Landscape->SetActorRelativeScale3D(UISettings->NewLandscape_Scale);

	// automatically calculate a lighting LOD that won't crash lightmass (hopefully)
	// < 2048x2048 -> LOD0
	// >=2048x2048 -> LOD1
	// >= 4096x4096 -> LOD2
	// >= 8192x8192 -> LOD3
	Landscape->StaticLightingLOD = FMath::DivideAndRoundUp(FMath::CeilLogTwo((SizeX * SizeY) / (2048 * 2048) + 1), (uint32)2);

	FString ReimportHeightmapFilePath;
	if (LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
	{
		ReimportHeightmapFilePath = UISettings->ImportLandscape_HeightmapFilename;
	}

	Landscape->Import(FGuid::NewGuid(), 0, 0, SizeX - 1, SizeY - 1, UISettings->NewLandscape_SectionsPerComponent, QuadsPerSection, HeightDataPerLayers, *ReimportHeightmapFilePath, MaterialLayerDataPerLayers, UISettings->ImportLandscape_AlphamapType, TArrayView<const FLandscapeLayer>());

	ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
	check(LandscapeInfo);

	FActorLabelUtilities::SetActorLabelUnique(Landscape, ALandscape::StaticClass()->GetName());

	LandscapeInfo->UpdateLayerInfoMap(Landscape);

	// Create a target layer for each import layer. Layers without an asset set will be created, but without layer info
	const TArray<FLandscapeImportLayer>& ImportLandscapeLayersList = UISettings->NewLandscape_Layers;
	for (int32 i = 0; i < ImportLandscapeLayersList.Num(); i++)
	{
		ULandscapeLayerInfoObject* LayerInfo = ImportLandscapeLayersList[i].LayerInfo;
		FName LayerName = ImportLandscapeLayersList[i].LayerName;

		if (LayerInfo != nullptr)
		{
			if (LandscapeEdMode->NewLandscapePreviewMode == ENewLandscapePreviewMode::ImportLandscape)
			{
				Landscape->AddTargetLayer(LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfo, ImportLandscapeLayersList[i].SourceFilePath));
			}
			else
			{
				Landscape->AddTargetLayer(LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(LayerInfo));
			}

			int32 LayerInfoIndex = LandscapeInfo->GetLayerInfoIndex(ImportLandscapeLayersList[i].LayerName);
			if (ensure(LayerInfoIndex != INDEX_NONE))
			{
				FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[LayerInfoIndex];
				LayerSettings.LayerInfoObj = LayerInfo;
			}
		}
		else
		{
			// Add a target layer with no layer info asset
			Landscape->AddTargetLayer(LayerName, FLandscapeTargetLayerSettings());
		}
	}

	LandscapeEdMode->UpdateLandscapeList();
	LandscapeEdMode->SetLandscapeInfo(LandscapeInfo);
	LandscapeEdMode->CurrentToolTarget.TargetType = ELandscapeToolTargetType::Heightmap;
	LandscapeEdMode->SetCurrentTargetLayer(NAME_None, nullptr);
	LandscapeEdMode->SetCurrentTool("Select"); // change tool so switching back to the manage mode doesn't give "New Landscape" again
	LandscapeEdMode->SetCurrentTool("Sculpt"); // change to sculpting mode and tool

	World->GetSubsystem<ULandscapeSubsystem>()->ChangeGridSize(LandscapeInfo, UISettings->WorldPartitionGridSize);

	ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>();
	ALandscapeProxy* LandscapeProxy = LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetLandscapeProxy();
	
	if (bNeedsLandscapeRegions)
	{
		ULevel* Level = LandscapeProxy->GetLevel();
		UPackage* LevelPackage = Level->GetPackage();

		TArray<FIntPoint> NewComponents;
		NewComponents.Empty(TotalLandscapeComponentSize.X * TotalLandscapeComponentSize.Y);
		for (int32 Y = 0; Y < TotalLandscapeComponentSize.Y; Y++)
		{
			for (int32 X = 0; X < TotalLandscapeComponentSize.X; X++)
			{
				NewComponents.Add(FIntPoint(X, Y));
			}
		}

		int32 NumRegions = FMath::DivideAndRoundUp(TotalLandscapeComponentSize.X, static_cast<int32>(UISettings->WorldPartitionRegionSize)) * FMath::DivideAndRoundUp(TotalLandscapeComponentSize.Y, static_cast<int32>(UISettings->WorldPartitionRegionSize));


		FScopedSlowTask Progress(static_cast<float>(NumRegions), LOCTEXT("CreateLandscapeRegions", "Creating Landscape Editor Regions..."));
		Progress.MakeDialog();

		TArray<ALocationVolume*> RegionVolumes;
		FBox LandscapeBounds;
		auto AddComponentsToRegion = [&Progress, NumRegions, bIsNewLandscape, LandscapeEdMode, World, LandscapeProxy, &UISettings, QuadsPerSection, LandscapeInfo, LandscapeSubsystem, &RegionVolumes, &LandscapeBounds,&MaterialLayerDataPerLayers](const FIntPoint& RegionCoordinate, const TArray<FIntPoint>& NewComponents)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AddComponentsToRegion);
						
			// Create a LocationVolume around the region 
			int32 RegionSizeXTexels = QuadsPerSection * UISettings->WorldPartitionRegionSize * UISettings->NewLandscape_SectionsPerComponent;
			int32 RegionSizeYTexels = QuadsPerSection * UISettings->WorldPartitionRegionSize * UISettings->NewLandscape_SectionsPerComponent;

			double RegionSizeX = RegionSizeXTexels * LandscapeProxy->GetActorScale3D().X;
			double RegionSizeY = RegionSizeYTexels * LandscapeProxy->GetActorScale3D().Y;
			ALocationVolume* RegionVolume = LandscapeRegionUtils::CreateLandscapeRegionVolume(World, LandscapeProxy, RegionCoordinate, RegionSizeX);
			if (RegionVolume)
			{
				RegionVolumes.Add(RegionVolume);
			}

			TArray<ALandscapeProxy*> CreatedStreamingProxies;
			AddComponents(LandscapeInfo, LandscapeSubsystem, NewComponents, CreatedStreamingProxies);
			
			FIntRect ImportRegion (RegionSizeXTexels * RegionCoordinate.X, RegionSizeYTexels* RegionCoordinate.Y, RegionSizeXTexels * (RegionCoordinate.X +1) + 1  , RegionSizeYTexels * (RegionCoordinate.Y +1) +1 );
			FIntPoint ImportOffset(0, 0);

			if (!bIsNewLandscape && !UISettings->ImportLandscape_HeightmapFilename.IsEmpty())
			{
				LandscapeEdMode->ImportHeightData(LandscapeInfo, LandscapeEdMode->GetCurrentLayerGuid(), UISettings->ImportLandscape_HeightmapFilename, ImportRegion, ELandscapeImportTransformType::Subregion, ImportOffset, ELandscapeLayerPaintingRestriction::None, LandscapeEdMode->UISettings->bFlipYAxis);

				TArray<FLandscapeImportLayerInfo>& Weights = *MaterialLayerDataPerLayers.Find(FGuid());

				for (const TPair<FGuid, TArray<FLandscapeImportLayerInfo>>& Layer : MaterialLayerDataPerLayers)
				{
					for (const FLandscapeImportLayerInfo& WeightMap : Layer.Value)
					{						
						LandscapeEdMode->ImportWeightData(LandscapeInfo, LandscapeEdMode->GetCurrentLayerGuid(), WeightMap.LayerInfo, WeightMap.SourceFilePath, ImportRegion, ELandscapeImportTransformType::Subregion, ImportOffset, ELandscapeLayerPaintingRestriction::None, LandscapeEdMode->UISettings->bFlipYAxis);
					}
				}
			}

			// ensures all the final height textures have been updated.
			LandscapeInfo->ForceLayersFullUpdate();
			LandscapeEditorUtils::SaveLandscapeProxies(World, MakeArrayView(CreatedStreamingProxies));
			LandscapeBounds += LandscapeInfo->GetCompleteBounds();

			Progress.EnterProgressFrame(1.0f , FText::Format(LOCTEXT("LandscapeCreateRegion", "Creating Landscape Editor Regions ({0}, {1})"), RegionCoordinate.X, RegionCoordinate.Y));
			return true;
		};

		LandscapeEditorUtils::SaveObjects(TArrayView<ALandscape*>(TArray<ALandscape*> { LandscapeInfo->LandscapeActor.Get() }));

		LandscapeRegionUtils::ForEachComponentByRegion(UISettings->WorldPartitionRegionSize, NewComponents, AddComponentsToRegion);

		// update the zcomponent of the volumes 
		//const float ZScale =  LandscapeBounds.Max.Z - LandscapeBounds.Min.Z;
		for (ALocationVolume* RegionVolume : RegionVolumes)
		{
			const FVector Scale = RegionVolume->GetActorScale();
			const FVector NewScale{ Scale.X, Scale.Y, 1000000.0f}; //ZScale * 10.0 
			RegionVolume->SetActorScale3D(NewScale);
		}

		LandscapeEditorUtils::SaveObjects(MakeArrayView(RegionVolumes));

		TArray<ALandscapeProxy*> AllProxies;
		
		// save the initial region & unload it
		LandscapeInfo->ForEachLandscapeProxy([&AllProxies](ALandscapeProxy* Proxy) {
			if (Proxy->IsA<ALandscapeStreamingProxy>())
			{		
				AllProxies.Add(Proxy);
				
			}
			return true;
		});

		LandscapeEditorUtils::SaveLandscapeProxies(World, MakeArrayView(AllProxies));
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_NewLandscape::OnFillWorldButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("FillWorld_Transaction", "Landscape Fill World"));

		LandscapeEdMode->UISettings->Modify();

		LandscapeEdMode->UISettings->NewLandscape_Location = FVector::ZeroVector;
		const int32 QuadsPerComponent = LandscapeEdMode->UISettings->NewLandscape_SectionsPerComponent * LandscapeEdMode->UISettings->NewLandscape_QuadsPerSection;
		LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X = FMath::CeilToInt32(WORLD_MAX / QuadsPerComponent / LandscapeEdMode->UISettings->NewLandscape_Scale.X);
		LandscapeEdMode->UISettings->NewLandscape_ComponentCount.Y = FMath::CeilToInt32(WORLD_MAX / QuadsPerComponent / LandscapeEdMode->UISettings->NewLandscape_Scale.Y);

		const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
		if (Settings->IsLandscapeResolutionRestricted())
		{
			auto ClampComponentCount = [Settings, &QuadsPerComponent](int32& ComponentCount)
			{
				const float MaxResolution = static_cast<float>(Settings->GetSideResolutionLimit());
				ComponentCount = FMath::Clamp(ComponentCount, 1, FMath::Min(32, FMath::FloorToInt((MaxResolution - 1) / QuadsPerComponent)));
			};

			ClampComponentCount(LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X);
			ClampComponentCount(LandscapeEdMode->UISettings->NewLandscape_ComponentCount.Y);
		}
		else
		{
			LandscapeEdMode->UISettings->NewLandscape_ClampSize();
		}
	}

	return FReply::Handled();
}

FReply FLandscapeEditorDetailCustomization_NewLandscape::OnFitImportDataButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->UISettings->ChooseBestComponentSizeForImport();
	}

	return FReply::Handled();
}

bool FLandscapeEditorDetailCustomization_NewLandscape::GetImportButtonIsEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		if (LandscapeEdMode->UISettings->ImportLandscape_HeightmapImportResult == ELandscapeImportResult::Error ||
			LandscapeEdMode->UISettings->ImportLandscape_HeightmapFilename.IsEmpty()) 
		{
			return false;
		}

		for (int32 i = 0; i < LandscapeEdMode->UISettings->NewLandscape_Layers.Num(); ++i)
		{
			if (LandscapeEdMode->UISettings->NewLandscape_Layers[i].ImportResult == ELandscapeImportResult::Error)
			{
				return false;
			}
		}

		return IsCreateButtonEnabled();
	}
	
	return false;
}

EVisibility FLandscapeEditorDetailCustomization_NewLandscape::GetHeightmapErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult)
{
	ELandscapeImportResult HeightmapImportResult;
	FPropertyAccess::Result Result = PropertyHandle_HeightmapImportResult->GetValue((uint8&)HeightmapImportResult);

	if (Result == FPropertyAccess::Fail)
	{
		return EVisibility::Collapsed;
	}

	if (Result == FPropertyAccess::MultipleValues)
	{
		return EVisibility::Visible;
	}

	if (HeightmapImportResult != ELandscapeImportResult::Success)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FSlateColor FLandscapeEditorDetailCustomization_NewLandscape::GetHeightmapErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult)
{
	ELandscapeImportResult HeightmapImportResult;
	FPropertyAccess::Result Result = PropertyHandle_HeightmapImportResult->GetValue((uint8&)HeightmapImportResult);

	if (Result == FPropertyAccess::Fail ||
		Result == FPropertyAccess::MultipleValues)
	{
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	}

	switch (HeightmapImportResult)
	{
	case ELandscapeImportResult::Success:
		return FCoreStyle::Get().GetColor("InfoReporting.BackgroundColor");
	case ELandscapeImportResult::Warning:
		return FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor");
	case ELandscapeImportResult::Error:
		return FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor");
	default:
		check(0);
		return FSlateColor();
	}
}

void FLandscapeEditorDetailCustomization_NewLandscape::SetImportHeightmapFilenameString(const FText& NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename)
{
	FString HeightmapFilename = NewValue.ToString();
	ensure(PropertyHandle_HeightmapFilename->SetValue(HeightmapFilename) == FPropertyAccess::Success);
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnImportHeightmapFilenameChanged()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->UISettings->OnImportHeightmapFilenameChanged();
	}
}

FReply FLandscapeEditorDetailCustomization_NewLandscape::OnImportHeightmapFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	check(LandscapeEdMode != nullptr);

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const FString FileTypes = LandscapeEditorModule.GetHeightmapImportDialogTypeString();

	TOptional<FString> OptionalFilename = LandscapeEditorUtils::GetImportExportFilename(NSLOCTEXT("UnrealEd", "Import", "Import").ToString(), LandscapeEdMode->UISettings->LastImportPath, FileTypes, /* bInImporting = */ true);
	if (OptionalFilename.IsSet())
	{
		const FString& Filename = OptionalFilename.GetValue();
		ensure(PropertyHandle_HeightmapFilename->SetValue(Filename) == FPropertyAccess::Success);
		LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(Filename);
	}

	return FReply::Handled();

}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_NewLandscape::GetImportLandscapeResolutionMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		for (int32 i = 0; i < LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions.Num(); i++)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Width"), LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions[i].Width);
			Args.Add(TEXT("Height"), LandscapeEdMode->UISettings->HeightmapImportDescriptor.ImportResolutions[i].Height);
			MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("ImportResolution_Format", "{Width}\u00D7{Height}"), Args), FText(), FSlateIcon(), FExecuteAction::CreateSP(this, &FLandscapeEditorDetailCustomization_NewLandscape::OnChangeImportLandscapeResolution, i));
		}
	}

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_NewLandscape::OnChangeImportLandscapeResolution(int32 Index)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->UISettings->OnChangeImportLandscapeResolution(Index);
	}
}

FText FLandscapeEditorDetailCustomization_NewLandscape::GetImportLandscapeResolution() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		const int32 Width = LandscapeEdMode->UISettings->ImportLandscape_Width;
		const int32 Height = LandscapeEdMode->UISettings->ImportLandscape_Height;
		if (Width != 0 && Height != 0)
		{
			FFormatNamedArguments Args;
			Args.Add(TEXT("Width"), Width);
			Args.Add(TEXT("Height"), Height);
			return FText::Format(LOCTEXT("ImportResolution_Format", "{Width}\u00D7{Height}"), Args);
		}
		else
		{
			return LOCTEXT("ImportResolution_Invalid", "(invalid)");
		}
	}

	return FText::GetEmpty();
}

EVisibility FLandscapeEditorDetailCustomization_NewLandscape::GetMaterialTipVisibility() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		if (LandscapeEdMode->UISettings->NewLandscape_Layers.Num() == 0)
		{
			return EVisibility::Visible;
		}
	}
	return EVisibility::Collapsed;
}

void FLandscapeEditorDetailCustomization_NewLandscape::ResetMaterialToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
	TSoftObjectPtr<UMaterialInterface> DefaultMaterial = Settings->GetDefaultLandscapeMaterial();
	UObject* ValueToSet = nullptr;

	if (!DefaultMaterial.IsNull())
	{
		ValueToSet = DefaultMaterial.LoadSynchronous();
	}

	InPropertyHandle->SetValue(ValueToSet);
}

bool FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetMaterialToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
	TSoftObjectPtr<UMaterialInterface> DefaultMaterial = Settings->GetDefaultLandscapeMaterial();

	UObject* Object;
	InPropertyHandle->GetValue(Object);

	return DefaultMaterial.Get() != Object;
}

void FLandscapeEditorDetailCustomization_NewLandscape::ResetLocationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	InPropertyHandle->SetValue(ULandscapeEditorObject::NewLandscape_DefaultLocation);
}

bool FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetLocationlToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FVector Location;
	InPropertyHandle->GetValue(Location);

	return Location != ULandscapeEditorObject::NewLandscape_DefaultLocation;
}

void FLandscapeEditorDetailCustomization_NewLandscape::ResetRotationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	InPropertyHandle->SetValue(ULandscapeEditorObject::NewLandscape_DefaultRotation);
}

bool FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetRotationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FRotator Rotation;
	InPropertyHandle->GetValue(Rotation);

	return Rotation != ULandscapeEditorObject::NewLandscape_DefaultRotation;
}

void FLandscapeEditorDetailCustomization_NewLandscape::ResetScaleToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	InPropertyHandle->SetValue(ULandscapeEditorObject::NewLandscape_DefaultScale);
}

bool FLandscapeEditorDetailCustomization_NewLandscape::ShouldShowResetScaleToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	FVector Scale;
	InPropertyHandle->GetValue(Scale);

	return Scale != ULandscapeEditorObject::NewLandscape_DefaultScale;
}

#undef LOCTEXT_NAMESPACE
