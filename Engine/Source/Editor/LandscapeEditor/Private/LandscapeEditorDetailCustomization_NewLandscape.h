// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "LandscapeEdMode.h"
#include "LandscapeFileFormatInterface.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "LandscapeEditorDetailCustomization_Base.h"
#include "LandscapeSubsystem.h"

class IDetailLayoutBuilder;

enum class ENewLandscapePreviewMode : uint8;

/**
 * Slate widgets customizer for the "New Landscape" tool
 */

class FLandscapeEditorArrayBuilder_NewTargetLayer : public FDetailArrayBuilder
{
public:
	FLandscapeEditorArrayBuilder_NewTargetLayer(TSharedRef<IPropertyHandle> InBaseProperty)
		: FDetailArrayBuilder(InBaseProperty)
	{
	}

	virtual void GenerateHeaderRowContent(FDetailWidgetRow& NodeRow) override;

private:
	static FReply ShowAutoFillTargetLayerDialog();
	static FReply HandleAutoFillTargetLayers(const bool bCreateAssetForEmptyLayers);
	static bool HasWeightmapTargetLayers();
};

class FLandscapeEditorDetailCustomization_NewLandscape : public FLandscapeEditorDetailCustomization_Base
{
public:
	FLandscapeEditorDetailCustomization_NewLandscape()
		: bUsingSlider(false)
	{}

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:
	static void SetScale(FVector::FReal NewValue, ETextCommit::Type, TSharedRef<IPropertyHandle> PropertyHandle);

	static TSharedRef<SWidget> GetSectionSizeMenu(TSharedRef<IPropertyHandle> PropertyHandle);
	static void OnChangeSectionSize(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize);
	static FText GetSectionSize(TSharedRef<IPropertyHandle> PropertyHandle);

	static TSharedRef<SWidget> GetSectionsPerComponentMenu(TSharedRef<IPropertyHandle> PropertyHandle);
	static void OnChangeSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize);
	static FText GetSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle);

	TOptional<int32> GetLandscapeResolutionX() const;
	void OnChangeLandscapeResolutionX(int32 NewValue, bool bCommit);
	
	TOptional<int32> GetLandscapeResolutionY() const;
	void OnChangeLandscapeResolutionY(int32 NewValue, bool bCommit);
	
	TOptional<int32> GetMinLandscapeResolution() const;
	TOptional<int32> GetMaxLandscapeResolution() const;

	FText GetTotalComponentCount() const;

	bool IsCreateButtonEnabled() const;
	EVisibility GetNewLandscapeErrorVisibility() const;
	FText GetNewLandscapeErrorText() const;
	
	FReply OnCreateButtonClicked();
	FReply OnFillWorldButtonClicked();

	static EVisibility GetVisibilityOnlyInNewLandscapeMode(ENewLandscapePreviewMode value);

	/** Called to generate ImportLayer children */
	void GenerateLayersArrayElementWidget(TSharedRef<IPropertyHandle> InPropertyHandle, int32 InArrayIndex, IDetailChildrenBuilder& InChildrenBuilder);
	EVisibility GetLayerVisibility(TSharedRef<IPropertyHandle> InPropertyHandle) const;

	// Import
	static EVisibility GetHeightmapErrorVisibility(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult);
	static FSlateColor GetHeightmapErrorColor(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapImportResult);
	static void SetImportHeightmapFilenameString(const FText& NewValue, ETextCommit::Type CommitInfo, TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename);
	void OnImportHeightmapFilenameChanged();
	static FReply OnImportHeightmapFilenameButtonClicked(TSharedRef<IPropertyHandle> PropertyHandle_HeightmapFilename);

	TSharedRef<SWidget> GetImportLandscapeResolutionMenu();
	void OnChangeImportLandscapeResolution(int32 NewConfigIndex);
	FText GetImportLandscapeResolution() const;

	bool GetImportButtonIsEnabled() const;
	FReply OnFitImportDataButtonClicked();

	FText GetOverallResolutionTooltip() const;

	// Import layers
	EVisibility GetMaterialTipVisibility() const;

	void ResetMaterialToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
	bool ShouldShowResetMaterialToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void ResetLocationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
	bool ShouldShowResetLocationlToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void ResetRotationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
	bool ShouldShowResetRotationToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

	void ResetScaleToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);
	bool ShouldShowResetScaleToDefault(TSharedPtr<IPropertyHandle> InPropertyHandle);

protected:

	bool bUsingSlider;

private:
	static void AddComponents(ULandscapeInfo* InLandscapeInfo, ULandscapeSubsystem* InLandscapeSubsystem, const TArray<FIntPoint>& InComponentCoordinates, TArray<ALandscapeProxy*>& OutCreatedStreamingProxies);
};
