// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_ResizeLandscape.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdMode.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

//#include "ObjectTools.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.ResizeLandscape"

#include "LandscapeConfigHelper.h"

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_ResizeLandscape::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_ResizeLandscape);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_ResizeLandscape::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	if (!IsToolActive("ResizeLandscape"))
	{
		return;
	}

	IDetailCategoryBuilder& ResizeLandscapeCategory = DetailBuilder.EditCategory("Change Component Size");

	ResizeLandscapeCategory.AddCustomRow(LOCTEXT("OriginalNewLabel", "Original New"))
	//.NameContent()
	//[
	//]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0,8,12,2)) // Line up with the other properties due to having no reset to default button
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("Original", "Original"))
				.ToolTipText(LOCTEXT("Original_Tip", "The properties of the landscape as it currently exists"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.1f)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text(LOCTEXT("New", "New"))
				.ToolTipText(LOCTEXT("New_Tip", "The properties the landscape will have after the resize operation is completed"))
			]
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_QuadsPerSection = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_QuadsPerSection));
	ResizeLandscapeCategory.AddProperty(PropertyHandle_QuadsPerSection)
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateStatic(&FLandscapeEditorDetailCustomization_ResizeLandscape::IsSectionSizeResetToDefaultVisible),
		FResetToDefaultHandler::CreateStatic(&FLandscapeEditorDetailCustomization_ResizeLandscape::OnSectionSizeResetToDefault)))
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_QuadsPerSection->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetOriginalSectionSize)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1.1f)
		[
			SNew(SComboButton)
			.OnGetMenuContent_Static(&GetSectionSizeMenu, PropertyHandle_QuadsPerSection)
			.ContentPadding(2.0f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetSectionSize, PropertyHandle_QuadsPerSection)
			]
		]
	];

	TSharedRef<IPropertyHandle> PropertyHandle_SectionsPerComponent = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_SectionsPerComponent));
	ResizeLandscapeCategory.AddProperty(PropertyHandle_SectionsPerComponent)
	.OverrideResetToDefault(FResetToDefaultOverride::Create(
		FIsResetToDefaultVisible::CreateStatic(&FLandscapeEditorDetailCustomization_ResizeLandscape::IsSectionsPerComponentResetToDefaultVisible),
		FResetToDefaultHandler::CreateStatic(&FLandscapeEditorDetailCustomization_ResizeLandscape::OnSectionsPerComponentResetToDefault)))
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_SectionsPerComponent->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetOriginalSectionsPerComponent)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1.1f)
		[
			SNew(SComboButton)
			.OnGetMenuContent_Static(&GetSectionsPerComponentMenu, PropertyHandle_SectionsPerComponent)
			.ContentPadding(2.0f)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetSectionsPerComponent, PropertyHandle_SectionsPerComponent)
			]
		]
	];
	
	TSharedRef<IPropertyHandle> PropertyHandle_ConvertMode = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_ConvertMode));
	ResizeLandscapeCategory.AddProperty(PropertyHandle_ConvertMode)
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_ConvertMode->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		PropertyHandle_ConvertMode->CreatePropertyValueWidget()
	];

	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_ComponentCount));
	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_X = PropertyHandle_ComponentCount->GetChildHandle("X").ToSharedRef();
	TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_Y = PropertyHandle_ComponentCount->GetChildHandle("Y").ToSharedRef();
	ResizeLandscapeCategory.AddProperty(PropertyHandle_ComponentCount)
	.OverrideResetToDefault(FResetToDefaultOverride::Hide())
	.CustomWidget()
	.NameContent()
	[
		PropertyHandle_ComponentCount->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetOriginalComponentCount)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1.1f)
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text_Static(&GetComponentCount, PropertyHandle_ComponentCount_X, PropertyHandle_ComponentCount_Y)
		]
	];

	ResizeLandscapeCategory.AddCustomRow(LOCTEXT("Resolution", "Overall Resolution"))
	.NameContent()
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(LOCTEXT("Resolution", "Overall Resolution"))
			.ToolTipText(LOCTEXT("Resolution_Tip", "Overall resolution of the entire landscape in vertices"))
		]
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0,0,12,0)) // Line up with the other properties due to having no reset to default button
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetOriginalLandscapeResolution)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.1f)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetLandscapeResolution)
			]
		]
	];

	ResizeLandscapeCategory.AddCustomRow(LOCTEXT("TotalComponents", "Total Components"))
	.NameContent()
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(2))
		[
			SNew(STextBlock)
			.Font(DetailBuilder.GetDetailFont())
			.Text(LOCTEXT("TotalComponents", "Total Components"))
			.ToolTipText(LOCTEXT("TotalComponents_Tip", "The total number of components in the landscape"))
		]
	]
	.ValueContent()
	.MinDesiredWidth(180)
	.MaxDesiredWidth(180)
	[
		SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0,0,12,0)) // Line up with the other properties due to having no reset to default button
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetOriginalTotalComponentCount)
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.1f)
			[
				SNew(STextBlock)
				.Font(DetailBuilder.GetDetailFont())
				.Text_Static(&GetTotalComponentCount)
			]
		]
	];

	ResizeLandscapeCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		//[
		//]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SButton)
			.Text(LOCTEXT("Apply", "Apply"))
			.OnClicked(this, &FLandscapeEditorDetailCustomization_ResizeLandscape::OnApplyButtonClicked)
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetOriginalSectionSize()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return FText::Format(LOCTEXT("NxNQuads", "{0}x{0} Quads"), FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_QuadsPerSection));
	}

	return FText::FromString("---");
}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_ResizeLandscape::GetSectionSizeMenu(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < UE_ARRAY_COUNT(FLandscapeConfig::SubsectionSizeQuadsValues); i++)
	{
		MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("NxNQuads", "{0}x{0} Quads"), FText::AsNumber(FLandscapeConfig::SubsectionSizeQuadsValues[i])), FText::GetEmpty(), FSlateIcon(), FExecuteAction::CreateStatic(&OnChangeSectionSize, PropertyHandle, FLandscapeConfig::SubsectionSizeQuadsValues[i]));
	}

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_ResizeLandscape::OnChangeSectionSize(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize)
{
	ensure(PropertyHandle->SetValue(NewSize) == FPropertyAccess::Success);
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetSectionSize(TSharedRef<IPropertyHandle> PropertyHandle)
{
	int32 QuadsPerSection = 0;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(QuadsPerSection);
	check(Result != FPropertyAccess::Fail);

	if (Result == FPropertyAccess::MultipleValues)
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	return FText::Format(LOCTEXT("NxNQuads", "{0}x{0} Quads"), FText::AsNumber(QuadsPerSection));
}

bool FLandscapeEditorDetailCustomization_ResizeLandscape::IsSectionSizeResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->UISettings->ResizeLandscape_QuadsPerSection != LandscapeEdMode->UISettings->ResizeLandscape_Original_QuadsPerSection;
	}

	return false;
}

void FLandscapeEditorDetailCustomization_ResizeLandscape::OnSectionSizeResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->UISettings->ResizeLandscape_QuadsPerSection = LandscapeEdMode->UISettings->ResizeLandscape_Original_QuadsPerSection;
	}
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetOriginalSectionsPerComponent()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		int32 SectionsPerComponent = LandscapeEdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent;

		FFormatNamedArguments Args;
		Args.Add(TEXT("Width"), SectionsPerComponent);
		Args.Add(TEXT("Height"), SectionsPerComponent);
		return FText::Format(SectionsPerComponent == 1 ? LOCTEXT("1x1Section", "{Width}\u00D7{Height} Section") : LOCTEXT("NxNSections", "{Width}\u00D7{Height} Sections"), Args);
	}

	return FText::FromString("---");
}

TSharedRef<SWidget> FLandscapeEditorDetailCustomization_ResizeLandscape::GetSectionsPerComponentMenu(TSharedRef<IPropertyHandle> PropertyHandle)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	for (int32 i = 0; i < UE_ARRAY_COUNT(FLandscapeConfig::NumSectionValues); i++)
	{
		FFormatNamedArguments Args;
		Args.Add(TEXT("Width"), FLandscapeConfig::NumSectionValues[i]);
		Args.Add(TEXT("Height"), FLandscapeConfig::NumSectionValues[i]);
		MenuBuilder.AddMenuEntry(FText::Format(FLandscapeConfig::NumSectionValues[i] == 1 ? LOCTEXT("1x1Section", "{Width}\u00D7{Height} Section") : LOCTEXT("NxNSections", "{Width}\u00D7{Height} Sections"), Args), FText::GetEmpty(), FSlateIcon(), FExecuteAction::CreateStatic(&OnChangeSectionsPerComponent, PropertyHandle, FLandscapeConfig::NumSectionValues[i]));
	}

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorDetailCustomization_ResizeLandscape::OnChangeSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize)
{
	ensure(PropertyHandle->SetValue(NewSize) == FPropertyAccess::Success);
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle)
{
	int32 SectionsPerComponent = 0;
	FPropertyAccess::Result Result = PropertyHandle->GetValue(SectionsPerComponent);
	check(Result != FPropertyAccess::Fail);

	if (Result == FPropertyAccess::MultipleValues)
	{
		return NSLOCTEXT("PropertyEditor", "MultipleValues", "Multiple Values");
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("Width"), SectionsPerComponent);
	Args.Add(TEXT("Height"), SectionsPerComponent);
	return FText::Format(SectionsPerComponent == 1 ? LOCTEXT("1x1Section", "{Width}\u00D7{Height} Section") : LOCTEXT("NxNSections", "{Width}\u00D7{Height} Sections"), Args);
}

bool FLandscapeEditorDetailCustomization_ResizeLandscape::IsSectionsPerComponentResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->UISettings->ResizeLandscape_SectionsPerComponent != LandscapeEdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent;
	}

	return false;
}

void FLandscapeEditorDetailCustomization_ResizeLandscape::OnSectionsPerComponentResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->UISettings->ResizeLandscape_SectionsPerComponent = LandscapeEdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent;
	}
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetOriginalComponentCount()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return FText::Format(LOCTEXT("NxN", "{0}\u00D7{1}"),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.X),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.Y));
	}
	return FText::FromString(TEXT("---"));
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetComponentCount(TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_X, TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_Y)
{
	return FText::Format(LOCTEXT("NxN", "{0}\u00D7{1}"),
		FLandscapeEditorDetailCustomization_Base::GetPropertyValueText(PropertyHandle_ComponentCount_X),
		FLandscapeEditorDetailCustomization_Base::GetPropertyValueText(PropertyHandle_ComponentCount_Y));
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetOriginalLandscapeResolution()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		const int32 Original_ComponentSizeQuads = LandscapeEdMode->UISettings->ResizeLandscape_Original_SectionsPerComponent * LandscapeEdMode->UISettings->ResizeLandscape_Original_QuadsPerSection;
		return FText::Format(LOCTEXT("NxN", "{0}\u00D7{1}"),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.X * Original_ComponentSizeQuads + 1),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.Y * Original_ComponentSizeQuads + 1));
	}

	return FText::FromString(TEXT("---"));
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetLandscapeResolution()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		const int32 ComponentSizeQuads = LandscapeEdMode->UISettings->ResizeLandscape_SectionsPerComponent * LandscapeEdMode->UISettings->ResizeLandscape_QuadsPerSection;
		return FText::Format(LOCTEXT("NxN", "{0}\u00D7{1}"),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_ComponentCount.X * ComponentSizeQuads + 1),
			FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_ComponentCount.Y * ComponentSizeQuads + 1));
	}

	return FText::FromString(TEXT("---"));
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetOriginalTotalComponentCount()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.X * LandscapeEdMode->UISettings->ResizeLandscape_Original_ComponentCount.Y);
	}

	return FText::FromString(TEXT("---"));
}

FText FLandscapeEditorDetailCustomization_ResizeLandscape::GetTotalComponentCount()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return FText::AsNumber(LandscapeEdMode->UISettings->ResizeLandscape_ComponentCount.X * LandscapeEdMode->UISettings->ResizeLandscape_ComponentCount.Y);
	}

	return FText::FromString(TEXT("---"));
}

FReply FLandscapeEditorDetailCustomization_ResizeLandscape::OnApplyButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		FScopedTransaction Transaction(LOCTEXT("Undo", "Changing Landscape Component Size"));

		const FIntPoint ComponentCount = LandscapeEdMode->UISettings->ResizeLandscape_ComponentCount;
		const int32 SectionsPerComponent = LandscapeEdMode->UISettings->ResizeLandscape_SectionsPerComponent;
		const int32 QuadsPerSection = LandscapeEdMode->UISettings->ResizeLandscape_QuadsPerSection;
		const bool bResample = (LandscapeEdMode->UISettings->ResizeLandscape_ConvertMode == ELandscapeConvertMode::Resample);
		ALandscape* NewLandscape = LandscapeEdMode->ChangeComponentSetting(ComponentCount.X, ComponentCount.Y, SectionsPerComponent, QuadsPerSection, bResample);

		LandscapeEdMode->UpdateLandscapeList();
		LandscapeEdMode->SetTargetLandscape(NewLandscape->GetLandscapeInfo());
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
