// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "LandscapeEditorDetailCustomization_Base.h"
#include "Widgets/SWidget.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"

class IDetailLayoutBuilder;

/**
 * Slate widgets customizer for the "Change Landscape Component Size" tool
 */

class FLandscapeEditorDetailCustomization_ResizeLandscape : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

public:
	static FText GetOriginalSectionSize();
	static TSharedRef<SWidget> GetSectionSizeMenu(TSharedRef<IPropertyHandle> PropertyHandle);
	static void OnChangeSectionSize(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize);
	static FText GetSectionSize(TSharedRef<IPropertyHandle> PropertyHandle);
	static bool IsSectionSizeResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle);
	static void OnSectionSizeResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);

	static FText GetOriginalSectionsPerComponent();
	static TSharedRef<SWidget> GetSectionsPerComponentMenu(TSharedRef<IPropertyHandle> PropertyHandle);
	static void OnChangeSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle, int32 NewSize);
	static FText GetSectionsPerComponent(TSharedRef<IPropertyHandle> PropertyHandle);
	static bool IsSectionsPerComponentResetToDefaultVisible(TSharedPtr<IPropertyHandle> PropertyHandle);
	static void OnSectionsPerComponentResetToDefault(TSharedPtr<IPropertyHandle> PropertyHandle);

	static FText GetOriginalComponentCount();
	static FText GetComponentCount(TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_X, TSharedRef<IPropertyHandle> PropertyHandle_ComponentCount_Y);

	static FText GetOriginalLandscapeResolution();
	static FText GetLandscapeResolution();

	static FText GetOriginalTotalComponentCount();
	static FText GetTotalComponentCount();

	FReply OnApplyButtonClicked();

protected:
	static const int32 SectionSizes[];
	static const int32 NumSections[];
};
