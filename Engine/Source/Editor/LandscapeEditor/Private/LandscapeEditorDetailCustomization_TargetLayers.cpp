// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetailCustomization_TargetLayers.h"
#include "IDetailChildrenBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "Brushes/SlateColorBrush.h"
#include "Layout/WidgetPath.h"
#include "SlateOptMacros.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Dialog/SCustomDialog.h"
#include "SSimpleButton.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorObject.h"
#include "Landscape.h"
#include "LandscapeUtils.h"
#include "Styling/AppStyle.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Materials/MaterialExpressionLandscapeVisibilityMask.h"
#include "Misc/Crc.h"

#include "SDocumentationToolTip.h"
#include "SLandscapeEditor.h"
#include "Dialogs/DlgPickAssetPath.h"
#include "Dialogs/DlgPickPath.h"
#include "ObjectTools.h"
#include "ScopedTransaction.h"
#include "DesktopPlatformModule.h"

#include "LandscapeRender.h"
#include "LandscapeEdit.h"
#include "Landscape.h"
#include "LandscapeEditorUtils.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.TargetLayers"

TSharedRef<IPropertyTypeCustomization> FLandscapeEditorStructCustomization_FTargetLayerAssetPath::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorStructCustomization_FTargetLayerAssetPath);
}

void FLandscapeEditorStructCustomization_FTargetLayerAssetPath::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	const TSharedRef<IPropertyHandle> PropertyHandle_TargetLayerDirectoryPath = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeTargetLayerAssetFilePath, DirectoryPath)).ToSharedRef();

	HeaderRow
		.NameContent()
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(5.0f, 0)
				[
					SNew(SCheckBox)
					.IsChecked_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::GetUseTargetLayerAssetPathCheckState)
					.OnCheckStateChanged_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::OnUseTargetLayerAssetPathCheckStateChanged)
					.ToolTipText(LOCTEXT("TargetLayerAssetPathCheckbox_ToolTip", "Enable to override the default asset path"))
				]
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.HAlign(HAlign_Fill)
				[
					PropertyHandle_TargetLayerDirectoryPath->CreatePropertyNameWidget(LOCTEXT("TargetLayerAssetPath", "Default Layer Asset Path"), LOCTEXT("TargetLayerAssetPath_ToolTip", "Asset folder where Landscape Layer Info assets will be searched when auto-populating landscape target layers"))
				]
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1)
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					// This path is edited via a SEditableTextBox (rather than using CreatePropertyValueWidget) so we have to specify a tooltip in order to display the full path if 
					//  the text box is to small to display the entire thing
					.ToolTipText_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::GetTargetLayerAssetFilePath)
					[
						SNew(SEditableTextBox)
						.Font(IDetailLayoutBuilder::GetDetailFont())
						.Text_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::GetTargetLayerAssetFilePath)
						.HintText(LOCTEXT("TargetLayerAssetPath_Hint", "(Specify a default path)"))
						.IsEnabled_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::IsUseTargetLayerAssetPathEnabled)
						.IsReadOnly(true)
					]
				]
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SSimpleButton)
					.OnClicked_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::OnSetTargetLayerAssetFilePath)
					.IsEnabled_Static(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::IsUseTargetLayerAssetPathEnabled)
					.Icon(FAppStyle::Get().GetBrush("Icons.FolderOpen"))
				]
		];
}

void FLandscapeEditorStructCustomization_FTargetLayerAssetPath::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Hide struct fields since the UI is all handled in the Header
	const TSharedRef<IPropertyHandle> PropertyHandle_TargetLayerFilePath = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeTargetLayerAssetFilePath, DirectoryPath)).ToSharedRef();
	PropertyHandle_TargetLayerFilePath->MarkHiddenByCustomization();
}

FText FLandscapeEditorStructCustomization_FTargetLayerAssetPath::GetTargetLayerAssetFilePath() 
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		return FText::FromString(LandscapeEdMode->GetTargetLayerAssetPackagePath(/* bIsEmptyPathValid */true));
	}

	return FText();
}

FReply FLandscapeEditorStructCustomization_FTargetLayerAssetPath::OnSetTargetLayerAssetFilePath()
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if ((LandscapeEdMode == nullptr) || (LandscapeEdMode->UISettings == nullptr))
	{
		return FReply::Unhandled();
	}

	FString DialogPath = LandscapeEdMode->GetTargetLayerAssetPackagePath(/*bIsEmptyPathValid = */false);

	const TSharedRef<SDlgPickPath> NewPathDlg =
		SNew(SDlgPickPath)
		.Title(LOCTEXT("TargetLayerAssetFilePath_Dlg", "Set Target Layer Asset File Path"))
		.DefaultPath(FText::FromString(DialogPath));

	if (NewPathDlg->ShowModal() != EAppReturnType::Cancel)
	{
		LandscapeEdMode->UISettings->TargetLayerAssetFilePath.DirectoryPath.Path = NewPathDlg->GetPath().ToString();
	}

	return FReply::Handled();
}

bool FLandscapeEditorStructCustomization_FTargetLayerAssetPath::IsUseTargetLayerAssetPathEnabled()
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		return LandscapeEdMode->UISettings->TargetLayerAssetFilePath.bUseAssetDirectoryPath;
	}

	return false;
}

ECheckBoxState FLandscapeEditorStructCustomization_FTargetLayerAssetPath::GetUseTargetLayerAssetPathCheckState()
{
	return IsUseTargetLayerAssetPathEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLandscapeEditorStructCustomization_FTargetLayerAssetPath::OnUseTargetLayerAssetPathCheckStateChanged(ECheckBoxState NewCheckedState)
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->UISettings)
	{
		if (NewCheckedState != ECheckBoxState::Undetermined)
		{
			LandscapeEdMode->UISettings->TargetLayerAssetFilePath.bUseAssetDirectoryPath = (NewCheckedState == ECheckBoxState::Checked);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

TSharedRef<IDetailCustomization> FLandscapeEditorDetailCustomization_TargetLayers::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetailCustomization_TargetLayers);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorDetailCustomization_TargetLayers::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TSharedRef<IPropertyHandle> PropertyHandle_PaintingRestriction = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintingRestriction));
	TSharedRef<IPropertyHandle> PropertyHandle_WeightBlendedTargetLayerPaintMode = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WeightBlendedTargetLayerPaintMode));

	TSharedRef<IPropertyHandle> PropertyHandle_TargetLayerAssetFilePath = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TargetLayerAssetFilePath));
	TSharedRef<IPropertyHandle> PropertyHandle_TargetDisplayOrder = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TargetDisplayOrder));
	PropertyHandle_TargetDisplayOrder->MarkHiddenByCustomization();
	TSharedRef<IPropertyHandle> PropertyHandle_TargetDisplayOrderIsAscendingPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bTargetDisplayOrdersAscending));
	PropertyHandle_TargetDisplayOrderIsAscendingPropertyHandle->MarkHiddenByCustomization();

	TSharedRef<IPropertyHandle> PropertyHandle_TargetShowUnusedLayers = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ShowUnusedLayers));
	PropertyHandle_TargetShowUnusedLayers->MarkHiddenByCustomization();

	if (!ShouldShowTargetLayers())
	{
		PropertyHandle_PaintingRestriction->MarkHiddenByCustomization();
		PropertyHandle_WeightBlendedTargetLayerPaintMode->MarkHiddenByCustomization();
		PropertyHandle_TargetLayerAssetFilePath->MarkHiddenByCustomization();
		return;
	}

	IDetailCategoryBuilder& TargetsCategory = DetailBuilder.EditCategory("Target Layers");
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	check(LandscapeEdMode);

	TargetsCategory.AddProperty(PropertyHandle_PaintingRestriction)
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetPaintingRestrictionVisibility));

	TargetsCategory.AddProperty(PropertyHandle_WeightBlendedTargetLayerPaintMode)
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetWeightBlendedTargetLayerPaintModeVisibility));

	TargetsCategory.AddProperty(PropertyHandle_TargetLayerAssetFilePath);

	TargetsCategory.AddCustomRow(FText())
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetVisibilityMaskTipVisibility))
		[
			SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.Font(DetailBuilder.GetDetailFontBold())
				.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
				.Text(LOCTEXT("Visibility_Tip", "Note: There are some areas where visibility painting is disabled because Component/Proxy don't have a \"Landscape Visibility Mask\" node in their material."))
				.AutoWrapText(true)
		];

	TargetsCategory.AddCustomBuilder(MakeShareable(new FLandscapeEditorCustomNodeBuilder_TargetLayers(DetailBuilder.GetThumbnailPool().ToSharedRef(), 
		PropertyHandle_TargetDisplayOrder, PropertyHandle_TargetDisplayOrderIsAscendingPropertyHandle, PropertyHandle_TargetShowUnusedLayers)));

	TargetsCategory.AddCustomRow(FText())
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetPopulateTargetLayersInfoTipVisibility))
		[
			SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.Font(DetailBuilder.GetDetailFontBold())
				.BackgroundColor(FAppStyle::GetColor("InfoReporting.BackgroundColor"))
				.Text(LOCTEXT("PopulateTargetLayers_Tip", "There are currently no target layers assigned to this landscape. Use the buttons above to add new ones or populate them from the material(s) currently assigned to the landscape"))
				.AutoWrapText(true)
		];

	TargetsCategory.AddCustomRow(FText())
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetTargetLayersInvalidInfoAssetTipVisibility))
		[
			SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.Font(DetailBuilder.GetDetailFontBold())
				.BackgroundColor(FAppStyle::GetColor("ErrorReporting.WarningBackgroundColor"))
				.Text(LOCTEXT("InvalidTargetLayers_Tip", "There are target layers with no layer info asset assigned. Create a new asset, select an existing, or use the Auto-Fill button above to quickly set assets for all layers"))
				.AutoWrapText(true)
		];

	TargetsCategory.AddCustomRow(FText())
		.Visibility(TAttribute<EVisibility>::CreateStatic(&FLandscapeEditorDetailCustomization_TargetLayers::GetFilteredTargetLayersListInfoTipVisibility))
		[
			SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.Font(DetailBuilder.GetDetailFontBold())
				.BackgroundColor(FAppStyle::GetColor("InfoReporting.BackgroundColor"))
				.Text(LOCTEXT("FilteredTargetLayers_Tip", "All target layers assigned to this landscape are currently filtered. Use the buttons and/or the filter above to un-hide them."))
				.AutoWrapText(true)
		];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool FLandscapeEditorDetailCustomization_TargetLayers::ShouldShowTargetLayers()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolMode && LandscapeEdMode->CurrentTool)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();

		// Target layers are included unless the tool focuses specifically on edit layers or components
		if (LandscapeEdMode->CurrentToolMode->SupportedTargetTypes != ELandscapeToolTargetTypeFlags::None
			&& CurrentToolName != TEXT("BlueprintBrush")
			&& CurrentToolName != TEXT("Mask"))
		{
			return true;
		}
	}

	return false;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetPaintingRestrictionVisibility()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();

		// Tool target type "Invalid" means Weightmap with no valid paint layer, so technically, it is weightmap and we therefore choose to show PaintingRestriction : 
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap && CurrentToolName != TEXT("BlueprintBrush"))
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetWeightBlendedTargetLayerPaintModeVisibility()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();

		// Tool target type "Invalid" means Weightmap with no valid paint layer, so technically, it is weightmap and we therefore choose to show WeightBlendedTargetLayerPaintMode : 
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap && CurrentToolName != TEXT("BlueprintBrush"))
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetVisibilityMaskTipVisibility()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility)
		{
			ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
			bool bHasValidHoleMaterial = true;
			LandscapeInfo->ForAllLandscapeComponents([&](ULandscapeComponent* LandscapeComponent)
			{
				bHasValidHoleMaterial &= LandscapeComponent->IsLandscapeHoleMaterialValid();
			});

			return bHasValidHoleMaterial ? EVisibility::Collapsed : EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetPopulateTargetLayersInfoTipVisibility()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid)) // ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" 
		{
			ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
			// Visibility layer is added by default behind the scenes, tooltip should be shown until there is a valid weightmap layer in the list
			const bool bIsVisibilityOnlyLayer = LandscapeInfo->Layers.Num() == 1 && LandscapeInfo->Layers[0].LayerInfoObj == ALandscapeProxy::VisibilityLayer;

			return LandscapeInfo->Layers.IsEmpty() || bIsVisibilityOnlyLayer ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetTargetLayersInvalidInfoAssetTipVisibility()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid)) // ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" 
		{
			ULandscapeInfo* LandscapeInfo = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
			// Visibility layer is added by default behind the scenes, tooltip should be shown until there is a valid weightmap layer in the list
			const bool bIsVisibilityOnlyLayer = LandscapeInfo->Layers.Num() == 1 && LandscapeInfo->Layers[0].LayerInfoObj == ALandscapeProxy::VisibilityLayer;

			// If we have no layers we cannot have missing layer info assets
			if (LandscapeInfo->Layers.IsEmpty() || bIsVisibilityOnlyLayer)
			{
				return EVisibility::Collapsed;
			}

			// Show the message if any layer is missing an asset
			for (FLandscapeInfoLayerSettings& Layer : LandscapeInfo->Layers)
			{
				if (Layer.LayerInfoObj == nullptr)
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorDetailCustomization_TargetLayers::GetFilteredTargetLayersListInfoTipVisibility()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid)) // ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" 
		{
			const TArray<TSharedRef<FLandscapeTargetListInfo>>& TargetList = LandscapeEdMode->GetTargetList();
			// The first target layers are for heightmap and visibility so only consider target layers above the starting index : 
			const bool bHasTargetLayers = TargetList.Num() > LandscapeEdMode->GetTargetLayerStartingIndex();
			const TArray<TSharedRef<FLandscapeTargetListInfo>> TargetDisplayList = FLandscapeEditorCustomNodeBuilder_TargetLayers::PrepareTargetLayerList(/*bInSort =*/ false, /*bInFilter = */true);
			return (bHasTargetLayers && TargetDisplayList.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	return EVisibility::Collapsed;
}


//////////////////////////////////////////////////////////////////////////

FEdModeLandscape* FLandscapeEditorCustomNodeBuilder_TargetLayers::GetEditorMode()
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

FLandscapeEditorCustomNodeBuilder_TargetLayers::FLandscapeEditorCustomNodeBuilder_TargetLayers(TSharedRef<FAssetThumbnailPool> InThumbnailPool, 
	TSharedRef<IPropertyHandle> InTargetDisplayOrderPropertyHandle, TSharedRef<IPropertyHandle> InTargetDisplayOrderIsAscendingPropertyHandle, TSharedRef<IPropertyHandle> InTargetShowUnusedLayersPropertyHandle)
	: ThumbnailPool(InThumbnailPool)
	, TargetDisplayOrderPropertyHandle(InTargetDisplayOrderPropertyHandle)
	, TargetDisplayOrderIsAscendingPropertyHandle(InTargetDisplayOrderIsAscendingPropertyHandle)
	, TargetShowUnusedLayersPropertyHandle(InTargetShowUnusedLayersPropertyHandle)
{
}

FLandscapeEditorCustomNodeBuilder_TargetLayers::~FLandscapeEditorCustomNodeBuilder_TargetLayers()
{
	FEdModeLandscape::TargetsListUpdated.RemoveAll(this);
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::SetOnRebuildChildren(FSimpleDelegate InOnRegenerateChildren)
{
	FEdModeLandscape::TargetsListUpdated.RemoveAll(this);
	if (InOnRegenerateChildren.IsBound())
	{
		FEdModeLandscape::TargetsListUpdated.Add(InOnRegenerateChildren);
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::GenerateHeaderRowContent(FDetailWidgetRow& NodeRow)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	
	if ((LandscapeEdMode == nullptr) || (LandscapeEdMode->CurrentToolMode == nullptr))
	{
		return;	
	}

	NodeRow.NameWidget
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("LayersLabel", "Layers"))
		];

	if (EnumHasAllFlags(LandscapeEdMode->CurrentToolMode->SupportedTargetTypes, ELandscapeToolTargetTypeFlags::Weightmap))
	{
		NodeRow.ValueWidget
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1) // Fill the entire width if possible
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.MinDesiredWidth(125.f)
				.Text(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetNumWeightmapTargetLayersText)
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 1.0f, 0.0f, 1.0f)
			[
				PropertyCustomizationHelpers::MakeAddButton(FSimpleDelegate::CreateRaw(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCreateLayer), NSLOCTEXT("Landscape", "CreateLayer", "Create Layer"))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 1.0f, 0.0f, 1.0f)
			[
				SNew(SButton)
				.ButtonStyle( FAppStyle::Get(), "SimpleButton" )
				.ToolTipText_Lambda([this]() { return HasUnassignedTargetLayers() ? LOCTEXT("TargetLayerCreateFromMaterialsToolTip", "Create Layers From Assigned Materials") : LOCTEXT("TargetLayerCreateFromMaterialsDisabledToolTip", "All Material Layers Created"); })
				.OnClicked(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCreateLayersFromMaterials)
				.IsEnabled(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HasUnassignedTargetLayers)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LandscapeEditor.Layer.Sync"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.0f, 1.0f, 0.0f, 1.0f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("TargetLayerAutoFillLayers", "Auto-Fill Target Layer Assets"))
				.OnClicked(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShowAutoFillTargetLayerDialog)
				.IsEnabled(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HasWeightmapTargetLayers)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("LandscapeEditor.Layer.AutoFill"))
				]
			]
		];
	}
}

FText FLandscapeEditorCustomNodeBuilder_TargetLayers::GetNumWeightmapTargetLayersText() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	ALandscape* Landscape = LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr;
	if (Landscape)
	{
		return FText::Format(LOCTEXT("NumTargetLayersText", "{0} Target {0}|plural(one=Layer, other=Layers)"), GetWeightmapTargetLayerCount());
	}

	return FText();
}

const TArray<const TSharedRef<FLandscapeTargetListInfo>> FLandscapeEditorCustomNodeBuilder_TargetLayers::GetUnassignedTargetLayersFromMaterial() const
{
	TArray<const TSharedRef<FLandscapeTargetListInfo>> TargetLayerList;

	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		TWeakObjectPtr<ALandscape> LandscapeActor = LandscapeEdMode->CurrentToolTarget.LandscapeInfo->LandscapeActor;

		if (!LandscapeActor.Get())
		{
			return TargetLayerList;
		}

		TSet<FName> LayerNames;
		LandscapeActor->GetLandscapeInfo()->ForEachLandscapeProxy([&LayerNames](ALandscapeProxy* Proxy)
			{
				LayerNames.Append(Proxy->RetrieveTargetLayerNamesFromMaterials(/*bIncludeVisibilityLayer =*/false));
				return true;
			});

		for (const FName& LayerName : LayerNames)
		{
			if (!LandscapeActor->GetTargetLayers().Find(LayerName))
			{
				const FLandscapeInfoLayerSettings LayerSettings = FLandscapeInfoLayerSettings(LayerName, LandscapeActor.Get());
				const TSharedRef<FLandscapeTargetListInfo> Target = MakeShareable(new FLandscapeTargetListInfo(FText::FromName(LayerName), ELandscapeToolTargetType::Weightmap, LayerSettings, LandscapeEdMode->GetSelectedEditLayerIndex(), /* bIsLayerReferencedByMaterial=*/ true));

				TargetLayerList.Add(Target);
			}
		}
	}
	
	return TargetLayerList;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::HasUnassignedTargetLayers() const
{
	return !GetUnassignedTargetLayersFromMaterial().IsEmpty();
}

FReply FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCreateLayersFromMaterials() const
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeTargetLayer_CreateFromMaterials", "Create Target Layers from Assigned materials"));
	
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		TWeakObjectPtr<ALandscape> LandscapeActor = LandscapeEdMode->CurrentToolTarget.LandscapeInfo->LandscapeActor;

		if (!LandscapeActor.Get())
		{
			return FReply::Handled();
		}

		const TArray<const TSharedRef<FLandscapeTargetListInfo>> TargetLayerList = GetUnassignedTargetLayersFromMaterial();

		if (!TargetLayerList.IsEmpty())
		{
			for (const TSharedRef<FLandscapeTargetListInfo>& TargetLayer : TargetLayerList)
			{
				// The user may have created a new layer in the dialog, only add layers that are not yet assigned
				if (!LandscapeActor->GetTargetLayers().Find(TargetLayer->GetLayerName()))
				{
					LandscapeActor->AddTargetLayer(TargetLayer->GetLayerName(), FLandscapeTargetLayerSettings(TargetLayer->LayerInfoObj.Get()));
				}
			}

			LandscapeEdMode->GetLandscape()->GetLandscapeInfo()->UpdateLayerInfoMap();
			LandscapeEdMode->UpdateTargetList();
		}
	}	 
	
	return FReply::Handled();
}


FReply FLandscapeEditorCustomNodeBuilder_TargetLayers::ShowAutoFillTargetLayerDialog() const
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	check(LandscapeEdMode);
	
	bool bIsCreateNewTargetLayersChecked = false;

	const TSharedPtr<SCustomDialog> DialogWindow = SNew(SCustomDialog)
		.Title(LOCTEXT("TargetLayerAutoFillLayers", "Auto-Fill Target Layer Assets"))
		.HAlignButtonBox(HAlign_Center)
		.Buttons({
			SCustomDialog::FButton(
				LOCTEXT("Unassigned Layers Only", "Unassigned Layers Only"), 
				/*InOnClicked = */FSimpleDelegate::CreateLambda([this, &bIsCreateNewTargetLayersChecked]()
						{
							HandleAutoFillTargetLayers(/*bUpdateAllLayers = */false, bIsCreateNewTargetLayersChecked);
						}),
					SCustomDialog::EButtonRole::Confirm)
					.SetPrimary(true),
			SCustomDialog::FButton(
				LOCTEXT("All Layers", "All Layers"),
				/*InOnClicked = */FSimpleDelegate::CreateLambda([this, &bIsCreateNewTargetLayersChecked]()
						{
							HandleAutoFillTargetLayers(/*bUpdateAllLayers = */true, bIsCreateNewTargetLayersChecked);
						})),
			SCustomDialog::FButton(LOCTEXT("Cancel", "Cancel"), /*InOnClicked = */FSimpleDelegate(), SCustomDialog::EButtonRole::Cancel)
			})
		.ContentAreaPadding(10.0f)
		.Content()
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
						.WrapTextAt(512.0f)
						.Text(FText::Format(LOCTEXT("TargetLayerAutoFillDialog", "This operation will assign the layer info assets found within the default asset folder {0} to the landscape's target layers."),
							FText::FromString(LandscapeEdMode->GetTargetLayerAssetPackagePath())))
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 10.0f, 0, 0)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.IsChecked_Lambda([&bIsCreateNewTargetLayersChecked]() { return bIsCreateNewTargetLayersChecked ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
							.OnCheckStateChanged_Lambda([&bIsCreateNewTargetLayersChecked](ECheckBoxState InState) { bIsCreateNewTargetLayersChecked = (InState == ECheckBoxState::Checked); })
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
								.Text(LOCTEXT("TargetLayerAutoFillDialog_CreateNew", "Create new assets in the default folder if none are found."))
						]
				]
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(0, 20.0f, 0, 0) // Create spacing between details and prompt
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TargetLayerAutoFillDialog_Prompt", "Please specify which target layers should be set."))
				]
		];

	DialogWindow->ShowModal();

	return FReply::Handled();
}

FReply FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleAutoFillTargetLayers(const bool bUpdateAllLayers, const bool bCreateNewTargetLayers) const
{
	FScopedTransaction Transaction(LOCTEXT("LandscapeTargetLayer_AutoFillTargetLayers", "Auto-Fill Target Layer Assets"));

	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode == nullptr)
	{
		return FReply::Handled();
	}

	for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : PrepareTargetLayerList(/*bInSort = */true, /*bInFilter = */false))
	{
		// Auto fill unassigned weightmap target layers or all target layers when flag is true
		if (TargetInfo->TargetType == ELandscapeToolTargetType::Weightmap && (!TargetInfo->LayerInfoObj.IsValid() || bUpdateAllLayers))
		{
			const FString TargetLayerAssetFilePath = LandscapeEdMode->GetTargetLayerAssetPackagePath();

			if (const TOptional<FAssetData> AssetData = LandscapeEditorUtils::FindLandscapeTargetLayerInfoAsset(TargetInfo->LayerName, TargetLayerAssetFilePath);
				AssetData.IsSet())
			{
				OnTargetLayerSetObject(*AssetData, TargetInfo);
			}
			else if (bCreateNewTargetLayers)
			{
				FName FileName;
				const FString PackageName = UE::Landscape::GetLayerInfoObjectPackageName(TargetInfo->LayerName, TargetLayerAssetFilePath, FileName);

				CreateTargetLayerInfoAsset(TargetInfo, TargetLayerAssetFilePath, FileName.ToString());
			}
		}
	}

	return FReply::Handled();
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCreateLayer()
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		TWeakObjectPtr<ALandscape> Landscape = LandscapeEdMode->CurrentToolTarget.LandscapeInfo->LandscapeActor;

		if (!Landscape.Get())
		{
			return;
		}

		FScopedTransaction Transaction(LOCTEXT("LandscapeTargetLayer_Create", "Create a Target Layer"));
		// TODO [jared.ritchie] - AddTargetLayer returns a FLandscapeTargetLayerSettings which does not give
		// any context about the newly created LayerName, position, etc. Deprecate AddTargetLayer
		// and use the return of new function to simplify setting the PendingRenameTargetLayerIndex
		const FName& NewTargetLayerName = Landscape->GenerateUniqueTargetLayerName();
		Landscape->AddTargetLayer(NewTargetLayerName, FLandscapeTargetLayerSettings());

		LandscapeEdMode->CurrentToolTarget.LandscapeInfo->UpdateLayerInfoMap();
		LandscapeEdMode->UpdateTargetList();

		// Trigger a rename for the new layer, enters edit mode on next tick
		const int32 NewTargetLayerIndex = LandscapeEdMode->GetTargetDisplayOrderList()->Find(NewTargetLayerName);
		LandscapeEdMode->PendingRenameTargetLayerIndex = NewTargetLayerIndex - LandscapeEdMode->GetTargetLayerStartingIndex();

		LandscapeEdMode->RefreshDetailPanel();

		// Auto-select new target layer in details panel
		if (LandscapeEdMode->GetTargetList().IsValidIndex(NewTargetLayerIndex))
		{
			OnTargetSelectionChanged(LandscapeEdMode->GetTargetList()[NewTargetLayerIndex]);
		}
	}
}

TSharedRef<SWidget> FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerDisplayOrderButtonMenuContent()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/ true);

	MenuBuilder.BeginSection("TargetLayerSortBy", LOCTEXT("SortByHeading", "Sort By"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderDefault", "Default"),
			LOCTEXT("TargetLayerDisplayOrderDefaultToolTip", "Sort using order defined in the material."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetSelectedDisplayOrder, ELandscapeLayerDisplayMode::Default),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsSelectedDisplayOrder, ELandscapeLayerDisplayMode::Default)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderAlphabetical", "Alphabetical"),
			LOCTEXT("TargetLayerDisplayOrderAlphabeticalToolTip", "Sort using alphabetical order."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetSelectedDisplayOrder, ELandscapeLayerDisplayMode::Alphabetical),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsSelectedDisplayOrder, ELandscapeLayerDisplayMode::Alphabetical)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderBlendType", "By Blend Method"),
			LOCTEXT("TargetLayerDisplayOrderBlendTypeToolTip", "Sort using the blending method."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetSelectedDisplayOrder, ELandscapeLayerDisplayMode::ByBlendMethod),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsSelectedDisplayOrder, ELandscapeLayerDisplayMode::ByBlendMethod)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderCustom", "Custom"),
			LOCTEXT("TargetLayerDisplayOrderCustomToolTip", "This sort options will be set when changing manually display order by dragging layers"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetSelectedDisplayOrder, ELandscapeLayerDisplayMode::UserSpecific),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsSelectedDisplayOrder, ELandscapeLayerDisplayMode::UserSpecific)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("TargetLayerSortType", LOCTEXT("SortTypeHeading", "Sort Type"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderAscending", "Ascending"),
			LOCTEXT("TargetLayerDisplayOrderAscendingToolTip", "Sort the items in Ascending order"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetAscendingDisplayOrder, true),
				FCanExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::CanChangeDisplayOrderSortType),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsAscendingDisplayOrder)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerDisplayOrderDescending", "Descending"),
			LOCTEXT("TargetLayerDisplayOrderDescendingToolTip", "Sort the items in Descending order"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::SetAscendingDisplayOrder, false),
				FCanExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::CanChangeDisplayOrderSortType),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::IsDescendingDisplayOrder)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerShowUnusedButtonMenuContent()
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/true, nullptr, nullptr, /*bCloseSelfOnly=*/ true);

	MenuBuilder.BeginSection("TargetLayerUnusedType", LOCTEXT("UnusedTypeHeading", "Layer Visibility"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerShowUnusedLayer", "Show all layers"),
			LOCTEXT("TargetLayerShowUnusedLayerToolTip", "Show all layers"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShowUnusedLayers, true),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowUnusedLayers, true)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("TargetLayerHideUnusedLayer", "Hide unused layers"),
			LOCTEXT("TargetLayerHideUnusedLayerToolTip", "Only show used layer"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShowUnusedLayers, false),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowUnusedLayers, false)
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_TargetLayers::GetShowUnusedBrush() const
{
	const FSlateBrush* Brush = FAppStyle::GetBrush("Level.VisibleIcon16x");
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->UISettings && !LandscapeEdMode->UISettings->ShowUnusedLayers)
	{
		Brush = FAppStyle::GetBrush("Level.NotVisibleIcon16x");
	}
	return Brush;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::ShowUnusedLayers(bool Result)
{
	TargetShowUnusedLayersPropertyHandle->SetValue(Result);
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowUnusedLayers(bool Result) const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		return LandscapeEdMode->UISettings->ShowUnusedLayers == Result;
	}

	return false;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::SetSelectedDisplayOrder(ELandscapeLayerDisplayMode InDisplayOrder)
{
	TargetDisplayOrderPropertyHandle->SetValue((uint8)InDisplayOrder);	
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::SetAscendingDisplayOrder(bool bInIsAscending)
{
	TargetDisplayOrderIsAscendingPropertyHandle->SetValue(bInIsAscending);
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::CanChangeDisplayOrderSortType() const 
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->UISettings->TargetDisplayOrder != ELandscapeLayerDisplayMode::UserSpecific;
	}

	return false;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::IsSelectedDisplayOrder(ELandscapeLayerDisplayMode InDisplayOrder) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->UISettings->TargetDisplayOrder == InDisplayOrder;
	}

	return false;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::IsAscendingDisplayOrder() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return LandscapeEdMode->UISettings->bTargetDisplayOrdersAscending;
	}

	return false;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::IsDescendingDisplayOrder() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		return !LandscapeEdMode->UISettings->bTargetDisplayOrdersAscending;
	}

	return false;
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerDisplayOrderBrush() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		auto AppendDisplayOrderSuffix = [bTargetDisplayOrdersAscending = LandscapeEdMode->UISettings->bTargetDisplayOrdersAscending](const TCHAR* InBaseBrushName) -> FName
			{
				return FName(FString::Format(TEXT("{0}.{1}"), { InBaseBrushName, bTargetDisplayOrdersAscending ? TEXT("Ascending") : TEXT("Descending")}));
			};

		switch (LandscapeEdMode->UISettings->TargetDisplayOrder)
		{
			case ELandscapeLayerDisplayMode::Default: return FAppStyle::Get().GetBrush(AppendDisplayOrderSuffix(TEXT("LandscapeEditor.Target_DisplayOrder.Default")));
			case ELandscapeLayerDisplayMode::Alphabetical: return FAppStyle::Get().GetBrush(AppendDisplayOrderSuffix(TEXT("LandscapeEditor.Target_DisplayOrder.Alphabetical")));
			case ELandscapeLayerDisplayMode::UserSpecific: return FAppStyle::Get().GetBrush("LandscapeEditor.Target_DisplayOrder.Custom");
			case ELandscapeLayerDisplayMode::ByBlendMethod: return FAppStyle::Get().GetBrush(AppendDisplayOrderSuffix(TEXT("LandscapeEditor.Target_DisplayOrder.ByBlendMethod")));
			default:
				checkNoEntry();
		}
	}

	return nullptr;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowLayer(TSharedRef<FLandscapeTargetListInfo> Target) const
{
	if ((Target->TargetType == ELandscapeToolTargetType::Weightmap)
		|| (Target->TargetType == ELandscapeToolTargetType::Invalid)) // Invalid means weightmap with no selected target layer
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();

		if (LandscapeEdMode != nullptr)
		{
			return LandscapeEdMode->ShouldShowLayer(Target) ? EVisibility::Visible : EVisibility::Collapsed;
		}
	}

	return EVisibility::Visible;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFilterTextChanged(const FText& InFilterText)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		LandscapeEdMode->UISettings->TargetLayersFilterString = InFilterText.ToString();
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared)
	{
		LayersFilterSearchBox->SetText(FText::GetEmpty());
		OnFilterTextChanged(FText::GetEmpty());
		FSlateApplication::Get().ClearKeyboardFocus(EFocusCause::Cleared);
	}
}

int32 FLandscapeEditorCustomNodeBuilder_TargetLayers::GetWeightmapTargetLayerCount() const
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolMode)
	{
		// ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" so we still want to display that property if it has been marked to be displayed in Weightmap target type, to be consistent 
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid))
		{
			return Algo::CountIf(LandscapeEdMode->GetTargetList(), [](TSharedRef<FLandscapeTargetListInfo> InInfo)
				{
					FName LayerName = InInfo->GetLayerName();
					return (LayerName != NAME_None) && (LayerName != UMaterialExpressionLandscapeVisibilityMask::ParameterName);
				});
		}
	}

	return 0;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::HasWeightmapTargetLayers() const
{
	return GetWeightmapTargetLayerCount() > 0;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersFilterVisibility() const
{
	return HasWeightmapTargetLayers() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersDisplayOptionsVisibility() const
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->CurrentToolMode)
	{
		// ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" so we still want to display that property if it has been marked to be displayed in Weightmap target type, to be consistent 
		if ((LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Weightmap)
			|| (LandscapeEdMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Invalid))
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Collapsed;
}

FText FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersFilterText() const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		return FText::FromString(LandscapeEdMode->UISettings->TargetLayersFilterString);
	}

	return FText();
}

TArray<TSharedRef<FLandscapeTargetListInfo>> FLandscapeEditorCustomNodeBuilder_TargetLayers::PrepareTargetLayerList(bool bInSort, bool bInFilter)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode == nullptr)
	{
		return {};
	}
	const TArray<TSharedRef<FLandscapeTargetListInfo>>& TargetList = LandscapeEdMode->GetTargetList();
	const TArray<FName>* TargetDisplayOrderList = LandscapeEdMode->GetTargetDisplayOrderList();
	if (TargetDisplayOrderList == nullptr)
	{
		return {};
	}

	TArray<TSharedRef<FLandscapeTargetListInfo>> FinalList(TargetList);
	if (bInFilter)
	{
		FinalList.RemoveAllSwap([LandscapeEdMode](TSharedRef<FLandscapeTargetListInfo> InTargetInfo)
			{
				return !LandscapeEdMode->ShouldShowLayer(InTargetInfo);
			});
	}

	if (bInSort)
	{
		Algo::SortBy(FinalList, [TargetDisplayOrderList](TSharedRef<FLandscapeTargetListInfo> InTargetInfo)
			{
				return TargetDisplayOrderList->Find(InTargetInfo->GetLayerName());
			});
	}
	return FinalList;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FLandscapeEditorCustomNodeBuilder_TargetLayers::GenerateChildContent(IDetailChildrenBuilder& ChildrenBuilder)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		TSharedPtr<SDragAndDropVerticalBox> TargetLayerList = SNew(SDragAndDropVerticalBox)
			.OnCanAcceptDropAdvanced(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCanAcceptDrop)
			.OnAcceptDrop(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleAcceptDrop)
			.OnDragDetected(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleDragDetected);

		TargetLayerList->SetDropIndicator_Above(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Above"));
		TargetLayerList->SetDropIndicator_Below(*FAppStyle::GetBrush("LandscapeEditor.TargetList.DropZone.Below"));

		ChildrenBuilder.AddCustomRow(LOCTEXT("LayersLabel", "Layers"))
			.Visibility(EVisibility::Visible)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Fill)
					.Padding(FMargin(0.0f, 2.0f, 0.0f, 2.0f))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SComboButton)
							.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
							.ForegroundColor(FSlateColor::UseForeground())
							.HasDownArrow(true)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.ToolTipText(LOCTEXT("TargetLayerSortButtonTooltip", "Define how we want to sort the displayed layers"))
							.OnGetMenuContent(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerDisplayOrderButtonMenuContent)
							.IsEnabled(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HasWeightmapTargetLayers)
							.Visibility(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersDisplayOptionsVisibility)
							.ButtonContent()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.WidthOverride(16.0f)
									.HeightOverride(16.0f)
									[
										SNew(SImage)
										.Image(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerDisplayOrderBrush)
									]
								]
							]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(4.0f, 1.0f, 0.0f, 1.0f)
						[
							SNew(SComboButton)
							.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon")
							.ForegroundColor(FSlateColor::UseForeground())
							.HasDownArrow(true)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.ToolTipText(LOCTEXT("TargetLayerUnusedLayerButtonTooltip", "Define if we want to display unused layers"))
							.OnGetMenuContent(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerShowUnusedButtonMenuContent)
							.IsEnabled(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::HasWeightmapTargetLayers)
							.Visibility(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersDisplayOptionsVisibility)
							.ButtonContent()
							[
								SNew(SHorizontalBox)
								+ SHorizontalBox::Slot()
								.AutoWidth()
								.VAlign(VAlign_Center)
								[
									SNew(SBox)
									.WidthOverride(16.0f)
									.HeightOverride(16.0f)
									[
										SNew(SImage)
										.Image(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetShowUnusedBrush)
									]
								]
							]
						]

						+ SHorizontalBox::Slot()
						//.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Fill)
						.Padding(5.0f, 0.0f, 0.0f, 0.0f)
						[
							SAssignNew(LayersFilterSearchBox, SSearchBox)
							.InitialText(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersFilterText)
							.SelectAllTextWhenFocused(true)
							.HintText(LOCTEXT("LayersSearch", "Filter Target Layers"))
							.OnTextChanged(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFilterTextChanged)
							.OnTextCommitted(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFilterTextCommitted)
							.Visibility(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersFilterVisibility)
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Top)
					.HAlign(HAlign_Fill)
					.Padding(2)
					[
						TargetLayerList.ToSharedRef()
					]
			];

		// Generate a row for all target layers, including those that will be filtered and let the row's visibility lambda to compute their visibility dynamically. This allows 
		//  filtering to work without refreshing the details panel (which causes the search box to lose focus) :
		InlineTextBlocks.Empty();
		InlineTextBlocks.Reserve(GetWeightmapTargetLayerCount());

		for (const TSharedRef<FLandscapeTargetListInfo>& TargetInfo : PrepareTargetLayerList(/*bInSort = */true, /*bInFilter = */false))
		{
			TSharedPtr<SWidget> GeneratedRowWidget = GenerateRow(TargetInfo);
			if (GeneratedRowWidget.IsValid())
			{
				TargetLayerList->AddSlot()
					.AutoHeight()
					[
						GeneratedRowWidget.ToSharedRef()
					];
			}
		}
	}
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FLandscapeEditorCustomNodeBuilder_TargetLayers::Tick(float DeltaTime)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && LandscapeEdMode->PendingRenameTargetLayerIndex != INDEX_NONE)
	{
		OnRenameLayer(LandscapeEdMode->PendingRenameTargetLayerIndex);
		LandscapeEdMode->PendingRenameTargetLayerIndex = INDEX_NONE;
	}
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_TargetLayers::GenerateRow(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	TSharedPtr<SWidget> RowWidget;

	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); LandscapeEdMode && LandscapeEdMode->CurrentTool && LandscapeEdMode->CurrentToolMode)
	{
		ELandscapeToolTargetTypeFlags SupportedTargetTypes = UE::Landscape::GetLandscapeToolTargetTypeAsFlags(LandscapeEdMode->CurrentTool->GetSupportedTargetType())
			& LandscapeEdMode->CurrentToolMode->SupportedTargetTypes
			& UE::Landscape::GetLandscapeToolTargetTypeAsFlags(Target->TargetType);
		if (SupportedTargetTypes == ELandscapeToolTargetTypeFlags::None)
		{
			return RowWidget;
		}
	}

	if (Target->TargetType != ELandscapeToolTargetType::Weightmap)
	{
		RowWidget = SNew(SLandscapeEditorSelectableBorder)
		.Padding(0.0f)
		.VAlign(VAlign_Center)
		.OnContextMenuOpening(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerContextMenuOpening, Target, static_cast<int32>(INDEX_NONE))
		.OnSelected_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetSelectionChanged, Target)
		.IsSelected_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerIsSelected, Target)
		.Visibility(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowLayer, Target)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(FMargin(2))
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				[
					SNew(SLandscapeAssetThumbnail, /*Asset = */ nullptr, ThumbnailPool, Target->TargetType == ELandscapeToolTargetType::Heightmap ? FName("LandscapeEditor.Target_Heightmap") : FName("LandscapeEditor.Target_Visibility"))
					.ThumbnailSize(FIntPoint(48, 48))
				]

				// Bottom-right corner text indicating how the current tool will affect the target layer
				+ SOverlay::Slot()
				.Padding(2)
				.VAlign(VAlign_Bottom)
				.HAlign(HAlign_Right)
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::GetBrush("AssetThumbnail", ".AssetThumbnailStatusBar"))
					[
						SNew(SImage)
						.Image_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerActionTypeBrush, Target)
						.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
					]
				]
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4, 0)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.VAlign(VAlign_Center)
				.Padding(0, 2)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(Target->TargetLayerDisplayName)
					.ColorAndOpacity_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetTextColor, Target)
				]
			]
		];
	}
	else
	{
		InlineTextBlocks.Add(TSharedPtr<SInlineEditableTextBlock>());

		ULandscapeLayerInfoObject* LayerInfo = Target->LayerInfoObj.Get();
		UMaterialInstanceConstant* ThumbnailMIC = Target->ThumbnailMIC.Get();
		UObject* ThumbnailObject = nullptr;

		if (Target->bIsLayerReferencedByMaterial)
		{
			// If the layer info is valid and has a custom thumbnail, it's considered as an override, so use this as the thumbnail object : 
			ThumbnailObject = ((LayerInfo != nullptr) && ThumbnailTools::AssetHasCustomThumbnail(LayerInfo->GetFullName())) ? static_cast<UObject*>(LayerInfo) : static_cast<UObject*>(ThumbnailMIC);
		}


		TSharedRef<SWidget> DeleteButton = PropertyCustomizationHelpers::MakeDeleteButton(
			FSimpleDelegate::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerDeleteClicked, Target),
			LOCTEXT("Tooltip_Delete", "Delete Layer"));

		TSharedRef<SWidget> CreateButton = PropertyCustomizationHelpers::MakeCustomButton(
			FAppStyle::GetBrush(TEXT("Icons.Plus")),
			FSimpleDelegate::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerCreateClicked, Target),
			LOCTEXT("Tooltip_Create", "Create Layer Info"),
			TAttribute<bool>::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerCreateEnabled, Target));

		// The tooltip mimics the asset thumbnail tooltips since we're using SLandscapeAssetThumbnail. It's displaying information about the layer info, 
		//  especially its blend group
		TSharedRef<SToolTip> TargetLayerToolTip = 
			SNew(SToolTip)
			.TextMargin(FMargin(1.f, -3.f))
			.BorderImage(FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border"))
			.Visibility((LayerInfo != nullptr) ? EVisibility::Visible : EVisibility::Collapsed)
			.Content()
			[
				SNew(SDocumentationToolTip)
				[
					SNew(SVerticalBox)

					// Layer name
					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 6.f)
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(Target->TargetLayerDisplayName)
						.Font(FAppStyle::GetFontStyle("AssetThumbnail.Font"))
						.ColorAndOpacity(FStyleColors::White)
					]

					// Separator
					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 6.f)
					.AutoHeight()
					[
						SNew(SSeparator)
						.Orientation(Orient_Horizontal)
						.Thickness(1.f)
						.ColorAndOpacity(COLOR("#484848FF"))
						.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
					]

					// Blend Method
					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 6.f)
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 0, 4, 0)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BlendMethodHeader", "Blend Method:"))
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.ColorAndOpacity(FStyleColors::White)
							.Text_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendMethodText, Target, /*bInIncludeBlendGroup = */true)
						]
					]

					// Blend Group (if applicable)
					+ SVerticalBox::Slot()
					.Padding(0.f, 0.f, 0.f, 6.f)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripVisibility, Target)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 4.f, 0.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("BlendGroupHeader", "Blend Group:"))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0.f, 0.f, 4.f, 0.f)
						.VAlign(VAlign_Center)
						[
							// Display a color box to explain the color in the thumbnail color strip
							SNew(SColorBlock)
							.Color_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripColor, Target)
							.Size(12.f)
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(STextBlock)
							.ColorAndOpacity(FStyleColors::White)
							.Text_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendGroupText, Target, /*bInAllowNoName = */true)
						]
					]
				]
			];

		RowWidget = 
			SNew(SLandscapeEditorSelectableBorder)
			.Padding(0.0f)
			.VAlign(VAlign_Center)
			.OnContextMenuOpening(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerContextMenuOpening, Target, InlineTextBlocks.Num() - 1)
			.OnSelected_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetSelectionChanged, Target)
			.IsSelected_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerIsSelected, Target)
			.Visibility(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldShowLayer, Target)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SBox)
					.Padding(FMargin(2.0f, 0.0f, 2.0f, 0.0f))
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicator"))
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(2))
				[
					SNew(SBox)
					.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeLayerUsageVisibility, Target)
					.WidthOverride(48.0f)
					.HeightOverride(48.0f)
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("WhiteBrush"))
						.ColorAndOpacity_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayerUsageDebugColor, Target)
					]
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(2))
				[
					SNew(SOverlay)
					.ToolTip(TargetLayerToolTip)
					+ SOverlay::Slot()
					[
						SNew(SLandscapeAssetThumbnail, ThumbnailObject, ThumbnailPool, Target->bIsLayerReferencedByMaterial ? FName("LandscapeEditor.Target_Weightmap") : FName("LandscapeEditor.Target_Unknown"))
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeLayerUsageVisibility_Invert, Target)
						.ThumbnailSize(FIntPoint(48, 48))
						// Open landscape layer info asset on double-click on the thumbnail : 
						.OnAccessAsset_Lambda([Target](UObject* InObject)
						{ 
							// Note : the object being returned here is the landscape MIC so it's not what we use for opening the landscape layer info asset : 
							if ((Target->TargetType == ELandscapeToolTargetType::Weightmap) && (Target->LayerInfoObj != nullptr))
							{
								UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
								return AssetEditorSubsystem->OpenEditorForAsset(Target->LayerInfoObj.Get());
							}
							return false;
						})
					]

					// Mimic the asset color strip to show a color for the blend group :
					+ SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Bottom)
					[
						SNew(SColorBlock)
						.Color_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripColor, Target)
						.Size(2.f)
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripVisibility, Target)
					]

					// Bottom-right corner text indicating how the current tool will affect the target layer
					+ SOverlay::Slot()
					.Padding(2)
					.VAlign(VAlign_Bottom)
					.HAlign(HAlign_Right)
					[
						SNew(SBorder)
						.BorderImage(FAppStyle::GetBrush("AssetThumbnail", ".AssetThumbnailStatusBar"))
						[
							SNew(SImage)
							.Image_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerActionTypeBrush, Target)
							.DesiredSizeOverride(FVector2D(12.0f, 12.0f))
						]
					]
				]
				
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4, 0)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					.Padding(4, 3, 0, 3)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						[
							SAssignNew(InlineTextBlocks.Last(), SInlineEditableTextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFontBold())
							.Text(Target->TargetLayerDisplayName)
							.ColorAndOpacity_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetTextColor, Target)
							.OnVerifyTextChanged_Lambda([Target](const FText& InNewText, FText& OutErrorMessage)
							{
								const FName NewName(InNewText.ToString());

								if (Target->LayerName == NewName)
								{
									return true;
								}

								if (NewName == UMaterialExpressionLandscapeVisibilityMask::ParameterName)
								{
									OutErrorMessage = LOCTEXT("LandscapeTargetLayer_RenameFailed_ReservedName", "This target layer name is reserved for internal usage");
									return false;
								}

								if (NewName.IsNone())
								{
									OutErrorMessage = LOCTEXT("LandscapeTargetLayer_RenameFailed_EmptyName", "Target layer name cannot be empty");
									return false;
								}

								ALandscape* Landscape = Cast<ALandscape>(Target->Owner);
								if (Landscape->HasTargetLayer(NewName))
								{
									OutErrorMessage = LOCTEXT("LandscapeTargetLayer_RenameFailed_AlreadyExists", "This target layer name already exists");
									return false;
								}

								return true;
							})
							.OnTextCommitted_Lambda([Target](const FText& Text, ETextCommit::Type Type)
							{
								const FName NewName(Text.ToString());
								if (Target->LayerName == NewName)
								{
									return;
								}

								FScopedTransaction Transaction(LOCTEXT("LandscapeTargetLayer_Rename", "Rename Target Layer"));
								ALandscape* Landscape = Cast<ALandscape>(Target->Owner);
												
								const TMap<FName, FLandscapeTargetLayerSettings>& TargetLayers = Landscape->GetTargetLayers();
								Landscape->RemoveTargetLayer(FName(Target->TargetLayerDisplayName.ToString()));
											
								Target->TargetLayerDisplayName = Text;
								Target->LayerName = FName(Text.ToString());
								Landscape->AddTargetLayer(Target->LayerName, FLandscapeTargetLayerSettings());	
												
								Target->LandscapeInfo->UpdateLayerInfoMap();
								if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
								{
									LandscapeEdMode->UpdateTargetList();
								}
							})
						]
						+ SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							SNew(STextBlock)
							.Font(IDetailLayoutBuilder::GetDetailFont())
							.Text_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendMethodText, Target, /*bInIncludeBlendGroup = */true)
							.ToolTipText_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendMethodTooltipText, Target)
							.ColorAndOpacity_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetTextColor, Target)
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					.VAlign(VAlign_Center)
					[
						SNew(SHorizontalBox)
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerInfoSelectorVisibility, Target)
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.VAlign(VAlign_Center)
						[
							SNew(SObjectPropertyEntryBox)
							.ObjectPath(Target->LayerInfoObj != nullptr ? Target->LayerInfoObj->GetPathName() : FString())
							.AllowedClass(ULandscapeLayerInfoObject::StaticClass())
							.OnObjectChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerSetObject, Target)
							.OnShouldFilterAsset_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldFilterLayerInfo, Target->LayerName)
							.AllowCreate(false)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							CreateButton
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							DeleteButton
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersSubstractiveBlendVisibility, Target)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 2, 2, 2)
						[
							SNew(SCheckBox)
							.IsChecked_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::IsLayersSubstractiveBlendChecked, Target)
							.OnCheckStateChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnLayersSubstractiveBlendChanged, Target)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("SubtractiveBlend", "Subtractive Blend"))
								.ToolTipText(LOCTEXT("SubtractiveBlendTooltip", "If enabled, this target layer's effect on this edit layer will be inverted, "
									"allowing a given edit layer to effectively decrease the target layer's weight from edit layers underneath."))
								.ColorAndOpacity_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetTextColor, Target)
							]
						]
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeColorChannelVisibility, Target)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(0, 2, 2, 2)
						[
							SNew(SCheckBox)
							.IsChecked_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::DebugModeColorChannelIsChecked, Target, 0)
							.OnCheckStateChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnDebugModeColorChannelChanged, Target, 0)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ViewMode.Debug_None", "None"))
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						[
							SNew(SCheckBox)
							.IsChecked_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::DebugModeColorChannelIsChecked, Target, 1)
							.OnCheckStateChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnDebugModeColorChannelChanged, Target, 1)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ViewMode.Debug_R", "R"))
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						[
							SNew(SCheckBox)
							.IsChecked_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::DebugModeColorChannelIsChecked, Target, 2)
							.OnCheckStateChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnDebugModeColorChannelChanged, Target, 2)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ViewMode.Debug_G", "G"))
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2)
						[
							SNew(SCheckBox)
							.IsChecked_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::DebugModeColorChannelIsChecked, Target, 4)
							.OnCheckStateChanged_Static(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnDebugModeColorChannelChanged, Target, 4)
							[
								SNew(STextBlock)
								.Text(LOCTEXT("ViewMode.Debug_B", "B"))
							]
						]
					]
				]
			];
	}

	return RowWidget;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode != nullptr)
	{
		// The slot index corresponds to what is actually shown, so we need to both sort and filter the target layer list here :
		TArray<TSharedRef<FLandscapeTargetListInfo>> TargetDisplayList = PrepareTargetLayerList(/*bInSort =*/ true, /*bInFilter = */true);
		if (TargetDisplayList.IsValidIndex(SlotIndex))
		{
			if (const TArray<FName>* TargetDisplayOrderList = LandscapeEdMode->GetTargetDisplayOrderList())
			{
				TSharedPtr<SWidget> Row = GenerateRow(TargetDisplayList[SlotIndex]);
				if (Row.IsValid())
				{
					return FReply::Handled().BeginDragDrop(FTargetLayerDragDropOp::New(SlotIndex, Slot, Row));
				}
			}
		}
	}

	return FReply::Unhandled();
}

TOptional<SDragAndDropVerticalBox::EItemDropZone> FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FTargetLayerDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FTargetLayerDragDropOp>();

	if (DragDropOperation.IsValid())
	{
		return DropZone;
	}

	return TOptional<SDragAndDropVerticalBox::EItemDropZone>();
}

FReply FLandscapeEditorCustomNodeBuilder_TargetLayers::HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot)
{
	TSharedPtr<FTargetLayerDragDropOp> DragDropOperation = DragDropEvent.GetOperationAs<FTargetLayerDragDropOp>();

	if (DragDropOperation.IsValid())
	{
		FEdModeLandscape* LandscapeEdMode = GetEditorMode();

		if (LandscapeEdMode != nullptr)
		{
			// The slot index corresponds to what is actually shown, so we need to both sort and filter the target layer list here :
			TArray<TSharedRef<FLandscapeTargetListInfo>> TargetDisplayList = PrepareTargetLayerList(/*bInSort =*/ true, /*bInFilter = */true);

			if (TargetDisplayList.IsValidIndex(DragDropOperation->SlotIndexBeingDragged) && TargetDisplayList.IsValidIndex(SlotIndex))
			{
				const FName TargetLayerNameBeingDragged = TargetDisplayList[DragDropOperation->SlotIndexBeingDragged]->GetLayerName();
				const FName DestinationTargetLayerName = TargetDisplayList[SlotIndex]->GetLayerName();
				if (const TArray<FName>* TargetDisplayOrderList = LandscapeEdMode->GetTargetDisplayOrderList())
				{
					int32 StartingLayerIndex = TargetDisplayOrderList->Find(TargetLayerNameBeingDragged);
					int32 DestinationLayerIndex = TargetDisplayOrderList->Find(DestinationTargetLayerName);
					if (StartingLayerIndex != INDEX_NONE && DestinationLayerIndex != INDEX_NONE)
					{
						LandscapeEdMode->MoveTargetLayerDisplayOrder(StartingLayerIndex, DestinationLayerIndex);
						return FReply::Handled();
					}
				}
			}
		}
	}

	return FReply::Unhandled();
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerIsSelected(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		return
			LandscapeEdMode->CurrentToolTarget.TargetType == Target->TargetType &&
			LandscapeEdMode->CurrentToolTarget.LayerName == Target->LayerName &&
			LandscapeEdMode->CurrentToolTarget.LayerInfo == Target->LayerInfoObj; // may be null
	}

	return false;
}

ELandscapeTargetLayerActionType FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerActionType(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode == nullptr)
	{
		return ELandscapeTargetLayerActionType::None;
	}

	return LandscapeEdMode->GetEstimatedTargetLayerActionType(Target);
}

const FSlateBrush* FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerActionTypeBrush(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	switch (GetTargetLayerActionType(Target))
	{
		case ELandscapeTargetLayerActionType::Normal:
		{
			return FAppStyle::GetBrush("LandscapeEditor.Brush.Action_Regular");
		}
		case ELandscapeTargetLayerActionType::Inverted:
		{
			return FAppStyle::GetBrush("LandscapeEditor.Brush.Action_Inverse");
		}
		default:
			break;
	}

	return nullptr;
}

FLinearColor FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripColor(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FLinearColor Color = FLinearColor::Gray;
	if (Target->LandscapeInfo.IsValid() && Target->LayerInfoObj.IsValid())
	{
		TOptional<FTargetLayerBlendGroup> BlendGroup = Target->LandscapeInfo->GetBlendGroupForTargetLayer(Target->LayerInfoObj.Get());
		if (BlendGroup.IsSet())
		{
			// Generate a random color name based on the blend group's data (in order to have a consistent result for a given blend method / group name)
			const FString BlendGroupName = BlendGroup->GetName().ToString();
			uint32 Hash = FCrc::TypeCrc32<uint32>(static_cast<uint32>(BlendGroup->GetBlendMethod()));
			Hash = FCrc::MemCrc32(*BlendGroupName, BlendGroupName.Len() * sizeof(BlendGroupName[0]), Hash);

			const uint8 H = Hash & 0xFF;
			const uint8 S = 128 + ((Hash >> 8) & 0x7F);
			const uint8 V = 128 + ((Hash >> 16) & 0x7F);
			return FLinearColor::MakeFromHSV8(H, S, V);
		}
	}
	return Color;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerBlendGroupStripVisibility(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	EVisibility Visibility = EVisibility::Collapsed;
	if (Target->LandscapeInfo.IsValid() && Target->LayerInfoObj.IsValid())
	{
		TOptional<FTargetLayerBlendGroup> BlendGroup = Target->LandscapeInfo->GetBlendGroupForTargetLayer(Target->LayerInfoObj.Get());
		if (BlendGroup.IsSet())
		{
			Visibility = EVisibility::Visible;
		}
	}
	return Visibility;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetSelectionChanged(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->CurrentToolTarget.TargetType = Target->TargetType;
		if (Target->TargetType == ELandscapeToolTargetType::Heightmap)
		{
			checkSlow(Target->LayerInfoObj == nullptr);
			LandscapeEdMode->SetCurrentTargetLayer(NAME_None, nullptr);
		}
		else
		{
			LandscapeEdMode->SetCurrentTargetLayer(Target->LayerName, Target->LayerInfoObj);
		}
	}
}

TSharedPtr<SWidget> FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerContextMenuOpening(const TSharedRef<FLandscapeTargetListInfo> Target, const int32 InLayerIndex)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("LandscapeEditorLayerActions", LOCTEXT("LayerContextMenu.Heading", "Layer Actions"));
	{
		if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
		{
			FUIAction LandscapeHeightmapChangeToolsAction = FUIAction(FExecuteAction::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnHeightmapLayerContextMenu, Target));
			MenuBuilder.AddMenuEntry(LOCTEXT("LayerContextMenu.ImportExport", "Import From/Export To File..."),
				LOCTEXT("LayerContextMenu.ImportExportToolTip", "Opens the Landscape Import tool in order to import / export layers from / to external files."), FSlateIcon(), LandscapeHeightmapChangeToolsAction);

			if (Target->TargetType == ELandscapeToolTargetType::Weightmap && InLayerIndex != INDEX_NONE)
			{
				// Rebuild material instances
				FUIAction RebuildAction = FUIAction(FExecuteAction::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnRebuildMICs, Target));
				MenuBuilder.AddMenuEntry(LOCTEXT("LayerContextMenu.Rebuild", "Rebuild Materials"), LOCTEXT("LayerContextMenu.Rebuild_Tooltip", "Rebuild material instances used for this landscape."), FSlateIcon(), RebuildAction);

				// Rename Layer
				FUIAction RenameAction = FUIAction(FExecuteAction::CreateSP(this, &FLandscapeEditorCustomNodeBuilder_TargetLayers::OnRenameLayer, InLayerIndex));
				MenuBuilder.AddMenuEntry(LOCTEXT("LayerContextMenu.Rename", "Rename Layer"), LOCTEXT("LayerContextMenu.Rename_Tooltip", "Rename this target layer."), FSlateIcon(), RenameAction);

				// Separate Generic vs Content based actions
				MenuBuilder.AddMenuSeparator();

				// Fill
				FUIAction FillAction = FUIAction(FExecuteAction::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFillLayer, Target),
					FCanExecuteAction::CreateLambda([Target]()
						{
							return Target->LayerInfoObj != nullptr;
						}));

				MenuBuilder.AddMenuEntry(LOCTEXT("LayerContextMenu.Fill", "Fill Layer"),
					LOCTEXT("LayerContextMenu.Fill_Tooltip", "Fills this layer to 100% across the entire landscape. If this is a weight-blended layer, all other weight-blended layers will be cleared."),
					FSlateIcon(),
					FillAction);
			}

			// Clear action shown for Weightmap, Heightmap, and Visibility 
			TAttribute<FText> CanClearLayerToolTipText = MakeAttributeSPLambda(this, [this, Target] {FText Reason; CanClearLayer(Target, Reason); return Reason; });
			FUIAction ClearAction = FUIAction(FExecuteAction::CreateStatic(&FLandscapeEditorCustomNodeBuilder_TargetLayers::OnClearLayer, Target),
				FCanExecuteAction::CreateSPLambda(this, [this, Target] {FText Reason; return CanClearLayer(Target, Reason); }));
			// Convert TargetType to UI label 
			const ELandscapeToolTargetTypeFlags TargetTypeFlags = UE::Landscape::GetLandscapeToolTargetTypeAsFlags(Target->TargetType);
			const FText& TargetTypeText = FText::FromString(UE::Landscape::GetLandscapeToolTargetTypeFlagsAsString(TargetTypeFlags));

			MenuBuilder.AddMenuEntry(FText::Format(LOCTEXT("LayerContextMenu.ClearLayer", "Clear {0} Layer"), TargetTypeText),
				CanClearLayerToolTipText,
				FSlateIcon(),
				ClearAction);
		}
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnExportLayer(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		check(!LandscapeEdMode->IsGridBased());
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		ULandscapeInfo* LandscapeInfo = Target->LandscapeInfo.Get();
		ULandscapeLayerInfoObject* LayerInfoObj = Target->LayerInfoObj.Get(); // nullptr for heightmaps

		// Prompt for filename
		FString SaveDialogTitle;
		FString DefaultFileName;
		const TCHAR* FileTypes = nullptr;

		ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");

		if (Target->TargetType == ELandscapeToolTargetType::Heightmap)
		{
			SaveDialogTitle = LOCTEXT("ExportHeightmap", "Export Landscape Heightmap").ToString();
			DefaultFileName = TEXT("Heightmap");
			FileTypes = LandscapeEditorModule.GetHeightmapExportDialogTypeString();
		}
		else //if (Target->TargetType == ELandscapeToolTargetType::Weightmap)
		{
			check(LayerInfoObj != nullptr);
			SaveDialogTitle = FText::Format(LOCTEXT("ExportLayer", "Export Landscape Layer: {0}"), FText::FromName(LayerInfoObj->GetLayerName())).ToString();
			DefaultFileName = LayerInfoObj->GetLayerName().ToString();
			FileTypes = LandscapeEditorModule.GetWeightmapExportDialogTypeString();
		}

		// Prompt the user for the filenames
		TArray<FString> SaveFilenames;
		bool bOpened = DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			*SaveDialogTitle,
			*LandscapeEdMode->UISettings->LastImportPath,
			*DefaultFileName,
			FileTypes,
			EFileDialogFlags::None,
			SaveFilenames
			);

		if (bOpened)
		{
			const FString& SaveFilename(SaveFilenames[0]);
			LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(SaveFilename);

			// Actually do the export
			if (Target->TargetType == ELandscapeToolTargetType::Heightmap)
			{
				LandscapeInfo->ExportHeightmap(SaveFilename);
			}
			else //if (Target->TargetType == ELandscapeToolTargetType::Weightmap)
			{
				LandscapeInfo->ExportLayer(LayerInfoObj, SaveFilename);
			}

			Target->SetReimportFilePath(SaveFilename);
		}
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnImportLayer(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		check(!LandscapeEdMode->IsGridBased());
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		ULandscapeInfo* LandscapeInfo = Target->LandscapeInfo.Get();
		ULandscapeLayerInfoObject* LayerInfoObj = Target->LayerInfoObj.Get(); // nullptr for heightmaps

		// Prompt for filename
		FString OpenDialogTitle;
		FString DefaultFileName;
		const TCHAR* FileTypes = nullptr;

		ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");

		if (Target->TargetType == ELandscapeToolTargetType::Heightmap)
		{
			OpenDialogTitle = *LOCTEXT("ImportHeightmap", "Import Landscape Heightmap").ToString();
			DefaultFileName = TEXT("Heightmap.png");
			FileTypes = LandscapeEditorModule.GetHeightmapImportDialogTypeString();
		}
		else //if (Target->TargetType == ELandscapeToolTargetType::Weightmap)
		{
			check(LayerInfoObj != nullptr);
			OpenDialogTitle = *FText::Format(LOCTEXT("ImportLayer", "Import Landscape Layer: {0}"), FText::FromName(LayerInfoObj->GetLayerName())).ToString();
			DefaultFileName = *FString::Printf(TEXT("%s.png"), *(LayerInfoObj->GetLayerName().ToString()));
			FileTypes = LandscapeEditorModule.GetWeightmapImportDialogTypeString();
		}

		// Prompt the user for the filenames
		TArray<FString> OpenFilenames;
		bool bOpened = DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			*OpenDialogTitle,
			*LandscapeEdMode->UISettings->LastImportPath,
			*DefaultFileName,
			FileTypes,
			EFileDialogFlags::None,
			OpenFilenames
			);

		if (bOpened)
		{
			const FString& OpenFilename(OpenFilenames[0]);
			LandscapeEdMode->UISettings->LastImportPath = FPaths::GetPath(OpenFilename);

			// Actually do the Import
			LandscapeEdMode->ImportData(*Target, OpenFilename);

			Target->SetReimportFilePath(OpenFilename);
		}
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnReimportLayer(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		check(!LandscapeEdMode->IsGridBased());
		LandscapeEdMode->ReimportData(*Target);
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnHeightmapLayerContextMenu(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->SetCurrentTool("ImportExport");
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnRenameLayer(const int32 InLayerIndex)
{
	// On Rename can be called from Tick or Quick Actions menu
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode(); (LandscapeEdMode != nullptr) && InlineTextBlocks.IsValidIndex(InLayerIndex) && InlineTextBlocks[InLayerIndex].IsValid())
	{
		InlineTextBlocks[InLayerIndex]->EnterEditingMode();
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnFillLayer(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FScopedTransaction Transaction(LOCTEXT("Undo_FillLayer", "Filling Landscape Layer"));
	if (Target->LandscapeInfo.IsValid() && Target->LayerInfoObj.IsValid())
	{
		FLandscapeEditDataInterface LandscapeEdit(Target->LandscapeInfo.Get());

		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		if (LandscapeEdMode)
		{
			FScopedSetLandscapeEditingLayer Scope(LandscapeEdMode->GetLandscape(), LandscapeEdMode->GetCurrentLayerGuid(), [&] { LandscapeEdMode->RequestLayersContentUpdateForceAll(); });
			LandscapeEdit.FillLayer(Target->LayerInfoObj.Get());
		}
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnClearLayer(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	// Heightmap and Visibility will always have valid landscape info but Target layer may be unset
	if (LandscapeEdMode && Target->LandscapeInfo.IsValid())
	{
		ALandscape* Landscape = LandscapeEdMode->GetLandscape();
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetLandscapeSelectedLayer();
		if (Landscape != nullptr && EditLayer != nullptr)
		{
			const FScopedTransaction Transaction(FText::Format(LOCTEXT("Landscape_TargetLayers_Clear", "Clear Edit Layer {0} Data"), UEnum::GetDisplayValueAsText(Target->TargetType)));

			switch (Target->TargetType)
			{
				case ELandscapeToolTargetType::Weightmap: 
					Landscape->ClearPaintLayer(LandscapeEdMode->GetSelectedEditLayerIndex(), Target->LayerInfoObj.Get()); 
					break;
				case ELandscapeToolTargetType::Heightmap:
				case ELandscapeToolTargetType::Visibility: 
					Landscape->ClearEditLayer(LandscapeEdMode->GetSelectedEditLayerIndex(), /*InComponents =*/nullptr, UE::Landscape::GetLandscapeToolTargetTypeAsFlags(Target->TargetType));
					break;
				default:
					check(false);
					return;
			}
	
			if (Target->TargetType == ELandscapeToolTargetType::Weightmap || Target->TargetType == ELandscapeToolTargetType::Visibility)
			{
				LandscapeEdMode->RequestUpdateLayerUsageInformation();
			}
		}
	}
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::CanClearLayer(const TSharedRef<FLandscapeTargetListInfo> Target, FText& OutToolTip)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && Target->LandscapeInfo.IsValid())
	{
		const ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetLandscapeSelectedLayer();

		switch (Target->TargetType)
		{
			case ELandscapeToolTargetType::Heightmap:
				OutToolTip = LOCTEXT("LayerContextMenu.Clear_HeightmapTooltip", "Clears all sculpting heightmap data for this edit layer.");
				break;
			case ELandscapeToolTargetType::Visibility:
				OutToolTip = LOCTEXT("LayerContextMenu.Clear_VisibilityLayerTooltip", "Removes all holes painted in the visibility for this edit layer.");
				break;
			case ELandscapeToolTargetType::Weightmap:
				OutToolTip = LOCTEXT("LayerContextMenu.Clear_TargetLayerTooltip", "Clears this target layer to 0% across the entire landscape. If this is a weight-blended layer, other weight-blended layers will be adjusted to compensate.");
				return LandscapeEdMode->CanEditTargetLayer(&OutToolTip, Target->LayerInfoObj.Get(), EditLayer);
			default:
				check(false);
				return false;
		}
		// Return tooltip should always be set (Shows valid tooltip or reason why the operation is disabled)
		check(OutToolTip.ToString() != FString());
		return LandscapeEdMode->CanEditLayer(&OutToolTip, EditLayer);
	}
	return false;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnRebuildMICs(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (Target->LandscapeInfo.IsValid())
	{
		Target->LandscapeInfo.Get()->UpdateAllComponentMaterialInstances(/*bInvalidateCombinationMaterials = */true);
	}
}


bool FLandscapeEditorCustomNodeBuilder_TargetLayers::ShouldFilterLayerInfo(const FAssetData& AssetData, FName LayerName)
{
	const FName LayerNameMetaData = AssetData.GetTagValueRef<FName>("LayerName");
	if (!LayerNameMetaData.IsNone())
	{
		return LayerNameMetaData != LayerName;
	}

	ULandscapeLayerInfoObject* LayerInfo = CastChecked<ULandscapeLayerInfoObject>(AssetData.GetAsset());
	return LayerInfo->GetLayerName() != LayerName;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerSetObject(const FAssetData& AssetData, const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FScopedTransaction Transaction(LOCTEXT("Undo_AssignTargetLayer", "Assigning Asset to Target Layer"));

	UObject* Object = AssetData.GetAsset();

	// Allow user to clear a layer. UI will display a warning message after to let them know a target layer is missing a layer info asset
	if (Object == nullptr)
	{
		ULandscapeInfo* LandscapeInfo = Target->LandscapeInfo.Get();
		ALandscape* LandscapeActor = LandscapeInfo->LandscapeActor.Get();

		int32 Index = LandscapeInfo->GetLayerInfoIndex(Target->LayerName, Target->Owner.Get());
		if (ensure(Index != INDEX_NONE))
		{
			FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[Index];
			LandscapeInfo->ReplaceLayer(LayerSettings.LayerInfoObj, nullptr);
		}

		if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
		{
			LandscapeEdMode->CurrentToolTarget.TargetType = Target->TargetType;
			LandscapeEdMode->SetCurrentTargetLayer(Target->LayerName, nullptr);
			LandscapeInfo->UpdateLayerInfoMap();
			LandscapeEdMode->UpdateTargetList();
			return;
		}
	}

	ULandscapeLayerInfoObject* SelectedLayerInfo = const_cast<ULandscapeLayerInfoObject*>(CastChecked<ULandscapeLayerInfoObject>(Object));

	if (SelectedLayerInfo != Target->LayerInfoObj.Get())
	{
		if (ensure(SelectedLayerInfo->GetLayerName() == Target->GetLayerName()))
		{
			ULandscapeInfo* LandscapeInfo = Target->LandscapeInfo.Get();
			ALandscape* LandscapeActor = LandscapeInfo->LandscapeActor.Get();
			
			if (!LandscapeActor->HasTargetLayer(Target->GetLayerName()))
			{
				LandscapeActor->AddTargetLayer(Target->GetLayerName(), FLandscapeTargetLayerSettings(SelectedLayerInfo));
			}
			
			if (Target->LayerInfoObj.IsValid())
			{
				int32 Index = LandscapeInfo->GetLayerInfoIndex(Target->LayerInfoObj.Get(), Target->Owner.Get());
				if (ensure(Index != INDEX_NONE))
				{
					FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[Index];
					LandscapeInfo->ReplaceLayer(LayerSettings.LayerInfoObj, SelectedLayerInfo);
					// Important : don't use LayerSettings after the call to ReplaceLayer as it will have been reallocated. 
					//  Validate that the replacement happened as expected : 
					check(LandscapeInfo->GetLayerInfoIndex(SelectedLayerInfo, Target->Owner.Get()) != INDEX_NONE);
				}
			}
			else
			{
				int32 Index = LandscapeInfo->GetLayerInfoIndex(Target->LayerName, Target->Owner.Get());
				if (ensure(Index != INDEX_NONE))
				{
					FLandscapeInfoLayerSettings& LayerSettings = LandscapeInfo->Layers[Index];
					LayerSettings.LayerInfoObj = SelectedLayerInfo;

					Target->LandscapeInfo->CreateTargetLayerSettingsFor(SelectedLayerInfo);
				}
			}
						
			if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
			{
				LandscapeEdMode->CurrentToolTarget.TargetType = Target->TargetType;
				LandscapeEdMode->SetCurrentTargetLayer(Target->LayerName, SelectedLayerInfo);
				LandscapeEdMode->UpdateTargetList();
			}
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("Error_LayerNameMismatch", "Can't use this layer info because the layer name does not match"));
		}
	}
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerInfoSelectorVisibility(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (Target->TargetType == ELandscapeToolTargetType::Weightmap)
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

bool FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerCreateEnabled(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (!Target->LayerInfoObj.IsValid())
	{
		return true;
	}

	return false;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerCreateClicked(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	check(!Target->LayerInfoObj.IsValid());

	FScopedTransaction Transaction(LOCTEXT("Undo_Create", "Creating New Landscape Layer"));

	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		FName LayerName = Target->GetLayerName();
		FName FileName;
		FString PackageName = UE::Landscape::GetLayerInfoObjectPackageName(LayerName, LandscapeEdMode->GetTargetLayerAssetPackagePath(), FileName);

		TSharedRef<SDlgPickAssetPath> NewLayerDlg =
			SNew(SDlgPickAssetPath)
			.Title(LOCTEXT("CreateNewLayerInfo", "Create New Landscape Layer Info Object"))
			.DefaultAssetPath(FText::FromString(PackageName));

		if (NewLayerDlg->ShowModal() != EAppReturnType::Cancel)
		{
			PackageName = NewLayerDlg->GetAssetPath().ToString();
			FileName = FName(*NewLayerDlg->GetAssetName().ToString());

			CreateTargetLayerInfoAsset(Target, PackageName, FileName.ToString());
		}
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::CreateTargetLayerInfoAsset(const TSharedRef<FLandscapeTargetListInfo> Target, const FString& PackageName, const FString& FileName)
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		const FName LayerName = Target->GetLayerName();

		ULandscapeLayerInfoObject* LayerInfo = UE::Landscape::CreateTargetLayerInfo(LayerName, PackageName, FileName);

		if (LandscapeEdMode->CurrentToolTarget.LayerName == Target->LayerName
			&& LandscapeEdMode->CurrentToolTarget.LayerInfo == Target->LayerInfoObj)
		{
			LandscapeEdMode->SetCurrentTargetLayer(Target->LayerName, Target->LayerInfoObj);
		}

		Target->LayerInfoObj = LayerInfo;
		Target->LandscapeInfo->CreateTargetLayerSettingsFor(LayerInfo);

		// Show in the content browser
		TArray<UObject*> Objects;
		Objects.Add(LayerInfo);
		GEditor->SyncBrowserToObjects(Objects);

		ALandscape* LandscapeActor = Target->LandscapeInfo->LandscapeActor.Get();
		LandscapeActor->UpdateTargetLayer(FName(LayerName), FLandscapeTargetLayerSettings(LayerInfo));

		LandscapeEdMode->UpdateTargetList();
	}
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnTargetLayerDeleteClicked(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	check(Target->LandscapeInfo.IsValid());

	if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("Prompt_DeleteLayer", "Are you sure you want to delete this layer?")) == EAppReturnType::Yes)
	{
		FScopedTransaction Transaction(LOCTEXT("Undo_Delete", "Delete Layer"));

		FEdModeLandscape* LandscapeEdMode = GetEditorMode();
		FScopedSetLandscapeEditingLayer Scope(LandscapeEdMode ? LandscapeEdMode->GetLandscape() : nullptr, LandscapeEdMode ? LandscapeEdMode->GetCurrentLayerGuid() : FGuid());

		Target->LandscapeInfo->DeleteLayer(Target->LayerInfoObj.Get(), Target->LayerName);

		if (LandscapeEdMode)
		{
			LandscapeEdMode->UpdateTargetList();
		}
	}
}

FSlateColor FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayerUsageDebugColor(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (GLandscapeViewMode == ELandscapeViewMode::LayerUsage && Target->TargetType != ELandscapeToolTargetType::Heightmap && ensure(Target->LayerInfoObj.IsValid()))
	{
		return FSlateColor(Target->LayerInfoObj->GetLayerUsageDebugColor());
	}
	return FSlateColor(FLinearColor(0, 0, 0, 0));
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeLayerUsageVisibility(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (GLandscapeViewMode == ELandscapeViewMode::LayerUsage && Target->TargetType != ELandscapeToolTargetType::Heightmap && Target->LayerInfoObj.IsValid())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeLayerUsageVisibility_Invert(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (GLandscapeViewMode == ELandscapeViewMode::LayerUsage && Target->TargetType != ELandscapeToolTargetType::Heightmap && Target->LayerInfoObj.IsValid())
	{
		return EVisibility::Collapsed;
	}

	return EVisibility::Visible;
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetLayersSubstractiveBlendVisibility(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr && Target->TargetType == ELandscapeToolTargetType::Weightmap && Target->LayerInfoObj.IsValid())
	{
		if (ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(LandscapeEdMode->GetSelectedEditLayerIndex()))
	    {
		    return EVisibility::Visible;
	    }
	}

	return EVisibility::Collapsed;
}

ECheckBoxState FLandscapeEditorCustomNodeBuilder_TargetLayers::IsLayersSubstractiveBlendChecked(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(LandscapeEdMode->GetSelectedEditLayerIndex());
		if (EditLayer && Target->LayerInfoObj.IsValid())
		{
			const bool* AllocationBlend = EditLayer->GetWeightmapLayerAllocationBlend().Find(Target.Get().LayerInfoObj.Get());

			if (AllocationBlend != nullptr)
			{
				return (*AllocationBlend) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}

	return ECheckBoxState::Unchecked;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnLayersSubstractiveBlendChanged(ECheckBoxState NewCheckedState, const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		FScopedTransaction Transaction(LOCTEXT("Undo_SubtractiveBlend", "Set Subtractive Blend Layer"));
		ULandscapeEditLayerBase* EditLayer = LandscapeEdMode->GetEditLayer(LandscapeEdMode->GetSelectedEditLayerIndex());
		if (EditLayer != nullptr)
		{
			EditLayer->AddOrUpdateWeightmapAllocationLayerBlend(Target.Get().LayerInfoObj.Get(), NewCheckedState == ECheckBoxState::Checked, /*bInModify =*/true);
		}
	}
}

EVisibility FLandscapeEditorCustomNodeBuilder_TargetLayers::GetDebugModeColorChannelVisibility(const TSharedRef<FLandscapeTargetListInfo> Target)
{
	if (GLandscapeViewMode == ELandscapeViewMode::DebugLayer && Target->TargetType != ELandscapeToolTargetType::Heightmap && Target->LayerInfoObj.IsValid())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

ECheckBoxState FLandscapeEditorCustomNodeBuilder_TargetLayers::DebugModeColorChannelIsChecked(const TSharedRef<FLandscapeTargetListInfo> Target, int32 Channel)
{
	if (Target->DebugColorChannel == Channel)
	{
		return ECheckBoxState::Checked;
	}

	return ECheckBoxState::Unchecked;
}

void FLandscapeEditorCustomNodeBuilder_TargetLayers::OnDebugModeColorChannelChanged(ECheckBoxState NewCheckedState, const TSharedRef<FLandscapeTargetListInfo> Target, int32 Channel)
{
	if (NewCheckedState == ECheckBoxState::Checked)
	{
		// Enable on us and disable colour channel on other targets
		if (ensure(Target->LayerInfoObj != nullptr))
		{
			ULandscapeInfo* LandscapeInfo = Target->LandscapeInfo.Get();
			int32 Index = LandscapeInfo->GetLayerInfoIndex(Target->LayerInfoObj.Get(), Target->Owner.Get());
			if (ensure(Index != INDEX_NONE))
			{
				for (auto It = LandscapeInfo->Layers.CreateIterator(); It; It++)
				{
					FLandscapeInfoLayerSettings& LayerSettings = *It;
					if (It.GetIndex() == Index)
					{
						LayerSettings.DebugColorChannel = Channel;
					}
					else
					{
						LayerSettings.DebugColorChannel &= ~Channel;
					}
				}
				LandscapeInfo->UpdateDebugColorMaterial();

				if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
				{
					LandscapeEdMode->UpdateTargetList();
				}
			}
		}
	}
}

FText FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendMethodText(const TSharedRef<FLandscapeTargetListInfo> InTarget, bool bInIncludeBlendGroup)
{
	const ULandscapeLayerInfoObject* LayerInfo = InTarget->LayerInfoObj.Get();
	if (LayerInfo == nullptr)
	{
		return FText::GetEmpty();
	}

	const UEnum* ModeEnum = StaticEnum<ELandscapeTargetLayerBlendMethod>();
	check(ModeEnum != nullptr);
	FText Result = ModeEnum->GetDisplayNameTextByValue(static_cast<int64>(LayerInfo->GetBlendMethod()));
	if (bInIncludeBlendGroup && (LayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending) && !LayerInfo->GetBlendGroup().IsNone())
	{
		FText BlendGroupName = GetTargetBlendGroupText(InTarget, /*bInAllowNoName = */false);
		if (!BlendGroupName.IsEmpty())
		{
			Result = FText::Format(LOCTEXT("PremultipliedAlphaBlendingBlendGroupName", "{0} ({1})"), Result, BlendGroupName);
		}
	}
	return Result;
}

FText FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendGroupText(const TSharedRef<FLandscapeTargetListInfo> InTarget, bool bInAllowNoName)
{
	const ULandscapeLayerInfoObject* LayerInfo = InTarget->LayerInfoObj.Get();
	if (LayerInfo == nullptr)
	{
		return FText::GetEmpty();
	}

	FName LayerGroupName;
	if (LayerInfo->GetBlendMethod() == ELandscapeTargetLayerBlendMethod::PremultipliedAlphaBlending)
	{
		LayerGroupName = LayerInfo->GetBlendGroup();
	}

	if (!LayerGroupName.IsNone())
	{
		return FText::FromName(LayerGroupName);
	}
	else if (bInAllowNoName)
	{
		return LOCTEXT("UnnamedBlendGroupName", "(unnamed)");
	}
	return FText::GetEmpty();
}

FText FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetBlendMethodTooltipText(const TSharedRef<FLandscapeTargetListInfo> InTarget)
{
	const ULandscapeLayerInfoObject* LayerInfo = InTarget->LayerInfoObj.Get();
	if (LayerInfo == nullptr)
	{
		return FText::GetEmpty();
	}

	const UEnum* ModeEnum = StaticEnum<ELandscapeTargetLayerBlendMethod>();
	check(ModeEnum != nullptr);
	return ModeEnum->GetToolTipTextByIndex(ModeEnum->GetIndexByValue(static_cast<int64>(LayerInfo->GetBlendMethod())));
}

FSlateColor FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetTextColor(const TSharedRef<FLandscapeTargetListInfo> InTarget)
{
	return FLandscapeEditorCustomNodeBuilder_TargetLayers::GetTargetLayerIsSelected(InTarget) ? FStyleColors::ForegroundHover : FSlateColor::UseForeground();
}

//////////////////////////////////////////////////////////////////////////

void SLandscapeEditorSelectableBorder::Construct(const FArguments& InArgs)
{
	SBorder::Construct(
		SBorder::FArguments()
		.HAlign(InArgs._HAlign)
		.VAlign(InArgs._VAlign)
		.Padding(InArgs._Padding)
		.BorderImage(this, &SLandscapeEditorSelectableBorder::GetBorder)
		.Content()
		[
			InArgs._Content.Widget
		]
	);

	OnContextMenuOpening = InArgs._OnContextMenuOpening;
	OnSelected = InArgs._OnSelected;
	IsSelected = InArgs._IsSelected;
	OnDoubleClick = InArgs._OnDoubleClick;
}

FReply SLandscapeEditorSelectableBorder::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton &&
			OnSelected.IsBound())
		{
			OnSelected.Execute();

			return FReply::Handled().ReleaseMouseCapture();
		}
		else if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton &&
			OnContextMenuOpening.IsBound())
			{
				TSharedPtr<SWidget> Content = OnContextMenuOpening.Execute();
				if (Content.IsValid())
				{
					FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

					FSlateApplication::Get().PushMenu(SharedThis(this), WidgetPath, Content.ToSharedRef(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
				}
			
			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return FReply::Unhandled();
}

FReply SLandscapeEditorSelectableBorder::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MyGeometry.IsUnderLocation(MouseEvent.GetScreenSpacePosition()))
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton &&
			OnDoubleClick.IsBound())
		{
			OnDoubleClick.Execute();

			return FReply::Handled().ReleaseMouseCapture();
		}
	}

	return FReply::Unhandled();
}

const FSlateBrush* SLandscapeEditorSelectableBorder::GetBorder() const
{
	const bool bIsSelected = IsSelected.Get();
	const bool bHovered = IsHovered() && OnSelected.IsBound();

	if (bIsSelected)
	{
		return bHovered
			? FAppStyle::GetBrush("LandscapeEditor.TargetList", ".RowSelectedHovered")
			: FAppStyle::GetBrush("LandscapeEditor.TargetList", ".RowSelected");
	}
	else
	{
		return bHovered
			? FAppStyle::GetBrush("LandscapeEditor.TargetList", ".RowBackgroundHovered")
			: FAppStyle::GetBrush("LandscapeEditor.TargetList", ".RowBackground");
	}
}

TSharedRef<FTargetLayerDragDropOp> FTargetLayerDragDropOp::New(int32 InSlotIndexBeingDragged, SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> WidgetToShow)
{
	TSharedRef<FTargetLayerDragDropOp> Operation = MakeShareable(new FTargetLayerDragDropOp);

	Operation->MouseCursor = EMouseCursor::GrabHandClosed;
	Operation->SlotIndexBeingDragged = InSlotIndexBeingDragged;
	Operation->SlotBeingDragged = InSlotBeingDragged;
	Operation->WidgetToShow = WidgetToShow;

	Operation->Construct();

	return Operation;
}

FTargetLayerDragDropOp::~FTargetLayerDragDropOp()
{
}

TSharedPtr<SWidget> FTargetLayerDragDropOp::GetDefaultDecorator() const
{
	return SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetDragDropTooltipBackground"))
			.Content()
			[
				WidgetToShow.ToSharedRef()
			];
		
}

#undef LOCTEXT_NAMESPACE