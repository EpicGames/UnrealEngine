// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Layout/Margin.h"
#include "Styling/SlateColor.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Layout/SBorder.h"
#include "LandscapeEdMode.h"
#include "IDetailCustomNodeBuilder.h"
#include "IDetailCustomization.h"
#include "AssetThumbnail.h"
#include "Framework/SlateDelegates.h"
#include "LandscapeEditorDetailCustomization_Base.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class SDragAndDropVerticalBox;

/**
 * Slate widgets customizer for the target layers list in the Landscape Editor
 */

class FLandscapeEditorStructCustomization_FTargetLayerAssetPath : public FLandscapeEditorStructCustomization_Base
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private: 
	static FText GetTargetLayerAssetFilePath();
	static FReply OnSetTargetLayerAssetFilePath();

	static bool IsUseTargetLayerAssetPathEnabled();
	static ECheckBoxState GetUseTargetLayerAssetPathCheckState();
	static void OnUseTargetLayerAssetPathCheckStateChanged(ECheckBoxState NewCheckedState);
};

class FLandscapeEditorDetailCustomization_TargetLayers : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	static bool ShouldShowTargetLayers();
	static EVisibility GetPaintingRestrictionVisibility();
	static EVisibility GetWeightBlendedTargetLayerPaintModeVisibility();
	static EVisibility GetVisibilityMaskTipVisibility();
	static EVisibility GetPopulateTargetLayersInfoTipVisibility();
	static EVisibility GetTargetLayersInvalidInfoAssetTipVisibility();
	static EVisibility GetFilteredTargetLayersListInfoTipVisibility();
};

class FLandscapeEditorCustomNodeBuilder_TargetLayers : public IDetailCustomNodeBuilder, public TSharedFromThis<FLandscapeEditorCustomNodeBuilder_TargetLayers>
{
public:
	FLandscapeEditorCustomNodeBuilder_TargetLayers(TSharedRef<FAssetThumbnailPool> ThumbnailPool, 
		TSharedRef<IPropertyHandle> InTargetDisplayOrderPropertyHandle, TSharedRef<IPropertyHandle> InTargetDisplayOrderIsAscendingPropertyHandle, TSharedRef<IPropertyHandle> InTargetShowUnusedLayersPropertyHandle);
	~FLandscapeEditorCustomNodeBuilder_TargetLayers();

	virtual void SetOnRebuildChildren( FSimpleDelegate InOnRegenerateChildren ) override;
	virtual void GenerateHeaderRowContent( FDetailWidgetRow& NodeRow ) override;
	virtual void GenerateChildContent( IDetailChildrenBuilder& ChildrenBuilder ) override;
	virtual void Tick(float DeltaTime) override;
	virtual bool RequiresTick() const override { return true; }
	virtual bool InitiallyCollapsed() const override { return false; }
	virtual FName GetName() const override { return "TargetLayers"; }

	static TArray<TSharedRef<FLandscapeTargetListInfo>> PrepareTargetLayerList(bool bInSort, bool bInFilter);

protected:
	TSharedRef<FAssetThumbnailPool> ThumbnailPool;
	TSharedRef<IPropertyHandle> TargetDisplayOrderPropertyHandle;
	TSharedRef<IPropertyHandle> TargetDisplayOrderIsAscendingPropertyHandle;
	TSharedRef<IPropertyHandle> TargetShowUnusedLayersPropertyHandle;
	TSharedPtr<SSearchBox> LayersFilterSearchBox;
	/** Widgets for displaying and editing the target layer name */
	TArray<TSharedPtr<SInlineEditableTextBlock>> InlineTextBlocks;

	static class FEdModeLandscape* GetEditorMode();

	TSharedPtr<SWidget> GenerateRow(const TSharedRef<FLandscapeTargetListInfo> Target);

	static ELandscapeTargetLayerActionType GetTargetLayerActionType(const TSharedRef<FLandscapeTargetListInfo> Target);
	static const FSlateBrush* GetTargetLayerActionTypeBrush(const TSharedRef<FLandscapeTargetListInfo> Target);
	static FLinearColor GetTargetLayerBlendGroupStripColor(const TSharedRef<FLandscapeTargetListInfo> Target);
	static EVisibility GetTargetLayerBlendGroupStripVisibility(const TSharedRef<FLandscapeTargetListInfo> Target);
	static bool GetTargetLayerIsSelected(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnTargetSelectionChanged(const TSharedRef<FLandscapeTargetListInfo> Target);
	TSharedPtr<SWidget> OnTargetLayerContextMenuOpening(const TSharedRef<FLandscapeTargetListInfo> Target, const int32 InLayerIndex);
	void OnRenameLayer(const int32 InLayerIndex);
	static void OnExportLayer(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnImportLayer(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnReimportLayer(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnHeightmapLayerContextMenu(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnFillLayer(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnClearLayer(const TSharedRef<FLandscapeTargetListInfo> Target);
	static bool CanClearLayer(const TSharedRef<FLandscapeTargetListInfo> Target, FText& OutToolTip);
	static void OnRebuildMICs(const TSharedRef<FLandscapeTargetListInfo> Target);
	static bool ShouldFilterLayerInfo(const struct FAssetData& AssetData, FName LayerName);
	static void OnTargetLayerSetObject(const FAssetData& AssetData, const TSharedRef<FLandscapeTargetListInfo> Target); 
	static EVisibility GetTargetLayerInfoSelectorVisibility(const TSharedRef<FLandscapeTargetListInfo> Target);
	static bool GetTargetLayerCreateEnabled(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnTargetLayerCreateClicked(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void CreateTargetLayerInfoAsset(const TSharedRef<FLandscapeTargetListInfo> Target, const FString& PackageName, const FString& FileName);
	static void OnTargetLayerDeleteClicked(const TSharedRef<FLandscapeTargetListInfo> Target);
	static FSlateColor GetLayerUsageDebugColor(const TSharedRef<FLandscapeTargetListInfo> Target);
	static EVisibility GetDebugModeLayerUsageVisibility(const TSharedRef<FLandscapeTargetListInfo> Target);
	static EVisibility GetDebugModeLayerUsageVisibility_Invert(const TSharedRef<FLandscapeTargetListInfo> Target);
	static EVisibility GetDebugModeColorChannelVisibility(const TSharedRef<FLandscapeTargetListInfo> Target);
	static ECheckBoxState DebugModeColorChannelIsChecked(const TSharedRef<FLandscapeTargetListInfo> Target, int32 Channel);
	static void OnDebugModeColorChannelChanged(ECheckBoxState NewCheckedState, const TSharedRef<FLandscapeTargetListInfo> Target, int32 Channel);
	static FText GetTargetBlendMethodText(const TSharedRef<FLandscapeTargetListInfo> InTarget, bool bInIncludeBlendGroup);
	static FText GetTargetBlendGroupText(const TSharedRef<FLandscapeTargetListInfo> InTarget, bool bInAllowNoName);
	static FText GetTargetBlendMethodTooltipText(const TSharedRef<FLandscapeTargetListInfo> InTarget);
	static FSlateColor GetTargetTextColor(const TSharedRef<FLandscapeTargetListInfo> InTarget);

	static EVisibility GetLayersSubstractiveBlendVisibility(const TSharedRef<FLandscapeTargetListInfo> Target);
	static ECheckBoxState IsLayersSubstractiveBlendChecked(const TSharedRef<FLandscapeTargetListInfo> Target);
	static void OnLayersSubstractiveBlendChanged(ECheckBoxState NewCheckedState, const TSharedRef<FLandscapeTargetListInfo> Target);

	// Drag/Drop handling
	FReply HandleDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	TOptional<SDragAndDropVerticalBox::EItemDropZone> HandleCanAcceptDrop(const FDragDropEvent& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);
	FReply HandleAcceptDrop(FDragDropEvent const& DragDropEvent, SDragAndDropVerticalBox::EItemDropZone DropZone, int32 SlotIndex, SVerticalBox::FSlot* Slot);

	// Create Layers From Materials
	const TArray<const TSharedRef<FLandscapeTargetListInfo>> GetUnassignedTargetLayersFromMaterial() const;
	bool HasUnassignedTargetLayers() const;
	FReply HandleCreateLayersFromMaterials() const;

	// Auto-Fill Target layers
	FReply ShowAutoFillTargetLayerDialog() const;
	FReply HandleAutoFillTargetLayers(const bool bUpdateAllLayers, const bool bCreateTargetLayers) const;

	void HandleCreateLayer();
	const FSlateBrush* GetTargetLayerDisplayOrderBrush() const;
	TSharedRef<SWidget> GetTargetLayerDisplayOrderButtonMenuContent();
	void SetSelectedDisplayOrder(ELandscapeLayerDisplayMode InDisplayOrder);
	void SetAscendingDisplayOrder(bool bInIsAscending);
	bool IsSelectedDisplayOrder(ELandscapeLayerDisplayMode InDisplayOrder) const;
	bool IsAscendingDisplayOrder() const;
	bool IsDescendingDisplayOrder() const;
	bool CanChangeDisplayOrderSortType() const;

	TSharedRef<SWidget> GetTargetLayerShowUnusedButtonMenuContent();
	const FSlateBrush* GetShowUnusedBrush() const;
	void ShowUnusedLayers(bool Result);
	bool ShouldShowUnusedLayers(bool Result) const;
	EVisibility ShouldShowLayer(TSharedRef<FLandscapeTargetListInfo> Target) const;

	void OnFilterTextChanged(const FText& InFilterText);
	void OnFilterTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	int32 GetWeightmapTargetLayerCount() const;
	bool HasWeightmapTargetLayers() const;
	EVisibility GetLayersFilterVisibility() const;
	EVisibility GetLayersDisplayOptionsVisibility() const;
	FText GetLayersFilterText() const;
	FText GetNumWeightmapTargetLayersText() const;
};

class SLandscapeEditorSelectableBorder : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SLandscapeEditorSelectableBorder)
		: _Content()
		, _HAlign(HAlign_Fill)
		, _VAlign(VAlign_Fill)
		, _Padding(FMargin(2.0f))
	{ }
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ARGUMENT(EHorizontalAlignment, HAlign)
		SLATE_ARGUMENT(EVerticalAlignment, VAlign)
		SLATE_ATTRIBUTE(FMargin, Padding)

		SLATE_EVENT(FOnContextMenuOpening, OnContextMenuOpening)
		SLATE_EVENT(FSimpleDelegate, OnSelected)
		SLATE_EVENT(FSimpleDelegate, OnDoubleClick)
		SLATE_ATTRIBUTE(bool, IsSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	const FSlateBrush* GetBorder() const;

protected:
	FOnContextMenuOpening OnContextMenuOpening;
	FSimpleDelegate OnSelected;
	TAttribute<bool> IsSelected;
	FSimpleDelegate OnDoubleClick;
};

class FTargetLayerDragDropOp : public FDragAndDropVerticalBoxOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FTargetLayerDragDropOp, FDragAndDropVerticalBoxOp)

	TSharedPtr<SWidget> WidgetToShow;

	static TSharedRef<FTargetLayerDragDropOp> New(int32 InSlotIndexBeingDragged, SVerticalBox::FSlot* InSlotBeingDragged, TSharedPtr<SWidget> InWidgetToShow);

public:
	virtual ~FTargetLayerDragDropOp();

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;
};
