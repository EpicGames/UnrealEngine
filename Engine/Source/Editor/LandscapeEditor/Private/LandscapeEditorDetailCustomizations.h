// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LandscapeEditorDetailCustomization_Base.h"
#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorDetailCustomization_ResizeLandscape.h"
#include "LandscapeEditorDetailCustomization_CopyPaste.h"
#include "LandscapeEditorDetailCustomization_MiscTools.h"
#include "LandscapeEditorDetailCustomization_AlphaBrush.h"
#include "LandscapeEditorDetailCustomization_TargetLayers.h"
