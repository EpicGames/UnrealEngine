// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorDetails.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Widgets/Text/STextBlock.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Input/SSegmentedControl.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorDetailCustomization_Blueprint.h"
#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorDetailCustomization_ResizeLandscape.h"
#include "LandscapeEditorDetailCustomization_CopyPaste.h"
#include "LandscapeEditorDetailCustomization_MiscTools.h"
#include "LandscapeEditorDetailCustomization_AlphaBrush.h"
#include "LandscapeEditorDetailCustomization_ImportExport.h"
#include "LandscapeSettings.h"
#include "DetailWidgetRow.h"
#include "LandscapeEditorDetailCustomization_TargetLayers.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h" 
#include "VariablePrecisionNumericInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"

#include "Templates/SharedPointer.h"

#include "SLandscapeEditor.h"
#include "LandscapeEditorCommands.h"
#include "LandscapeEditorDetailCustomization_LayersBrushStack.h"
#include "LandscapeEditorObject.h"
#include "Landscape.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"

TSharedRef<IDetailCustomization> FLandscapeEditorDetails::MakeInstance()
{
	return MakeShareable(new FLandscapeEditorDetails);
}

void FLandscapeEditorDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();

	if (LandscapeEdMode == nullptr)
	{
		return;
	}

	static const FLinearColor BorderColor = FLinearColor(0.2f, 0.2f, 0.2f, 0.2f);
	static const FSlateBrush* BorderStyle = FAppStyle::GetBrush("DetailsView.GroupSection");

	IDetailCategoryBuilder& LandscapeEditorCategory = DetailBuilder.EditCategory("LandscapeEditor", NSLOCTEXT("Contexts", "LandscapeEditor", "Landscape Editor"), ECategoryPriority::TypeSpecific);

	IDetailCategoryBuilder& BrushSettingsCategory = DetailBuilder.EditCategory("Brush Settings");

	// Ensure the categories in the Landscape Editor Details panel is stable. Most importantly that the Brush
	// and Tool Settings are adjacent to each other. 
	auto CategorySorter = [](const TMap<FName, IDetailCategoryBuilder*>& Categories)
	{
		int32 Order = 0;
		auto SafeSetOrder = [&Categories, &Order](const FName& CategoryName )
		{
			if (IDetailCategoryBuilder* const* Builder = Categories.Find(CategoryName))
			{
				(*Builder)->SetSortOrder(Order++);	
			}
		};
		
		SafeSetOrder(FName("LandscapeEditor"));
		SafeSetOrder(FName("Import / Export"));
		SafeSetOrder(FName("Change Component Size"));
		SafeSetOrder(FName("New Landscape"));
		
		SafeSetOrder(FName("Tool Settings"));
		SafeSetOrder(FName("Brush Settings"));
		SafeSetOrder(FName("Select Mask"));
		
		SafeSetOrder(FName("Edit Layers"));
		SafeSetOrder(FName("Edit Layer Blueprint Brushes"));
		SafeSetOrder(FName("Target Layers"));
	};
	
	DetailBuilder.SortCategories(CategorySorter);

	// UIMax and ClampMax for the brush radius come from the project settings : 	
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
	TSharedRef<IPropertyHandle> BrushRadiusProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushRadius));
	TSharedRef<IPropertyHandle> PaintBrushRadiusProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintBrushRadius));
	const FString UIMaxString = FString::Printf(TEXT("%f"), Settings->GetBrushSizeUIMax());
	const FString ClampMaxString = FString::Printf(TEXT("%f"), Settings->GetBrushSizeClampMax());
	BrushRadiusProperty->SetInstanceMetaData(TEXT("UIMax"), *UIMaxString);
	BrushRadiusProperty->SetInstanceMetaData(TEXT("ClampMax"), *ClampMaxString);
	PaintBrushRadiusProperty->SetInstanceMetaData(TEXT("UIMax"), *UIMaxString);
	PaintBrushRadiusProperty->SetInstanceMetaData(TEXT("ClampMax"), *ClampMaxString);

	LandscapeEditorCategory.AddCustomRow(FText::GetEmpty())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&FLandscapeEditorDetails::GetTargetLandscapeSelectorVisibility)))
	[
		SNew(SComboButton)
		.OnGetMenuContent_Static(&FLandscapeEditorDetails::GetTargetLandscapeMenu)
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text_Static(&FLandscapeEditorDetails::GetTargetLandscapeName)
		]
	];
		
	FText Reason;
	bool bDisabledEditing = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && !LandscapeEdMode->CanEditCurrentTarget(&Reason);

	if (bDisabledEditing)
	{
		LandscapeEditorCategory.AddCustomRow(FText::GetEmpty())
			[
				SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.AutoWrapText(true)
				.Font(FCoreStyle::GetDefaultFontStyle("Bold", 10))
				.Justification(ETextJustify::Center)
				.BackgroundColor(FCoreStyle::Get().GetColor("ErrorReporting.BackgroundColor"))
				.ForegroundColor(FCoreStyle::Get().GetColor("ErrorReporting.ForegroundColor"))
				.Text(Reason)
			];
	}
		
	// Only continue customization if we are in NewLandscape mode or if editing is not disabled
	if (bDisabledEditing && LandscapeEdMode->CurrentTool->GetToolName() != FName("NewLandscape"))
	{
		return;
	}

	// Custom Brush Selectors 

	TSharedPtr<SSegmentedControl<FName>> BrushSelectors = SNew(SSegmentedControl<FName>)
	.OnValueChanged(this, &FLandscapeEditorDetails::SetBrushCommand)
	.Value(this, &FLandscapeEditorDetails::GetCurrentBrushFName) ;
	for (FName BrushName : LandscapeEdMode->CurrentTool->ValidBrushes)
	{
		TSharedPtr<FUICommandInfo> Command = FLandscapeEditorCommands::Get().NameToCommandMap.FindRef(BrushName);
		if (Command.IsValid())
		{
			BrushSelectors->AddSlot(BrushName)
			.Icon(Command->GetIcon().GetIcon())
			.ToolTip(Command->GetDescription());
		}
	}
	BrushSelectors->RebuildChildren();

	TSharedPtr<SSegmentedControl<FName>> FalloffSelectors = SNew(SSegmentedControl<FName>)
		.OnValueChanged(this, &FLandscapeEditorDetails::SetBrushCommand)
		.Value(this, &FLandscapeEditorDetails::GetCurrentBrushFalloffFName)
		+ SSegmentedControl<FName>::Slot(FName("Circle_Smooth")).Icon(FLandscapeEditorCommands::Get().CircleBrush_Smooth->GetIcon().GetIcon()).ToolTip(FLandscapeEditorCommands::Get().CircleBrush_Smooth->GetDescription())
		+ SSegmentedControl<FName>::Slot(FName("Circle_Linear")).Icon(FLandscapeEditorCommands::Get().CircleBrush_Linear->GetIcon().GetIcon()).ToolTip(FLandscapeEditorCommands::Get().CircleBrush_Linear->GetDescription())
		+ SSegmentedControl<FName>::Slot(FName("Circle_Spherical")).Icon(FLandscapeEditorCommands::Get().CircleBrush_Spherical->GetIcon().GetIcon()).ToolTip(FLandscapeEditorCommands::Get().CircleBrush_Spherical->GetDescription())
		+ SSegmentedControl<FName>::Slot(FName("Circle_Tip")).Icon(FLandscapeEditorCommands::Get().CircleBrush_Tip->GetIcon().GetIcon()).ToolTip(FLandscapeEditorCommands::Get().CircleBrush_Tip->GetDescription());

	LandscapeEditorCategory.AddCustomRow(FText::GetEmpty())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FLandscapeEditorDetails::GetBrushSelectorVisibility)))
	.NameContent()
	[
		SNew(STextBlock)
		.TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ) )
		.Text(LOCTEXT("BrushSelector", "Brush Type"))
		.ToolTipText(LOCTEXT("BrushSelectorToolTip", "Selects the type of brush to use"))
	]
	.ValueContent()
	[
		BrushSelectors.ToSharedRef()
	];

	LandscapeEditorCategory.AddCustomRow(FText::GetEmpty())
	.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FLandscapeEditorDetails::GetBrushFalloffSelectorVisibility)))
	.NameContent()
	[
		SNew(STextBlock)
		.TextStyle( &FCoreStyle::Get().GetWidgetStyle<FTextBlockStyle>( "SmallText" ) )
		.Text(LOCTEXT("BrushFalloff", "Brush Falloff"))
		.ToolTipText(LOCTEXT("BrushFalloffToolTip", "Selects the profile shape of the brush falloff"))

	]
	.ValueContent()
	[
		FalloffSelectors.ToSharedRef()
	];

	// Tools:
	Customization_NewLandscape = MakeShareable(new FLandscapeEditorDetailCustomization_NewLandscape);
	Customization_NewLandscape->CustomizeDetails(DetailBuilder);
	Customization_ImportExport = MakeShareable(new FLandscapeEditorDetailCustomization_ImportExport);
	Customization_ImportExport->CustomizeDetails(DetailBuilder);
	Customization_ResizeLandscape = MakeShareable(new FLandscapeEditorDetailCustomization_ResizeLandscape);
	Customization_ResizeLandscape->CustomizeDetails(DetailBuilder);
	Customization_CopyPaste = MakeShareable(new FLandscapeEditorDetailCustomization_CopyPaste);
	Customization_CopyPaste->CustomizeDetails(DetailBuilder);
	Customization_MiscTools = MakeShareable(new FLandscapeEditorDetailCustomization_MiscTools);
	Customization_MiscTools->CustomizeDetails(DetailBuilder);

	// Brushes:
	Customization_AlphaBrush = MakeShareable(new FLandscapeEditorDetailCustomization_AlphaBrush);
	Customization_AlphaBrush->CustomizeDetails(DetailBuilder);

	// Hide BlueprintTool/Layers/LayersBrushStack when New Landscape tab is active
	if (LandscapeEdMode->CurrentTool->GetToolName() != FName("NewLandscape"))
	{
		// Blueprint tool
		Customization_Blueprint = MakeShareable(new FLandscapeEditorDetailCustomization_Blueprint);
		Customization_Blueprint->CustomizeDetails(DetailBuilder);

		// Layers
		Customization_EditLayersStack = MakeShareable(new FLandscapeEditorDetailCustomization_EditLayersStack);
		Customization_EditLayersStack->CustomizeDetails(DetailBuilder);

		// Brush Stack
		Customization_LayersBrushStack = MakeShareable(new FLandscapeEditorDetailCustomization_LayersBrushStack);
		Customization_LayersBrushStack->CustomizeDetails(DetailBuilder);
	}

	// Target Layers:
	Customization_TargetLayers = MakeShareable(new FLandscapeEditorDetailCustomization_TargetLayers);
	Customization_TargetLayers->CustomizeDetails(DetailBuilder);
}

EVisibility FLandscapeEditorDetails::GetTargetLandscapeSelectorVisibility()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && (LandscapeEdMode->GetLandscapeList().Num() > 1) && !IsToolActive("NewLandscape"))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

FText FLandscapeEditorDetails::GetLandscapeDisplayName(ALandscape* InLandscape)
{
	FString DisplayString = InLandscape->GetActorLabel();
	if (UWorld* OwningWorld = InLandscape->GetWorld())
	{
		ULevel* Level = InLandscape->GetLevel();
		ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(OwningWorld);
		if ((LevelInstanceSubsystem != nullptr)
			&& (Level != nullptr)
			&& (Level != OwningWorld->PersistentLevel))
		{
			DisplayString = LevelInstanceSubsystem->PrefixWithParentLevelInstanceActorLabels(DisplayString, Level);
		}
	}
	return FText::FromString(DisplayString);
}

FText FLandscapeEditorDetails::GetTargetLandscapeName()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		ULandscapeInfo* Info = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get();
		if (Info)
		{
			if (ALandscape* LandscapeActor = Info->LandscapeActor.Get())
			{
				return GetLandscapeDisplayName(LandscapeActor);
			}
		}
	}
	return FText();
}

TSharedRef<SWidget> FLandscapeEditorDetails::GetTargetLandscapeMenu()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		FMenuBuilder MenuBuilder(true, NULL);

		TArray<ULandscapeInfo*> LandscapeInfoList;

		const TArray<FLandscapeListInfo>& LandscapeList = LandscapeEdMode->GetLandscapeList();
		for (auto It = LandscapeList.CreateConstIterator(); It; It++)
		{
			if (ALandscape* LandscapeActor = It->Info->LandscapeActor.Get())
			{
				LandscapeInfoList.Add(It->Info);
			}
		}

		// Sort landscapes alphabetically : 
		LandscapeInfoList.Sort([](const ULandscapeInfo& InLHS, const ULandscapeInfo& InRHS) { return GetLandscapeDisplayName(InLHS.LandscapeActor.Get()).ToString().Compare(GetLandscapeDisplayName(InRHS.LandscapeActor.Get()).ToString()) < 0;});

		for (ULandscapeInfo* LandscapeInfo : LandscapeInfoList)
		{
			FUIAction Action = FUIAction(FExecuteAction::CreateStatic(&FLandscapeEditorDetails::OnChangeTargetLandscape, MakeWeakObjectPtr(LandscapeInfo)));
			MenuBuilder.AddMenuEntry(GetLandscapeDisplayName(LandscapeInfo->LandscapeActor.Get()), FText(), FSlateIcon(), Action);
		}

		return MenuBuilder.MakeWidget();
	}

	return SNullWidget::NullWidget;
}

void FLandscapeEditorDetails::OnChangeTargetLandscape(TWeakObjectPtr<ULandscapeInfo> LandscapeInfo)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->SetTargetLandscape(LandscapeInfo);
	}
}

FName FLandscapeEditorDetails::GetCurrentBrushFName() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && LandscapeEdMode->CurrentBrush != NULL)
	{
		return LandscapeEdMode->LandscapeBrushSets[LandscapeEdMode->CurrentBrushSetIndex].BrushSetName;
	}

	return NAME_None;
}

bool FLandscapeEditorDetails::GetBrushSelectorIsVisible() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		if (LandscapeEdMode->CurrentTool->ValidBrushes.Num() >= 2)
		{
			return true;
		}
	}

	return false;
}

EVisibility FLandscapeEditorDetails::GetBrushSelectorVisibility() const
{
	return GetBrushSelectorIsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

FName FLandscapeEditorDetails::GetCurrentBrushFalloffFName() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != NULL && LandscapeEdMode->CurrentBrush != NULL && GetBrushFalloffSelectorIsVisible())
	{
		return LandscapeEdMode->CurrentBrush->GetBrushName();
	}

	return NAME_None;
}

void FLandscapeEditorDetails::SetBrushCommand(FName InBrush)
{
	TSharedPtr<FUICommandList> CommandList = GetEditorMode()->GetUICommandList();
	TSharedPtr<FUICommandInfo> BrushCommand = FLandscapeEditorCommands::Get().NameToCommandMap.FindRef(InBrush);
	if (CommandList.IsValid() && BrushCommand.IsValid())
	{
		CommandList->ExecuteAction( BrushCommand.ToSharedRef() );
	}
}

bool FLandscapeEditorDetails::GetBrushFalloffSelectorIsVisible() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentBrush != NULL)
	{
		const FLandscapeBrushSet& CurrentBrushSet = LandscapeEdMode->LandscapeBrushSets[LandscapeEdMode->CurrentBrushSetIndex];

		if (CurrentBrushSet.Brushes.Num() >= 2)
		{
			return true;
		}
	}

	return false;
}

EVisibility FLandscapeEditorDetails::GetBrushFalloffSelectorVisibility() const
{
	return GetBrushFalloffSelectorIsVisible() ? EVisibility::Visible : EVisibility::Collapsed;
}

bool FLandscapeEditorDetails::IsBrushSetEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	return (LandscapeEdMode != nullptr && LandscapeEdMode->GetLandscapeList().Num() > 0);
}

#undef LOCTEXT_NAMESPACE
