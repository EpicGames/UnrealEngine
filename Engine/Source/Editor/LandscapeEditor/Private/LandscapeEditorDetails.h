// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SWidget.h"
#include "UnrealClient.h"
#include "IDetailCustomization.h"
#include "LandscapeEditorDetailCustomization_Base.h"
#include "LandscapeEditorDetailCustomization_EditLayersStack.h"
#include "LandscapeEditorDetailCustomization_LayersBrushStack.h"

class FLandscapeEditorDetailCustomization_AlphaBrush;
class FLandscapeEditorDetailCustomization_Blueprint;
class FLandscapeEditorDetailCustomization_CopyPaste;
class FLandscapeEditorDetailCustomization_MiscTools;
class FLandscapeEditorDetailCustomization_NewLandscape;
class FLandscapeEditorDetailCustomization_ResizeLandscape;
class FLandscapeEditorDetailCustomization_TargetLayers;
class FLandscapeEditorDetailCustomization_ImportExport;
class FUICommandList;
class IDetailLayoutBuilder;
class ULandscapeInfo;
class FLandscapeToolKit;


// 
// FLandscapeEditorDetails
// 
// NOTE: If and when the legacy LandscapeEditor Mode (pre the ToolBar implementation) is removed, 
// this class can cease to inheriting from DetailsCustomization as it will no longer be used as such.
//
// The toolbar implementation directly creates an instances and uses the LandscapeEditorDetails
// to generate and manage the brush and falloff ui.
// 

class FLandscapeEditorDetails : public FLandscapeEditorDetailCustomization_Base
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	static EVisibility GetTargetLandscapeSelectorVisibility();
	static FText GetLandscapeDisplayName(ALandscape* InLandscape);
	static FText GetTargetLandscapeName();
	static TSharedRef<SWidget> GetTargetLandscapeMenu();
	static void OnChangeTargetLandscape(TWeakObjectPtr<ULandscapeInfo> LandscapeInfo);

	FName GetCurrentBrushFName() const;
	bool GetBrushSelectorIsVisible() const;
	EVisibility GetBrushSelectorVisibility() const;

	FName GetCurrentBrushFalloffFName() const;
	bool GetBrushFalloffSelectorIsVisible() const;
	EVisibility GetBrushFalloffSelectorVisibility() const;

	void SetBrushCommand(FName);

	bool IsBrushSetEnabled() const;

	TSharedPtr<FLandscapeEditorDetailCustomization_NewLandscape> Customization_NewLandscape;
	TSharedPtr<FLandscapeEditorDetailCustomization_ResizeLandscape> Customization_ResizeLandscape;
	TSharedPtr<FLandscapeEditorDetailCustomization_ImportExport> Customization_ImportExport;
	TSharedPtr<FLandscapeEditorDetailCustomization_CopyPaste> Customization_CopyPaste;
	TSharedPtr<FLandscapeEditorDetailCustomization_MiscTools> Customization_MiscTools;
	TSharedPtr<FLandscapeEditorDetailCustomization_AlphaBrush> Customization_AlphaBrush;
	TSharedPtr<FLandscapeEditorDetailCustomization_Blueprint> Customization_Blueprint;
	TSharedPtr<FLandscapeEditorDetailCustomization_TargetLayers> Customization_TargetLayers;
	TSharedPtr<FLandscapeEditorDetailCustomization_EditLayersStack> Customization_EditLayersStack;
	TSharedPtr<FLandscapeEditorDetailCustomization_LayersBrushStack> Customization_LayersBrushStack;
};
