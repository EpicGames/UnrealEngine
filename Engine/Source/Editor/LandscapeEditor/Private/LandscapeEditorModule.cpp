// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorModule.h"

#include "Modules/ModuleManager.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Styling/AppStyle.h"
#include "EditorModeRegistry.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "EditorWidgetsModule.h"
#include "ObjectNameEditSinkRegistry.h"
#include "LandscapeEditLayerObjectNameEditSink.h"

#include "LandscapeFileFormatInterface.h"
#include "LandscapeProxy.h"
#include "LandscapeEdMode.h"
#include "Landscape.h"
#include "LandscapeUtils.h"
#include "LandscapeEditorCommands.h"
#include "Classes/ActorFactoryLandscape.h"
#include "LandscapeFileFormatPng.h"
#include "LandscapeFileFormatRaw.h"
#include "LandscapeEditorServices.h"
#include "LandscapeEditTypes.h"
#include "LandscapeImageFileCache.h"
#include "LandscapeSettings.h"
#include "SLandscapeLayerListDialog.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyEditorModule.h"
#include "LandscapeEditorDetails.h"
#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorDetailCustomization_CopyPaste.h"
#include "LandscapeEditorDetailCustomization_ImportLayers.h"
#include "LandscapeEditorDetailCustomization_TargetLayers.h"
#include "LandscapeEditorDetailCustomization_EditLayers.h"
#include "LandscapeEditLayerCustomization.h"
#include "LandscapeSplineDetails.h"
#include "LandscapeGrassTypeDetails.h"
#include "LandscapeProxyUIDetails.h"
#include "LandscapeModule.h"

#include "LevelEditor.h"
#include "Filters/CustomClassFilterData.h"
#include "ToolMenus.h"
#include "Editor/EditorEngine.h"
#include "LandscapeSubsystem.h"

#include "LandscapeRender.h"
#include "UObject/ObjectSaveContext.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"

struct FRegisteredLandscapeHeightmapFileFormat
{
	TSharedRef<ILandscapeHeightmapFileFormat> FileFormat;
	FLandscapeFileTypeInfo FileTypeInfo;
	FString ConcatenatedFileExtensions;

	FRegisteredLandscapeHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> InFileFormat);
};

struct FRegisteredLandscapeWeightmapFileFormat
{
	TSharedRef<ILandscapeWeightmapFileFormat> FileFormat;
	FLandscapeFileTypeInfo FileTypeInfo;
	FString ConcatenatedFileExtensions;

	FRegisteredLandscapeWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> InFileFormat);
};

class FLandscapeEditorModule : public ILandscapeEditorModule, public ILandscapeEditorServices
{
public:

	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override
	{
		FLandscapeEditorCommands::Register();

		// register the editor mode
		FEditorModeRegistry::Get().RegisterMode<FEdModeLandscape>(
			FBuiltinEditorModes::EM_Landscape,
			NSLOCTEXT("EditorModes", "LandscapeMode", "Landscape"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.LandscapeMode", "LevelEditor.LandscapeMode.Small"),
			true,
			300
			);

		// register custom editor widgets 
		FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
		EditorWidgetsModule.GetObjectNameEditSinkRegistry()->RegisterObjectNameEditSink(MakeShared<FLandscapeEditLayerObjectNameEditSink>());

		// register customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout("LandscapeEditorObject", FOnGetDetailCustomizationInstance::CreateStatic(&FLandscapeEditorDetails::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("GizmoImportLayer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLandscapeEditorStructCustomization_FGizmoImportLayer::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("LandscapeImportLayer", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLandscapeEditorStructCustomization_FLandscapeImportLayer::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("LandscapeTargetLayerAssetFilePath", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLandscapeEditorStructCustomization_FTargetLayerAssetPath::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("LandscapeGrassName", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FLandscapeGrassNameCustomization::MakeInstance));

		PropertyModule.RegisterCustomClassLayout("LandscapeSplineControlPoint", FOnGetDetailCustomizationInstance::CreateStatic(&FLandscapeSplineDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("LandscapeSplineSegment", FOnGetDetailCustomizationInstance::CreateStatic(&FLandscapeSplineDetails::MakeInstance));

		PropertyModule.RegisterCustomClassLayout("LandscapeProxy", FOnGetDetailCustomizationInstance::CreateStatic(&FLandscapeProxyUIDetails::MakeInstance));
		PropertyModule.RegisterCustomClassLayout("LandscapeGrassType", FOnGetDetailCustomizationInstance::CreateStatic(&FLandscapeGrassTypeDetails::MakeInstance));

		// register edit layer customizations (Ex. custom Context Menu)
		RegisterCustomEditLayerClassLayout("LandscapeEditLayerBase", FOnGetLandscapeEditLayerCustomizationInstance::CreateStatic(&FLandscapeEditLayerContextMenuCustomization_Base::MakeInstance));
		RegisterCustomEditLayerClassLayout("LandscapeEditLayer", FOnGetLandscapeEditLayerCustomizationInstance::CreateStatic(&FLandscapeEditLayerContextMenuCustomization_Layer::MakeInstance));
		RegisterCustomEditLayerClassLayout("LandscapeEditLayerPersistent", FOnGetLandscapeEditLayerCustomizationInstance::CreateStatic(&FLandscapeEditLayerContextMenuCustomization_Persistent::MakeInstance));
		RegisterCustomEditLayerClassLayout("LandscapeEditLayerSplines", FOnGetLandscapeEditLayerCustomizationInstance::CreateStatic(&FLandscapeEditLayerContextMenuCustomization_Splines::MakeInstance));

		// register property sections
		RegisterPropertySectionMappings();
		
		GlobalUICommandList = MakeShareable(new FUICommandList);
		const FLandscapeEditorCommands& LandscapeActions = FLandscapeEditorCommands::Get();
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeNormal, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::Normal), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::Normal));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeLOD, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::LOD), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::LOD));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeLayerDensity, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::LayerDensity), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::LayerDensity));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeLayerDebug, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::DebugLayer), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::DebugLayer));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeWireframeOnTop, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::WireframeOnTop), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::WireframeOnTop));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeLayerUsage, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::LayerUsage), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::LayerUsage));
		GlobalUICommandList->MapAction(LandscapeActions.ViewModeLayerContribution, FExecuteAction::CreateStatic(&ChangeLandscapeViewMode, ELandscapeViewMode::LayerContribution), FCanExecuteAction(), FIsActionChecked::CreateStatic(&IsLandscapeViewModeSelected, ELandscapeViewMode::LayerContribution));

		// add menu extension
		UToolMenu* ViewportMenu = UToolMenus::Get()->ExtendMenu("UnrealEd.ViewportToolbar.View");
		FToolMenuSection& LandscapeSection = ViewportMenu->FindOrAddSection("LevelViewportLandscape");
		LandscapeSection.AddSubMenu(
			"LandscapeVisualizers",
			LOCTEXT("LandscapeSubMenu", "Visualizers"), 
			LOCTEXT("LandscapeSubMenu_ToolTip", "Select a landscape visualizer"), 
			FNewToolMenuDelegate::CreateRaw(this, &FLandscapeEditorModule::ConstructLandscapeViewportMenu),
			/*bInOpenSubMenuOnClick = */false,
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "EditorViewport.Visualizers"));

		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		// Add Level Editor Outliner Filter
		if (TSharedPtr<FFilterCategory> EnvironmentFilterCategory = LevelEditorModule.GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::Environment()))
		{
			TSharedRef<FCustomClassFilterData> LandscapeActorClassData = MakeShared<FCustomClassFilterData>(ALandscape::StaticClass(), EnvironmentFilterCategory, FLinearColor::White);
			LevelEditorModule.AddCustomClassFilterToOutliner(LandscapeActorClassData);
		}
		
		// add actor factories
		UActorFactoryLandscape* LandscapeActorFactory = NewObject<UActorFactoryLandscape>();
		LandscapeActorFactory->NewActorClass = ALandscape::StaticClass();
		GEditor->ActorFactories.Add(LandscapeActorFactory);

		UActorFactoryLandscape* LandscapeProxyActorFactory = NewObject<UActorFactoryLandscape>();
		LandscapeProxyActorFactory->NewActorClass = ALandscapeProxy::StaticClass();
		GEditor->ActorFactories.Add(LandscapeProxyActorFactory);

		// Built-in File Formats
		RegisterHeightmapFileFormat(MakeShareable(new FLandscapeHeightmapFileFormat_Png()));
		RegisterWeightmapFileFormat(MakeShareable(new FLandscapeWeightmapFileFormat_Png()));
		RegisterHeightmapFileFormat(MakeShareable(new FLandscapeHeightmapFileFormat_Raw()));
		RegisterWeightmapFileFormat(MakeShareable(new FLandscapeWeightmapFileFormat_Raw()));

		//Landscape extended menu
		UToolMenu* BuildMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.MainMenu.Build");
		if (BuildMenu)
		{
			FToolMenuSection& Section = BuildMenu->FindOrAddSection("LevelEditorLandscape");

			FUIAction ActionSaveModifiedLandscapes(FExecuteAction::CreateStatic(&UE::Landscape::SaveModifiedLandscapes, UE::Landscape::EBuildFlags::WriteFinalLog), FCanExecuteAction::CreateStatic(&UE::Landscape::HasModifiedLandscapes));
			Section.AddMenuEntry(NAME_None,
				LOCTEXT("SaveModifiedLandscapes", "Save Modified Landscapes"), LOCTEXT("SaveModifiedLandscapesToolTip", "Save landscapes that were modified outside of the editor mode"),
				TAttribute<FSlateIcon>(), ActionSaveModifiedLandscapes, EUserInterfaceActionType::Button);
		}
		
		ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
		LandscapeModule.SetLandscapeEditorServices(this);

		LandscapeImageFileCache.Reset(new FLandscapeImageFileCache());
	}

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override
	{
		FLandscapeEditorCommands::Unregister();

		// unregister the editor mode
		FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_Landscape);

		// unregister custom editor widgets 
		FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
		EditorWidgetsModule.GetObjectNameEditSinkRegistry()->UnregisterObjectNameEditSink(MakeShared<FLandscapeEditLayerObjectNameEditSink>());

		// unregister customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.UnregisterCustomClassLayout("LandscapeEditorObject");
		PropertyModule.UnregisterCustomPropertyTypeLayout("GizmoImportLayer");
		PropertyModule.UnregisterCustomPropertyTypeLayout("LandscapeImportLayer");
		PropertyModule.UnregisterCustomPropertyTypeLayout("LandscapeTargetLayerAssetFilePath");
		PropertyModule.UnregisterCustomPropertyTypeLayout("LandscapeGrassName");

		PropertyModule.UnregisterCustomClassLayout("LandscapeSplineControlPoint");
		PropertyModule.UnregisterCustomClassLayout("LandscapeSplineSegment");

		PropertyModule.UnregisterCustomClassLayout("LandscapeProxy");
		PropertyModule.UnregisterCustomClassLayout("LandscapeGrassType");

		// unregister edit layer customizations (Ex. custom Context Menu)
		UnregisterCustomEditLayerClassLayout("LandscapeEditLayerBase");
		UnregisterCustomEditLayerClassLayout("LandscapeEditLayer");
		UnregisterCustomEditLayerClassLayout("LandscapeEditLayerPersistent");
		UnregisterCustomEditLayerClassLayout("LandscapeEditLayerSplines");

		// unregister property sections
		UnregisterPropertySectionMappings();

		// remove menu extension
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		GlobalUICommandList = nullptr;

		// remove actor factories
		// TODO - this crashes on shutdown
		// GEditor->ActorFactories.RemoveAll([](const UActorFactory* ActorFactory) { return ActorFactory->IsA<UActorFactoryLandscape>(); });

		ILandscapeModule& LandscapeModule = FModuleManager::GetModuleChecked<ILandscapeModule>("Landscape");
		if (LandscapeModule.GetLandscapeEditorServices() == this)
		{
			LandscapeModule.SetLandscapeEditorServices(nullptr);
		}
		LandscapeImageFileCache.Reset();
	}

	void ConstructLandscapeViewportMenu(UToolMenu* InMenu)
	{
		const FLandscapeEditorCommands& LandscapeActions = FLandscapeEditorCommands::Get();
		FToolMenuSection& MenuSection = InMenu->AddSection("LandscapeVisualizers", LOCTEXT("LandscapeHeader", "Landscape Visualizers"));
		MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeNormal, GlobalUICommandList, LOCTEXT("LandscapeViewModeNormal", "Normal"));
		MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeLOD, GlobalUICommandList, LOCTEXT("LandscapeViewModeLOD", "LOD"));
		MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeLayerDensity, GlobalUICommandList, LOCTEXT("LandscapeViewModeLayerDensity", "Layer Density"));
		MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeLayerUsage, GlobalUICommandList, LOCTEXT("LandscapeViewModeLayerUsage", "Layer Usage"));

		if (GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Landscape))
		{
			MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeLayerDebug, GlobalUICommandList, LOCTEXT("LandscapeViewModeLayerDebug", "Layer Debug"));
			FEdModeLandscape* LandscapeMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
			MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeLayerContribution, GlobalUICommandList, LOCTEXT("LandscapeViewModeLayerContribution", "Layer Contribution"));
		}
		MenuSection.AddMenuEntryWithCommandList(LandscapeActions.ViewModeWireframeOnTop, GlobalUICommandList, LOCTEXT("LandscapeViewModeWireframeOnTop", "Wireframe on Top"));
	}

	static void ChangeLandscapeViewMode(ELandscapeViewMode::Type ViewMode)
	{
		if (ViewMode != GLandscapeViewMode)
		{
			GLandscapeViewMode = ViewMode;

			if (GEditor)
			{
				GEditor->RedrawAllViewports(/*bInvalidateHitProxies =*/false);
			}
		}
	}

	static bool IsLandscapeViewModeSelected(ELandscapeViewMode::Type ViewMode)
	{
		return GLandscapeViewMode == ViewMode;
	}

	/**
	 * ILandscapeEditorModule implementation
	 */
	virtual void RegisterHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> FileFormat) override
	{
		HeightmapFormats.Emplace(FileFormat);
		HeightmapImportDialogTypeString.Reset();
		HeightmapExportDialogTypeString.Reset();
	}

	virtual void RegisterWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> FileFormat) override
	{
		WeightmapFormats.Emplace(FileFormat);
		WeightmapImportDialogTypeString.Reset();
		WeightmapExportDialogTypeString.Reset();
	}

	virtual void UnregisterHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> FileFormat) override
	{
		int32 Index = HeightmapFormats.IndexOfByPredicate(
			[FileFormat](const FRegisteredLandscapeHeightmapFileFormat& RegisteredFileFormat)
			{
				return RegisteredFileFormat.FileFormat == FileFormat;
			});
		if (Index != INDEX_NONE)
		{
			HeightmapFormats.RemoveAt(Index);
			HeightmapImportDialogTypeString.Reset();
			HeightmapExportDialogTypeString.Reset();
		}
	}

	virtual void UnregisterWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> FileFormat) override
	{
		int32 Index = WeightmapFormats.IndexOfByPredicate(
			[FileFormat](const FRegisteredLandscapeWeightmapFileFormat& RegisteredFileFormat)
			{
				return RegisteredFileFormat.FileFormat == FileFormat;
			});
		if (Index != INDEX_NONE)
		{
			WeightmapFormats.RemoveAt(Index);
			WeightmapImportDialogTypeString.Reset();
			WeightmapExportDialogTypeString.Reset();
		}
	}

	void RegisterCustomEditLayerClassLayout(FName ClassName, FOnGetLandscapeEditLayerCustomizationInstance EditLayerLayoutDelegate) override
	{
		if (ClassName != NAME_None)
		{
			FOnGetLandscapeEditLayerCustomizationInstance Callback;
			Callback = EditLayerLayoutDelegate;

			ClassNameToEditLayerCustomizationMap.Add(ClassName, Callback);
		}
	}

	void UnregisterCustomEditLayerClassLayout(FName ClassName)
	{
		if (ClassName.IsValid() && (ClassName != NAME_None))
		{
			ClassNameToEditLayerCustomizationMap.Remove(ClassName);
		}
	}

	TArray<TSharedRef<IEditLayerCustomization>> QueryCustomEditLayerLayoutRecursive(UStruct* InClass)
	{
		TArray<TSharedRef<IEditLayerCustomization>> EditLayerCustomizationClassInstances;
		if (InClass == nullptr)
		{
			return EditLayerCustomizationClassInstances;
		}

		TSet<UStruct*> ClassesToQuery;
	
		// Ensure that the base class and its parents are always queried
		ClassesToQuery.Add(InClass);

		UStruct* ParentStruct = InClass->GetSuperStruct();
		while (ParentStruct && ParentStruct->IsA(UClass::StaticClass()) && !ClassesToQuery.Contains(ParentStruct))
		{
			ClassesToQuery.Add(ParentStruct);
			ParentStruct = ParentStruct->GetSuperStruct();
		}

		// Query class inheritance chain
		for (auto ParentIt = ClassesToQuery.CreateConstIterator(); ParentIt; ++ParentIt)
		{
			if (TSharedPtr<IEditLayerCustomization> CustomizationInstance = QueryCustomEditLayerLayoutForClass(*ParentIt))
			{
				EditLayerCustomizationClassInstances.Add(CustomizationInstance.ToSharedRef());
			} 
		}

		return EditLayerCustomizationClassInstances;
	}

	TSharedPtr<IEditLayerCustomization> QueryCustomEditLayerLayoutForClass(UStruct* InClass)
	{
		const FOnGetLandscapeEditLayerCustomizationInstance* CustomizationDelegate = ClassNameToEditLayerCustomizationMap.Find(InClass->GetFName());

		if (CustomizationDelegate && CustomizationDelegate->IsBound())
		{
			// Create a new instance of the customization for the current edit layer class
			TSharedPtr<IEditLayerCustomization> CustomizationInstance = CustomizationDelegate->Execute();

			// Caller's responsibility to handle the instance lifetime
			return CustomizationInstance;
		}

		return nullptr;
	}

	void ApplyEditLayerContextMenuCustomizations(ULandscapeEditLayerBase* InEditLayer, const TArray<TSharedRef<IEditLayerCustomization>>& InCustomizations, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap)
	{
		if (InEditLayer == nullptr || InCustomizations.IsEmpty())
		{
			return;
		}

		for (TSharedRef<IEditLayerCustomization> CustomizationInstance : InCustomizations)
		{
			CustomizationInstance->CustomizeContextMenu(InEditLayer, OutContextMenuEntryMap);
		}
	}

	virtual const TCHAR* GetHeightmapImportDialogTypeString() const override;
	virtual const TCHAR* GetWeightmapImportDialogTypeString() const override;

	virtual const TCHAR* GetHeightmapExportDialogTypeString() const override;
	virtual const TCHAR* GetWeightmapExportDialogTypeString() const override;

	virtual const ILandscapeHeightmapFileFormat* GetHeightmapFormatByExtension(const TCHAR* Extension) const override;
	virtual const ILandscapeWeightmapFileFormat* GetWeightmapFormatByExtension(const TCHAR* Extension) const override;

	virtual TSharedPtr<FUICommandList> GetLandscapeLevelViewportCommandList() const override;

	FLandscapeImageFileCache& GetImageFileCache() const override;

	/**
	* ILandscapeEditorServices implementation
	*/
	virtual int32 GetOrCreateEditLayer(FName InEditLayerName, ALandscape* InTargetLandscape, const TSubclassOf<ULandscapeEditLayerBase>& InEditLayerClass) override;
	virtual void RefreshDetailPanel() override;
	virtual void RegenerateLayerThumbnails() override;

private:
	TSharedRef<FPropertySection> RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName);
	void RegisterPropertySectionMappings();
	void UnregisterPropertySectionMappings();

protected:
	TSharedPtr<FUICommandList> GlobalUICommandList;
	TArray<FRegisteredLandscapeHeightmapFileFormat> HeightmapFormats;
	TArray<FRegisteredLandscapeWeightmapFileFormat> WeightmapFormats;
	mutable FString HeightmapImportDialogTypeString;
	mutable FString WeightmapImportDialogTypeString;
	mutable FString HeightmapExportDialogTypeString;
	mutable FString WeightmapExportDialogTypeString;
	TUniquePtr<FLandscapeImageFileCache> LandscapeImageFileCache;

	TMultiMap<FName, FName> RegisteredPropertySections;
	TArray<FName> PropertyTypesToUnregisterOnShutdown;

	/** A mapping of class names to edit layer delegates, called when querying for custom edit layer layouts */
	FCustomEditLayerLayoutNameMap ClassNameToEditLayerCustomizationMap;
};

IMPLEMENT_MODULE(FLandscapeEditorModule, LandscapeEditor);

FRegisteredLandscapeHeightmapFileFormat::FRegisteredLandscapeHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> InFileFormat)
	: FileFormat(MoveTemp(InFileFormat))
	, FileTypeInfo(FileFormat->GetInfo())
{
	bool bJoin = false;
	for (const FString& Extension : FileTypeInfo.Extensions)
	{
		if (bJoin)
		{
			ConcatenatedFileExtensions += TEXT(';');
		}
		ConcatenatedFileExtensions += TEXT('*');
		ConcatenatedFileExtensions += Extension;
		bJoin = true;
	}
}

FRegisteredLandscapeWeightmapFileFormat::FRegisteredLandscapeWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> InFileFormat)
	: FileFormat(MoveTemp(InFileFormat))
	, FileTypeInfo(FileFormat->GetInfo())
{
	bool bJoin = false;
	for (const FString& Extension : FileTypeInfo.Extensions)
	{
		if (bJoin)
		{
			ConcatenatedFileExtensions += TEXT(';');
		}
		ConcatenatedFileExtensions += TEXT('*');
		ConcatenatedFileExtensions += Extension;
		bJoin = true;
	}
}

TSharedRef<FPropertySection> FLandscapeEditorModule::RegisterPropertySection(FPropertyEditorModule& PropertyModule, FName ClassName, FName SectionName, FText DisplayName)
{
	TSharedRef<FPropertySection> PropertySection = PropertyModule.FindOrCreateSection(ClassName, SectionName, DisplayName);
	RegisteredPropertySections.Add(ClassName, SectionName);

	return PropertySection;
}

void FLandscapeEditorModule::RegisterPropertySectionMappings()
{
	const FName PropertyEditorModuleName("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

	// Segment Physics
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "LandscapeSplineSegment", "Physics", LOCTEXT("Physics", "Physics"));
		Section->AddCategory("Collision");
		Section->AddCategory("Physics");
	}

	// Segment Rendering
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "LandscapeSplineSegment", "Rendering", LOCTEXT("Rendering", "Rendering"));
		Section->AddCategory("VirtualTexture");
		Section->AddCategory("Mesh");
		Section->AddCategory("LandscapeSplineMeshes");
		Section->AddCategory("LandscapeSplineMeshEntry");
		Section->AddCategory("Rendering");
	}

	// Control Point Physics
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "LandscapeSplineControlPoint", "Physics", LOCTEXT("Physics", "Physics"));
		Section->AddCategory("Collision");
		Section->AddCategory("Physics");
	}

	// Control Point Rendering
	{
		TSharedRef<FPropertySection> Section = RegisterPropertySection(PropertyModule, "LandscapeSplineControlPoint", "Rendering", LOCTEXT("Rendering", "Rendering"));
		Section->AddCategory("VirtualTexture");
		Section->AddCategory("Mesh");
		Section->AddCategory("Rendering");
	}
}

void FLandscapeEditorModule::UnregisterPropertySectionMappings()
{
	const FName PropertyEditorModuleName("PropertyEditor");
	FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>(PropertyEditorModuleName);

	if (!PropertyModule)
	{
		return;
	}

	for (TMultiMap<FName, FName>::TIterator PropertySectionIterator = RegisteredPropertySections.CreateIterator(); PropertySectionIterator; ++PropertySectionIterator)
	{
		PropertyModule->RemoveSection(PropertySectionIterator->Key, PropertySectionIterator->Value);
		PropertySectionIterator.RemoveCurrent();
	}

	check(RegisteredPropertySections.IsEmpty());
}

const TCHAR* FLandscapeEditorModule::GetHeightmapImportDialogTypeString() const
{
	if (HeightmapImportDialogTypeString.IsEmpty())
	{
		HeightmapImportDialogTypeString = TEXT("All Heightmap files|");
		bool bJoin = false;
		for (const FRegisteredLandscapeHeightmapFileFormat& HeightmapFormat : HeightmapFormats)
		{
			if (bJoin)
			{
				HeightmapImportDialogTypeString += TEXT(';');
			}
			HeightmapImportDialogTypeString += HeightmapFormat.ConcatenatedFileExtensions;
			bJoin = true;
		}
		HeightmapImportDialogTypeString += TEXT('|');
		for (const FRegisteredLandscapeHeightmapFileFormat& HeightmapFormat : HeightmapFormats)
		{
			HeightmapImportDialogTypeString += HeightmapFormat.FileTypeInfo.Description.ToString();
			HeightmapImportDialogTypeString += TEXT('|');
			HeightmapImportDialogTypeString += HeightmapFormat.ConcatenatedFileExtensions;
			HeightmapImportDialogTypeString += TEXT('|');
		}
		HeightmapImportDialogTypeString += TEXT("All Files (*.*)|*.*");
	}

	return *HeightmapImportDialogTypeString;
}

const TCHAR* FLandscapeEditorModule::GetWeightmapImportDialogTypeString() const
{
	if (WeightmapImportDialogTypeString.IsEmpty())
	{
		WeightmapImportDialogTypeString = TEXT("All Layer files|");
		bool bJoin = false;
		for (const FRegisteredLandscapeWeightmapFileFormat& WeightmapFormat : WeightmapFormats)
		{
			if (bJoin)
			{
				WeightmapImportDialogTypeString += TEXT(';');
			}
			WeightmapImportDialogTypeString += WeightmapFormat.ConcatenatedFileExtensions;
			bJoin = true;
		}
		WeightmapImportDialogTypeString += TEXT('|');
		for (const FRegisteredLandscapeWeightmapFileFormat& WeightmapFormat : WeightmapFormats)
		{
			WeightmapImportDialogTypeString += WeightmapFormat.FileTypeInfo.Description.ToString();
			WeightmapImportDialogTypeString += TEXT('|');
			WeightmapImportDialogTypeString += WeightmapFormat.ConcatenatedFileExtensions;
			WeightmapImportDialogTypeString += TEXT('|');
		}
		WeightmapImportDialogTypeString += TEXT("All Files (*.*)|*.*");
	}

	return *WeightmapImportDialogTypeString;
}

const TCHAR* FLandscapeEditorModule::GetHeightmapExportDialogTypeString() const
{
	if (HeightmapExportDialogTypeString.IsEmpty())
	{
		HeightmapExportDialogTypeString = TEXT("");

		for (const FRegisteredLandscapeHeightmapFileFormat& HeightmapFormat : HeightmapFormats)
		{
			if (!HeightmapFormat.FileTypeInfo.bSupportsExport)
			{
				continue;
			}
			HeightmapExportDialogTypeString += HeightmapFormat.FileTypeInfo.Description.ToString();
			HeightmapExportDialogTypeString += TEXT('|');
			HeightmapExportDialogTypeString += HeightmapFormat.ConcatenatedFileExtensions;
			HeightmapExportDialogTypeString += TEXT('|');
		}
		HeightmapExportDialogTypeString += TEXT("All Files (*.*)|*.*");
	}

	return *HeightmapExportDialogTypeString;
}

const TCHAR* FLandscapeEditorModule::GetWeightmapExportDialogTypeString() const
{
	if (WeightmapExportDialogTypeString.IsEmpty())
	{
		WeightmapExportDialogTypeString = TEXT("");
		for (const FRegisteredLandscapeWeightmapFileFormat& WeightmapFormat : WeightmapFormats)
		{
			if (!WeightmapFormat.FileTypeInfo.bSupportsExport)
			{
				continue;
			}
			WeightmapExportDialogTypeString += WeightmapFormat.FileTypeInfo.Description.ToString();
			WeightmapExportDialogTypeString += TEXT('|');
			WeightmapExportDialogTypeString += WeightmapFormat.ConcatenatedFileExtensions;
			WeightmapExportDialogTypeString += TEXT('|');
		}
		WeightmapExportDialogTypeString += TEXT("All Files (*.*)|*.*");
	}

	return *WeightmapExportDialogTypeString;
}

const ILandscapeHeightmapFileFormat* FLandscapeEditorModule::GetHeightmapFormatByExtension(const TCHAR* Extension) const
{
	auto* FoundFormat = HeightmapFormats.FindByPredicate(
		[Extension](const FRegisteredLandscapeHeightmapFileFormat& HeightmapFormat)
		{
			return HeightmapFormat.FileTypeInfo.Extensions.Contains(Extension);
		});

	return FoundFormat ? &FoundFormat->FileFormat.Get() : nullptr;
}

const ILandscapeWeightmapFileFormat* FLandscapeEditorModule::GetWeightmapFormatByExtension(const TCHAR* Extension) const
{
	auto* FoundFormat = WeightmapFormats.FindByPredicate(
		[Extension](const FRegisteredLandscapeWeightmapFileFormat& WeightmapFormat)
	{
		return WeightmapFormat.FileTypeInfo.Extensions.Contains(Extension);
	});

	return FoundFormat ? &FoundFormat->FileFormat.Get() : nullptr;
}

TSharedPtr<FUICommandList> FLandscapeEditorModule::GetLandscapeLevelViewportCommandList() const
{
	return GlobalUICommandList;
}

FLandscapeImageFileCache& FLandscapeEditorModule::GetImageFileCache() const
{
	check(LandscapeImageFileCache != nullptr);
	return *LandscapeImageFileCache;
}

int32 FLandscapeEditorModule::GetOrCreateEditLayer(FName InEditLayerName, ALandscape* InTargetLandscape, const TSubclassOf<ULandscapeEditLayerBase>& InEditLayerClass)
{
	// Insertion logic is left to the user through modal drag + drop dialog : 
	int32 ExistingLayerIndex = InTargetLandscape->GetLayerIndex(InEditLayerName);
	if (ExistingLayerIndex == INDEX_NONE)
	{
		ExistingLayerIndex = InTargetLandscape->CreateLayer(InEditLayerName, InEditLayerClass);

		const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
		if (Settings && Settings->bShowDialogForAutomaticLayerCreation)
		{
			TSharedPtr<SLandscapeLayerListDialog> Dialog = SNew(SLandscapeLayerListDialog, InTargetLandscape);
			Dialog->ShowModal();
			ExistingLayerIndex = Dialog->GetInsertedLayerIndex();
		}
	}

	RefreshDetailPanel();

	return ExistingLayerIndex;
}

void FLandscapeEditorModule::RefreshDetailPanel()
{
	if (FEdModeLandscape* LandscapeMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape))
	{
		LandscapeMode->RefreshDetailPanel();
	}
}

void FLandscapeEditorModule::RegenerateLayerThumbnails()
{
	if (FEdModeLandscape* LandscapeMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape))
	{
		LandscapeMode->RegenerateLayerThumbnails();
	}
}

#undef LOCTEXT_NAMESPACE
