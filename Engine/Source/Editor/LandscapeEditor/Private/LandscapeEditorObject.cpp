// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorObject.h"
#include "LandscapeDataAccess.h"
#include "Engine/Texture2D.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "UObject/ConstructorHelpers.h"
#include "LandscapeEditorModule.h"
#include "LandscapeEditorPrivate.h"
#include "LandscapeEdMode.h"
#include "LandscapeRender.h"
#include "LandscapeSettings.h"
#include "LandscapeImportHelper.h"
#include "LandscapeTiledImage.h"
#include "LandscapeMaterialInstanceConstant.h"
#include "LandscapeUtils.h"
#include "LandscapeEditorUtils.h"
#include "Misc/ConfigCacheIni.h"
#include "EngineUtils.h"
#include "RenderUtils.h"
#include "Misc/MessageDialog.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeEditorObject)

static TAutoConsoleVariable<bool> CVarLandscapeSimulateAlphaBrushTextureLoadFailure(
	TEXT("landscape.SimulateAlphaBrushTextureLoadFailure"),
	false,
	TEXT("Debug utility to simulate a loading failure (e.g. invalid source data, which can happen in cooked editor or with a badly virtualized texture) when loading the alpha brush texture"));

const FVector ULandscapeEditorObject::NewLandscape_DefaultLocation = FVector(0, 0, 0);
const FRotator ULandscapeEditorObject::NewLandscape_DefaultRotation = FRotator::ZeroRotator;
const FVector ULandscapeEditorObject::NewLandscape_DefaultScale = FVector(100, 100, 100);

ULandscapeEditorObject::ULandscapeEditorObject(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)

	// Tool Settings:
	, ToolStrength(0.3f)
    , PaintToolStrength(0.3f)
	, bUseWeightTargetValue(false)
	, WeightTargetValue(1.0f)
	, MaximumValueRadius(10000.0f)
	, bCombinedLayersOperation(true)

	, FlattenMode(ELandscapeToolFlattenMode::Both)
	, bUseSlopeFlatten(false)
	, bPickValuePerApply(false)
	, bUseFlattenTarget(false)
	, FlattenTarget(0)
	, bShowFlattenTargetPreview(true)
	
	, TerraceInterval(1.0f)
	, TerraceSmooth(0.0001f)

	, RampWidth(2000)
	, RampSideFalloff(0.4f)

	, SmoothFilterKernelSize(4)
	, bDetailSmooth(false)
	, DetailScale(0.3f)

	, ErodeThresh(64)
	, ErodeSurfaceThickness(256)
	, ErodeIterationNum(28)
	, ErosionNoiseMode(ELandscapeToolErosionMode::Lower)
	, ErosionNoiseScale(60.0f)
	, bErosionUseLayerHardness(false)

	, RainAmount(128)
	, SedimentCapacity(0.3f)
	, HErodeIterationNum(75)
	, RainDistMode(ELandscapeToolHydroErosionMode::Both)
	, RainDistScale(60.0f)
	, bHErosionDetailSmooth(true)
	, HErosionDetailScale(0.01f)

	, NoiseMode(ELandscapeToolNoiseMode::Both)
	, NoiseScale(128.0f)

	, bUseSelectedRegion(true)
	, bUseNegativeMask(true)

	, PasteMode(ELandscapeToolPasteMode::Both)
	, bApplyToAllTargets(true)
	, SnapMode(ELandscapeGizmoSnapType::None)
	, bSmoothGizmoBrush(true)

	, MirrorPoint(FVector::ZeroVector)
	, MirrorOp(ELandscapeMirrorOperation::MinusXToPlusX)

	, ResizeLandscape_QuadsPerSection(0)
	, ResizeLandscape_SectionsPerComponent(0)
	, ResizeLandscape_ComponentCount(0, 0)
	, ResizeLandscape_ConvertMode(ELandscapeConvertMode::Expand)

	, NewLandscape_Material(nullptr)
	, NewLandscape_QuadsPerSection(63)
	, NewLandscape_SectionsPerComponent(1)
	, NewLandscape_ComponentCount(8, 8)
	, NewLandscape_Location(NewLandscape_DefaultLocation)
	, NewLandscape_Rotation(NewLandscape_DefaultRotation)
	, NewLandscape_Scale(NewLandscape_DefaultScale)
	, ImportLandscape_Width(0)
	, ImportLandscape_Height(0)
	, ImportLandscape_AlphamapType(ELandscapeImportAlphamapType::Additive)

	// Brush Settings:
	, BrushRadius(2048.0f)
    , PaintBrushRadius(2048.0f) 
	, BrushFalloff(0.5f)
	, PaintBrushFalloff(0.5f)
	, bUseClayBrush(false)
	, bApplyWithoutMovingSculpt(true)
	, bApplyWithoutMovingPaint(false)

	, AlphaBrushScale(0.5f)
	, bAlphaBrushAutoRotate(true)
	, AlphaBrushRotation(0.0f)
	, AlphaBrushPanU(0.5f)
	, AlphaBrushPanV(0.5f)
	, bUseWorldSpacePatternBrush(false)
	, WorldSpacePatternBrushSettings(FVector2D::ZeroVector, 0.0f, false, 3200)
	, AlphaTexture(nullptr)
	, AlphaTextureChannel(ELandscapeTextureColorChannel::Red)
	, AlphaTextureSizeX(1)
	, AlphaTextureSizeY(1)

	, BrushComponentSize(1)
	, TargetDisplayOrder(ELandscapeLayerDisplayMode::Default)
	, bTargetDisplayOrdersAscending(true)
	, ShowUnusedLayers(true)
{
	// Structure to hold one-time initialization
	struct FConstructorStatics
	{
		ConstructorHelpers::FObjectFinder<UTexture2D> AlphaTexture;

		FConstructorStatics()
			: AlphaTexture(TEXT("/Engine/EditorLandscapeResources/DefaultAlphaTexture"))
		{
		}
	};
	static FConstructorStatics ConstructorStatics;

	SetAlphaTexture(ConstructorStatics.AlphaTexture.Object, AlphaTextureChannel);
}

void ULandscapeEditorObject::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	SetbUseSelectedRegion(bUseSelectedRegion);
	SetbUseNegativeMask(bUseNegativeMask);
	SetPasteMode(PasteMode);
	SetGizmoSnapMode(SnapMode);

	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTexture) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTextureChannel))
	{
		SetAlphaTexture(AlphaTexture, AlphaTextureChannel);
	}


	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_QuadsPerSection) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_SectionsPerComponent) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_ComponentCount))
	{
		// Only clamp the landscape size if we're not in World Partition
		if (ParentMode && !ParentMode->IsGridBased())
		{
			NewLandscape_ClampSize();
		}
	}

	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_QuadsPerSection) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_SectionsPerComponent) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_ConvertMode))
	{
		UpdateComponentCount();
	}

	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NewLandscape_Material) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_HeightmapFilename) ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_Layers))
	{
		// In Import/Export tool we need to refresh from the existing material
		const bool bRefreshFromTarget = ParentMode && ParentMode->CurrentTool && ParentMode->CurrentTool->GetToolName() == FName(TEXT("ImportExport"));
		RefreshImportLayersList(bRefreshFromTarget);
	}

	if (PropertyChangedEvent.MemberProperty == nullptr ||
		PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintingRestriction))
	{
		UpdateComponentLayerAllowList();
	}

	if (PropertyChangedEvent.MemberProperty == nullptr 
		|| (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TargetDisplayOrder))
		|| (PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bTargetDisplayOrdersAscending)))
	{
		UpdateTargetLayerDisplayOrder();
	}
}

namespace UE::Landscape::Editor::Private
{
	template <typename EnumType>
	EnumType GetConfigEnum(const TCHAR* InSection, const TCHAR* InKey, const EnumType& InDefaultValue, const TOptional<EnumType>& InInvalidValue = TOptional<EnumType>())
	{
		EnumType Result = InDefaultValue;
		int32 ParsedValue = static_cast<int32>(Result);
		if (GConfig->GetInt(InSection, InKey, ParsedValue, GEditorPerProjectIni))
		{
			UEnum* Enum = StaticEnum<EnumType>();
			check(Enum != nullptr);
			if (Enum->IsValidEnumValue(ParsedValue)
				&& (!InInvalidValue.IsSet() || (static_cast<int32>(*InInvalidValue) != ParsedValue)))
			{
				Result = static_cast<EnumType>(ParsedValue);
			}
			else
			{
				UE_LOGF(LogLandscapeTools, Error, "Invalid enum value read from config file for enum %ls : %i, defaulting to : %ls", 
					*Enum->GetName(), ParsedValue, *Enum->GetDisplayNameTextByValue(static_cast<int64>(InDefaultValue)).ToString());
			}
		}

		return Result;
	}
} // namespace UE::Landscape::Editor::Private

/** Load UI settings from ini file */
void ULandscapeEditorObject::Load()
{
	using namespace UE::Landscape::Editor::Private;

	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ToolStrength).ToString(), ToolStrength, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintToolStrength).ToString(), PaintToolStrength, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WeightTargetValue).ToString(), WeightTargetValue, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseWeightTargetValue).ToString(), bUseWeightTargetValue, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushRadius).ToString(), BrushRadius, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintBrushRadius).ToString(), PaintBrushRadius, GEditorPerProjectIni);
	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushComponentSize).ToString(), BrushComponentSize, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushFalloff).ToString(), BrushFalloff, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintBrushFalloff).ToString(), PaintBrushFalloff, GEditorPerProjectIni);
	GConfig->GetString(TEXT("LandscapeEdit"), TEXT("TargetLayerAssetFilePath"), TargetLayerAssetFilePath.DirectoryPath.Path, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), TEXT("bUseTargetLayerAssetFilePath"), TargetLayerAssetFilePath.bUseAssetDirectoryPath, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseClayBrush).ToString(), bUseClayBrush, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushScale).ToString(), AlphaBrushScale, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bAlphaBrushAutoRotate).ToString(), bAlphaBrushAutoRotate, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushRotation).ToString(), AlphaBrushRotation, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanU).ToString(), AlphaBrushPanU, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanV).ToString(), AlphaBrushPanV, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseWorldSpacePatternBrush).ToString(), bUseWorldSpacePatternBrush, GEditorPerProjectIni);
	GConfig->GetVector2D(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.Origin"), WorldSpacePatternBrushSettings.Origin, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.bCenterTextureOnOrigin"), WorldSpacePatternBrushSettings.bCenterTextureOnOrigin, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.RepeatSize"), WorldSpacePatternBrushSettings.RepeatSize, GEditorPerProjectIni);
	AlphaTextureChannel = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTextureChannel).ToString(), /*InDefaultValue = */ELandscapeTextureColorChannel::Red);
	FString AlphaTextureName = (AlphaTexture != nullptr) ? AlphaTexture->GetPathName() : FString();
	GConfig->GetString(TEXT("LandscapeEdit"), TEXT("AlphaTextureName"), AlphaTextureName, GEditorPerProjectIni);
	UTexture2D* LoadedTexture = LoadObject<UTexture2D>(nullptr, *AlphaTextureName, nullptr, LOAD_NoWarn);
	if ((LoadedTexture == nullptr) && !AlphaTextureName.IsEmpty())
	{
		UE_LOGF(LogLandscapeTools, Error, "Cannot load alpha texture (%ls)", *AlphaTextureName);
	}
	SetAlphaTexture(LoadedTexture, AlphaTextureChannel);

	FlattenMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, FlattenMode).ToString(), /*InDefaultValue = */ELandscapeToolFlattenMode::Both, /*InInvalidValue = */{ ELandscapeToolFlattenMode::Invalid });
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseSlopeFlatten).ToString(), bUseSlopeFlatten, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bPickValuePerApply).ToString(), bPickValuePerApply, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseFlattenTarget).ToString(), bUseFlattenTarget, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, FlattenTarget).ToString(), FlattenTarget, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bShowFlattenTargetPreview).ToString(), bShowFlattenTargetPreview, GEditorPerProjectIni);

	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TerraceSmooth).ToString(), TerraceSmooth, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TerraceInterval).ToString(), TerraceInterval, GEditorPerProjectIni);

	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RampWidth).ToString(), RampWidth, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RampSideFalloff).ToString(), RampSideFalloff, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bCombinedLayersOperation).ToString(), bCombinedLayersOperation, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyWithoutMovingSculpt).ToString(), bApplyWithoutMovingSculpt, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyWithoutMovingPaint).ToString(), bApplyWithoutMovingPaint, GEditorPerProjectIni);

	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeThresh).ToString(), ErodeThresh, GEditorPerProjectIni);
	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeIterationNum).ToString(), ErodeIterationNum, GEditorPerProjectIni);
	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeSurfaceThickness).ToString(), ErodeSurfaceThickness, GEditorPerProjectIni);
	ErosionNoiseMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErosionNoiseMode).ToString(), /*InDefaultValue = */ELandscapeToolErosionMode::Both, /*InInvalidValue = */{ ELandscapeToolErosionMode::Invalid });
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErosionNoiseScale).ToString(), ErosionNoiseScale, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bErosionUseLayerHardness).ToString(), bErosionUseLayerHardness, GEditorPerProjectIni);

	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainAmount).ToString(), RainAmount, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SedimentCapacity).ToString(), SedimentCapacity, GEditorPerProjectIni);
	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, HErodeIterationNum).ToString(), HErodeIterationNum, GEditorPerProjectIni);
	RainDistMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainDistMode).ToString(), /*InDefaultValue = */ELandscapeToolHydroErosionMode::Both, /*InInvalidValue = */{ ELandscapeToolHydroErosionMode::Invalid });
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainDistScale).ToString(), RainDistScale, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, HErosionDetailScale).ToString(), HErosionDetailScale, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bHErosionDetailSmooth).ToString(), bHErosionDetailSmooth, GEditorPerProjectIni);

	NoiseMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NoiseMode).ToString(), /*InDefaultValue = */ELandscapeToolNoiseMode::Both, /*InInvalidValue = */{ ELandscapeToolNoiseMode::Invalid });
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NoiseScale).ToString(), NoiseScale, GEditorPerProjectIni);

	GConfig->GetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SmoothFilterKernelSize).ToString(), SmoothFilterKernelSize, GEditorPerProjectIni);
	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, DetailScale).ToString(), DetailScale, GEditorPerProjectIni);
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bDetailSmooth).ToString(), bDetailSmooth, GEditorPerProjectIni);

	GConfig->GetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MaximumValueRadius).ToString(), MaximumValueRadius, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bSmoothGizmoBrush).ToString(), bSmoothGizmoBrush, GEditorPerProjectIni);

	SetPasteMode(GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PasteMode).ToString(), /*InDefaultValue = */ELandscapeToolPasteMode::Both, /*InInvalidValue = */{ ELandscapeToolPasteMode::Invalid }));

	MirrorOp = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MirrorOp).ToString(), /*InDefaultValue = */ELandscapeMirrorOperation::MinusXToPlusX);

	ResizeLandscape_ConvertMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_ConvertMode).ToString(), /*InDefaultValue = */ELandscapeConvertMode::Expand, /*InInvalidValue = */{ ELandscapeConvertMode::Invalid });

	// Region
	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyToAllTargets).ToString(), bApplyToAllTargets, GEditorPerProjectIni);

	GConfig->GetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ShowUnusedLayers).ToString(), ShowUnusedLayers, GEditorPerProjectIni);

	// Set EditRenderMode
	SetbUseSelectedRegion(bUseSelectedRegion);
	SetbUseNegativeMask(bUseNegativeMask);

	FString NewLandscapeMaterialName;

	// If NewLandscape_Material is not null, we will try to use it
	if (!NewLandscape_Material.IsExplicitlyNull())
	{
		NewLandscapeMaterialName = NewLandscape_Material->GetPathName();
	}
	else
	{
		// If this project already has a saved NewLandscapeMaterialName, we use it
		GConfig->GetString(TEXT("LandscapeEdit"), TEXT("NewLandscapeMaterialName"), NewLandscapeMaterialName, GEditorPerProjectIni);

		if (NewLandscapeMaterialName.IsEmpty())
		{
			/* Project does not have a saved NewLandscapeMaterialNameand and NewLandscape_Material is not already assigned;
			 * we fallback to the DefaultLandscapeMaterial for the project, if set */
			const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
			TSoftObjectPtr<UMaterialInterface> DefaultMaterial = Settings->GetDefaultLandscapeMaterial();

			if (!DefaultMaterial.IsNull())
			{
				NewLandscapeMaterialName = DefaultMaterial.ToString();
			}
		}
	}
	
	if (!NewLandscapeMaterialName.IsEmpty())
	{
		NewLandscape_Material = LoadObject<UMaterialInterface>(nullptr, *NewLandscapeMaterialName, nullptr, LOAD_NoWarn);
	}
	
	ImportType = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportType).ToString(), /*InDefaultValue = */ELandscapeImportTransformType::None);
	ImportExportMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportExportMode).ToString(), /*InDefaultValue = */ELandscapeImportExportMode::LoadedOnly);
	SetGizmoSnapMode(GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SnapMode).ToString(), /*InDefaultValue = */ELandscapeGizmoSnapType::None));
	ImportLandscape_AlphamapType = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_AlphamapType).ToString(), /*InDefaultValue = */ELandscapeImportAlphamapType::Additive);

	WeightBlendedTargetLayerPaintMode = GetConfigEnum(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WeightBlendedTargetLayerPaintMode).ToString(), /*InDefaultValue = */ELandscapeWeightBlendPaintMode::ExclusiveRequiresNoCtrl);

	RefreshImportLayersList();
}

/** Save UI settings to ini file */
void ULandscapeEditorObject::Save()
{
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ToolStrength).ToString(), ToolStrength, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintToolStrength).ToString(), PaintToolStrength, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WeightTargetValue).ToString(), WeightTargetValue, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseWeightTargetValue).ToString(), bUseWeightTargetValue, GEditorPerProjectIni);

	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushRadius).ToString(), BrushRadius, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintBrushRadius).ToString(), PaintBrushRadius, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushComponentSize).ToString(), BrushComponentSize, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, BrushFalloff).ToString(), BrushFalloff, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PaintBrushFalloff).ToString(), PaintBrushFalloff, GEditorPerProjectIni);
	GConfig->SetString(TEXT("LandscapeEdit"), TEXT("TargetLayerAssetFilePath"), *TargetLayerAssetFilePath.DirectoryPath.Path, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), TEXT("bUseTargetLayerAssetFilePath"), TargetLayerAssetFilePath.bUseAssetDirectoryPath, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseClayBrush).ToString(), bUseClayBrush, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushScale).ToString(), AlphaBrushScale, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bAlphaBrushAutoRotate).ToString(), bAlphaBrushAutoRotate, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushRotation).ToString(), AlphaBrushRotation, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanU).ToString(), AlphaBrushPanU, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaBrushPanV).ToString(), AlphaBrushPanV, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseWorldSpacePatternBrush).ToString(), bUseWorldSpacePatternBrush, GEditorPerProjectIni);
	GConfig->SetVector2D(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.Origin"), WorldSpacePatternBrushSettings.Origin, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.bCenterTextureOnOrigin"), WorldSpacePatternBrushSettings.bCenterTextureOnOrigin, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), TEXT("WorldSpacePatternBrushSettings.RepeatSize"), WorldSpacePatternBrushSettings.RepeatSize, GEditorPerProjectIni);
	const FString AlphaTextureName = (AlphaTexture != nullptr) ? AlphaTexture->GetPathName() : FString();
	GConfig->SetString(TEXT("LandscapeEdit"), TEXT("AlphaTextureName"), *AlphaTextureName, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, AlphaTextureChannel).ToString(), (int32)AlphaTextureChannel, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, FlattenMode).ToString(), (int32)FlattenMode, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseSlopeFlatten).ToString(), bUseSlopeFlatten, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bPickValuePerApply).ToString(), bPickValuePerApply, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bUseFlattenTarget).ToString(), bUseFlattenTarget, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, FlattenTarget).ToString(), FlattenTarget, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bShowFlattenTargetPreview).ToString(), bShowFlattenTargetPreview, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TerraceSmooth).ToString(), TerraceSmooth, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, TerraceInterval).ToString(), TerraceInterval, GEditorPerProjectIni);

	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RampWidth).ToString(), RampWidth, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RampSideFalloff).ToString(), RampSideFalloff, GEditorPerProjectIni);

	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bCombinedLayersOperation).ToString(), bCombinedLayersOperation, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyWithoutMovingSculpt).ToString(), bApplyWithoutMovingSculpt, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyWithoutMovingPaint).ToString(), bApplyWithoutMovingPaint, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeThresh).ToString(), ErodeThresh, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeIterationNum).ToString(), ErodeIterationNum, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErodeSurfaceThickness).ToString(), ErodeSurfaceThickness, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErosionNoiseMode).ToString(), (int32)ErosionNoiseMode, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ErosionNoiseScale).ToString(), ErosionNoiseScale, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bErosionUseLayerHardness).ToString(), bErosionUseLayerHardness, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainAmount).ToString(), RainAmount, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SedimentCapacity).ToString(), SedimentCapacity, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, HErodeIterationNum).ToString(), HErodeIterationNum, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainDistMode).ToString(), (int32)RainDistMode, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, RainDistScale).ToString(), RainDistScale, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, HErosionDetailScale).ToString(), HErosionDetailScale, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bHErosionDetailSmooth).ToString(), bHErosionDetailSmooth, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NoiseMode).ToString(), (int32)NoiseMode, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, NoiseScale).ToString(), NoiseScale, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SmoothFilterKernelSize).ToString(), SmoothFilterKernelSize, GEditorPerProjectIni);
	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, DetailScale).ToString(), DetailScale, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bDetailSmooth).ToString(), bDetailSmooth, GEditorPerProjectIni);

	GConfig->SetFloat(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MaximumValueRadius).ToString(), MaximumValueRadius, GEditorPerProjectIni);

	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bSmoothGizmoBrush).ToString(), bSmoothGizmoBrush, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, PasteMode).ToString(), (int32)PasteMode, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, MirrorOp).ToString(), (int32)MirrorOp, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ResizeLandscape_ConvertMode).ToString(), (int32)ResizeLandscape_ConvertMode, GEditorPerProjectIni);
	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, bApplyToAllTargets).ToString(), bApplyToAllTargets, GEditorPerProjectIni);

	const FString NewLandscapeMaterialName = (NewLandscape_Material != nullptr) ? NewLandscape_Material->GetPathName() : FString();
	GConfig->SetString(TEXT("LandscapeEdit"), TEXT("NewLandscapeMaterialName"), *NewLandscapeMaterialName, GEditorPerProjectIni);

	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportType).ToString(), (int8)ImportType, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportExportMode).ToString(), (int8)ImportExportMode, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, SnapMode).ToString(), (int8)SnapMode, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ImportLandscape_AlphamapType).ToString(), (uint8)ImportLandscape_AlphamapType, GEditorPerProjectIni);

	GConfig->SetBool(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, ShowUnusedLayers).ToString(), ShowUnusedLayers, GEditorPerProjectIni);
	GConfig->SetInt(TEXT("LandscapeEdit"), *GET_MEMBER_NAME_CHECKED(ULandscapeEditorObject, WeightBlendedTargetLayerPaintMode).ToString(), (int32)WeightBlendedTargetLayerPaintMode, GEditorPerProjectIni);
}

// Region
void ULandscapeEditorObject::SetbUseSelectedRegion(bool InbUseSelectedRegion)
{ 
	bUseSelectedRegion = InbUseSelectedRegion;
	if (bUseSelectedRegion)
	{
		GLandscapeEditRenderMode |= ELandscapeEditRenderMode::Mask;
	}
	else
	{
		GLandscapeEditRenderMode &= ~(ELandscapeEditRenderMode::Mask);
	}
}
void ULandscapeEditorObject::SetbUseNegativeMask(bool InbUseNegativeMask) 
{ 
	bUseNegativeMask = InbUseNegativeMask; 
	if (bUseNegativeMask)
	{
		GLandscapeEditRenderMode |= ELandscapeEditRenderMode::InvertedMask;
	}
	else
	{
		GLandscapeEditRenderMode &= ~(ELandscapeEditRenderMode::InvertedMask);
	}
}

void ULandscapeEditorObject::SetPasteMode(ELandscapeToolPasteMode InPasteMode)
{
	PasteMode = InPasteMode;
}

void ULandscapeEditorObject::SetGizmoSnapMode(ELandscapeGizmoSnapType InSnapMode)
{
	SnapMode = InSnapMode;

	if (CurrentGizmoActor.IsValid())
	{
		CurrentGizmoActor->SnapType = SnapMode;
	}

	if (SnapMode != ELandscapeGizmoSnapType::None)
	{
		// Ignore gizmo updates if landscape info is invalid (Ex. landscape is deleted after using Copy Tool)
		if (CurrentGizmoActor.IsValid() && CurrentGizmoActor->TargetLandscapeInfo.IsValid())
		{
			const FVector WidgetLocation = CurrentGizmoActor->GetActorLocation();
			const FRotator WidgetRotation = CurrentGizmoActor->GetActorRotation();

			const FVector SnappedWidgetLocation = CurrentGizmoActor->SnapToLandscapeGrid(WidgetLocation);
			const FRotator SnappedWidgetRotation = CurrentGizmoActor->SnapToLandscapeGrid(WidgetRotation);

			CurrentGizmoActor->SetActorLocation(SnappedWidgetLocation, false);
			CurrentGizmoActor->SetActorRotation(SnappedWidgetRotation);
		}
	}
}

bool ULandscapeEditorObject::IsWeightmapTarget() const
{
	check(ParentMode);
	return !(ParentMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Heightmap || ParentMode->CurrentToolTarget.TargetType == ELandscapeToolTargetType::Visibility);
}

bool ULandscapeEditorObject::LoadAlphaTextureSourceData(UTexture2D* InTexture, TArray<uint8>& OutSourceData, int32& OutSourceDataSizeX, int32& OutSourceDataSizeY, ELandscapeTextureColorChannel& InOutTextureChannel)
{
	check(InTexture != nullptr);

	if (InTexture && InTexture->Source.IsValid() 
		&& !CVarLandscapeSimulateAlphaBrushTextureLoadFailure.GetValueOnGameThread()) // For debug purposes, we can also simulate a loading failure for the alpha brush texture
	{
		FImage SourceImage;
		if (InTexture->Source.GetMipImage(SourceImage, 0) && (SourceImage.Format != ERawImageFormat::Invalid))
		{
			OutSourceDataSizeX = SourceImage.SizeX;
			OutSourceDataSizeY = SourceImage.SizeY;
			const int32 NumPixels = OutSourceDataSizeX * OutSourceDataSizeY;

			// Handle the case where we're being asked to sample from a channel that is non-existent in the source image :
			EPixelFormat PixelFormat = InTexture->GetPixelFormat();
			EPixelFormatChannelFlags ValidTextureChannels = GetPixelFormatValidChannels(PixelFormat);
			if (((InOutTextureChannel == ELandscapeTextureColorChannel::Green && !EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::G)))
				|| ((InOutTextureChannel == ELandscapeTextureColorChannel::Blue && !EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::B)))
				|| ((InOutTextureChannel == ELandscapeTextureColorChannel::Alpha && !EnumHasAnyFlags(ValidTextureChannels, EPixelFormatChannelFlags::A))))
			{
				// Fallback to Red
				InOutTextureChannel = ELandscapeTextureColorChannel::Red;
			}

			// Convert/expand the image to BGRA8 : 
			SourceImage.ChangeFormat(ERawImageFormat::BGRA8, EGammaSpace::Linear);

			const int32 FinalDataSizeBytes = NumPixels;
			check(SourceImage.RawData.Num() == FinalDataSizeBytes * 4);
			OutSourceData.SetNum(FinalDataSizeBytes);

			uint8* SrcPtr = SourceImage.RawData.GetData();
			// Properly offset the source data as we're reading a single channel from a BGRA8 source :
			switch (InOutTextureChannel)
			{
			case ELandscapeTextureColorChannel::Blue:
				SrcPtr += 0;
				break;
			case ELandscapeTextureColorChannel::Green:
				SrcPtr += 1;
				break;
			case ELandscapeTextureColorChannel::Red:
				SrcPtr += 2;
				break;
			case ELandscapeTextureColorChannel::Alpha:
				SrcPtr += 3;
				break;
			default:
				check(false);
				break;
			}

			for (int32 Index = 0; Index < NumPixels; Index++, SrcPtr += 4)
			{
				OutSourceData[Index] = *SrcPtr;
			}

			return true;
		}
	}

	return false;
}

void ULandscapeEditorObject::SetAlphaTexture(UTexture2D* InTexture, ELandscapeTextureColorChannel InTextureChannel)
{
	AlphaTexture = nullptr;
	AlphaTextureChannel = InTextureChannel;
	AlphaTextureSizeX = 0;
	AlphaTextureSizeY = 0;

	UTexture2D* DefaultAlphaTexture = GetClass()->GetDefaultObject<ULandscapeEditorObject>()->AlphaTexture;
	// Try to read the texture data from the specified texture if any : 
	if (InTexture != nullptr)
	{
		if (LoadAlphaTextureSourceData(InTexture, AlphaTextureData, AlphaTextureSizeX, AlphaTextureSizeY, AlphaTextureChannel))
		{
			AlphaTexture = InTexture;
		}
		else
		{
			UE_LOGF(LogLandscapeTools, Error, "Invalid source data detected for texture (%ls), the default AlphaTexture (%ls) will be used.", *InTexture->GetPathName(), DefaultAlphaTexture ? *DefaultAlphaTexture->GetPathName() : TEXT("None"));
		}
	}

	if (AlphaTexture == nullptr)
	{
		if (DefaultAlphaTexture == nullptr)
		{
			UE_LOGF(LogLandscapeTools, Error, "No default AlphaTexture specified : the alpha brush won't work as expected.");
		}
		else if (LoadAlphaTextureSourceData(DefaultAlphaTexture, AlphaTextureData, AlphaTextureSizeX, AlphaTextureSizeY, AlphaTextureChannel))
		{
			AlphaTexture = DefaultAlphaTexture;
		}
		else
		{
			UE_LOGF(LogLandscapeTools, Error, "Invalid source data detected for default AlphaTexture (%ls)", *DefaultAlphaTexture->GetPathName());
		}
	}

	// If the AlphaTexture was successfully loaded, all read data should be valid :
	check ((AlphaTexture == nullptr) || HasValidAlphaTextureData());
}

bool ULandscapeEditorObject::HasValidAlphaTextureData() const
{
	return ((AlphaTextureSizeX > 0) && (AlphaTextureSizeY > 0) && !AlphaTextureData.IsEmpty());
}

void ULandscapeEditorObject::ChooseBestComponentSizeForImport()
{
	FLandscapeImportHelper::ChooseBestComponentSizeForImport(ImportLandscape_Width, ImportLandscape_Height, NewLandscape_QuadsPerSection, NewLandscape_SectionsPerComponent, NewLandscape_ComponentCount);
}

bool ULandscapeEditorObject::UseSingleFileImport() const
{
	if (ParentMode)
	{
		return ParentMode->UseSingleFileImport();
	}

	return true;
}

void ULandscapeEditorObject::RefreshImports()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULandscapeEditorObject::RefreshImports);

	ClearImportLandscapeData();
	HeightmapImportDescriptorIndex = 0;
	HeightmapImportDescriptor.Reset();
	ImportLandscape_Width = 0;
	ImportLandscape_Height = 0;

	ImportLandscape_HeightmapImportResult = ELandscapeImportResult::Success;
	ImportLandscape_HeightmapErrorMessage = FText();

	if (!ImportLandscape_HeightmapFilename.IsEmpty())
	{
		FLandscapeTiledImage TiledImage;
		
		FLandscapeFileInfo FileInfo = TiledImage.Load<uint16>(*ImportLandscape_HeightmapFilename);

		if (FileInfo.PossibleResolutions.Num() > 0)
		{
			ImportLandscape_Width = TiledImage.GetResolution().X;
			ImportLandscape_Height = TiledImage.GetResolution().Y;
		}

		if ( FileInfo.ResultCode != ELandscapeImportResult::Error)
		{
			ChooseBestComponentSizeForImport();
			ImportLandscapeData();
		}

		ImportLandscape_HeightmapImportResult = FileInfo.ResultCode;
		ImportLandscape_HeightmapErrorMessage = FileInfo.ErrorMessage;
	}

	RefreshLayerImports();
}

void ULandscapeEditorObject::RefreshLayerImports()
{
	// Make sure to reset import width and height if we don't have a Heightmap to import
	if (ImportLandscape_HeightmapFilename.IsEmpty())
	{
		HeightmapImportDescriptorIndex = 0;
		ImportLandscape_Width = 0;
		ImportLandscape_Height = 0;
	}

	for (FLandscapeImportLayer& UIImportLayer : ImportLandscape_Layers)
	{
		RefreshLayerImport(UIImportLayer);
	}
}

void ULandscapeEditorObject::RefreshLayerImport(FLandscapeImportLayer& ImportLayer)
{
	ImportLayer.ErrorMessage = FText();
	ImportLayer.ImportResult = ELandscapeImportResult::Success;

	if (ImportLayer.LayerName == ALandscapeProxy::VisibilityLayer->GetLayerName())
	{
		ImportLayer.LayerInfo = ALandscapeProxy::VisibilityLayer;
	}

	if (!ImportLayer.SourceFilePath.IsEmpty())
	{
		if (!ImportLayer.LayerInfo)
		{
			ImportLayer.ImportResult = ELandscapeImportResult::Error;
			ImportLayer.ErrorMessage = NSLOCTEXT("LandscapeEditor.NewLandscape", "Import_LayerInfoNotSet", "Can't import a layer file without a layer info");
		}
		else
		{
			FLandscapeTiledImage TiledImage;
			FLandscapeFileInfo FileInfo = TiledImage.Load<uint8>(*ImportLayer.SourceFilePath);
			ImportLayer.ImportResult = FileInfo.ResultCode;
			ImportLayer.ErrorMessage = FileInfo.ErrorMessage;
			if (FileInfo.ResultCode == ELandscapeImportResult::Success)
			{
				if (FileInfo.PossibleResolutions[0] != FLandscapeFileResolution(ImportLandscape_Width, ImportLandscape_Height) && bHeightmapSelected)
				{
					ImportLayer.ImportResult = ELandscapeImportResult::Error;
					ImportLayer.ErrorMessage = NSLOCTEXT("LandscapeEditor.ImportLandscape", "Import_WeightHeightResolutionMismatch", "Weightmap import resolution isn't same as Heightmap resolution.");
				}
			}
		}
	}
}

void ULandscapeEditorObject::OnChangeImportLandscapeResolution(int32 DescriptorIndex)
{
	check(DescriptorIndex >= 0 && DescriptorIndex < HeightmapImportDescriptor.ImportResolutions.Num());
	HeightmapImportDescriptorIndex = DescriptorIndex;
	ImportLandscape_Width = HeightmapImportDescriptor.ImportResolutions[HeightmapImportDescriptorIndex].Width;
	ImportLandscape_Height = HeightmapImportDescriptor.ImportResolutions[HeightmapImportDescriptorIndex].Height;
	ClearImportLandscapeData();
	ImportLandscapeData();
	ChooseBestComponentSizeForImport();
}

void ULandscapeEditorObject::ImportLandscapeData()
{
	FLandscapeTiledImage TiledImage;
	FLandscapeFileInfo FileInfo = TiledImage.Load<uint16>(*ImportLandscape_HeightmapFilename);

	if (FileInfo.ResultCode == ELandscapeImportResult::Error)
	{
		ImportLandscape_Data.Empty();
	}

	TiledImage.Read(ImportLandscape_Data, bFlipYAxis);
}

ELandscapeImportResult ULandscapeEditorObject::CreateImportLayersInfo(TArray<FLandscapeImportLayerInfo>& OutImportLayerInfos)
{
	const uint32 ImportSizeX = ImportLandscape_Width;
	const uint32 ImportSizeY = ImportLandscape_Height;

	if (ImportLandscape_HeightmapImportResult == ELandscapeImportResult::Error)
	{
		// Cancel import
		return ELandscapeImportResult::Error;
	}

	OutImportLayerInfos.Reserve(NewLandscape_Layers.Num());

	// Fill in LayerInfos array and allocate data
	for (FLandscapeImportLayer& UIImportLayer : NewLandscape_Layers)
	{
		OutImportLayerInfos.Add((const FLandscapeImportLayer&)UIImportLayer); //slicing is fine here
		FLandscapeImportLayerInfo& ImportLayer = OutImportLayerInfos.Last();

		if (ImportLayer.LayerInfo != nullptr && !ImportLayer.SourceFilePath.IsEmpty())
		{
			FLandscapeTiledImage LayerImage;
			FLandscapeFileInfo LayerFileInfo = LayerImage.Load<uint8>(*ImportLayer.SourceFilePath);

			UIImportLayer.ImportResult = LayerFileInfo.ResultCode;

			if (UIImportLayer.ImportResult == ELandscapeImportResult::Error)
			{
				FMessageDialog::Open(EAppMsgType::Ok, UIImportLayer.ErrorMessage);
				return ELandscapeImportResult::Error;
			}

			LayerImage.Read(ImportLayer.LayerData);
		}
	}

	return ELandscapeImportResult::Success;
}

ELandscapeImportResult ULandscapeEditorObject::CreateNewLayersInfo(TArray<FLandscapeImportLayerInfo>& OutNewLayerInfos)
{
	const int32 QuadsPerComponent = NewLandscape_SectionsPerComponent * NewLandscape_QuadsPerSection;
	const int32 SizeX = NewLandscape_ComponentCount.X * QuadsPerComponent + 1;
	const int32 SizeY = NewLandscape_ComponentCount.Y * QuadsPerComponent + 1;

	OutNewLayerInfos.Reset(NewLandscape_Layers.Num());

	// Fill in LayerInfos array and allocate data
	for (const FLandscapeImportLayer& UIImportLayer : NewLandscape_Layers)
	{
		FLandscapeImportLayerInfo ImportLayer = FLandscapeImportLayerInfo(UIImportLayer.LayerName);
		ImportLayer.LayerInfo = UIImportLayer.LayerInfo;
		ImportLayer.SourceFilePath = "";
		ImportLayer.LayerData = TArray<uint8>();
		OutNewLayerInfos.Add(MoveTemp(ImportLayer));
	}

	return ELandscapeImportResult::Success;
}

void ULandscapeEditorObject::InitializeDefaultHeightData(TArray<uint16>& OutData)
{
	const int32 QuadsPerComponent = NewLandscape_SectionsPerComponent * NewLandscape_QuadsPerSection;
	const int32 SizeX = NewLandscape_ComponentCount.X * QuadsPerComponent + 1;
	const int32 SizeY = NewLandscape_ComponentCount.Y * QuadsPerComponent + 1;
	const int32 TotalSize = SizeX * SizeY;
	// Initialize heightmap data
	OutData.Reset();
	OutData.AddUninitialized(TotalSize);
	
	TArray<uint16> StrideData;
	StrideData.AddUninitialized(SizeX);
	// Initialize blank heightmap data
	for (int32 X = 0; X < SizeX; ++X)
	{
		StrideData[X] = static_cast<uint16>(LandscapeDataAccess::MidValue);
	}
	for (int32 Y = 0; Y < SizeY; ++Y)
	{
		FMemory::Memcpy(&OutData[Y * SizeX], StrideData.GetData(), sizeof(uint16) * SizeX);
	}
}

void ULandscapeEditorObject::ExpandImportData(TArray<uint16>& OutHeightData, TArray<FLandscapeImportLayerInfo>& OutImportLayerInfos)
{
	const TArray<uint16>& ImportData = GetImportLandscapeData();
	if (ImportData.Num())
	{
		const int32 QuadsPerComponent = NewLandscape_SectionsPerComponent * NewLandscape_QuadsPerSection;
		FLandscapeImportResolution RequiredResolution(NewLandscape_ComponentCount.X * QuadsPerComponent + 1, NewLandscape_ComponentCount.Y * QuadsPerComponent + 1);
		FLandscapeImportResolution ImportResolution(ImportLandscape_Width, ImportLandscape_Height);

		FLandscapeImportHelper::TransformHeightmapImportData(ImportData, OutHeightData, ImportResolution, RequiredResolution, ELandscapeImportTransformType::ExpandCentered);

		for (int32 LayerIdx = 0; LayerIdx < OutImportLayerInfos.Num(); ++LayerIdx)
		{
			TArray<uint8>& OutImportLayerData = OutImportLayerInfos[LayerIdx].LayerData;
			TArray<uint8> OutLayerData;
			if (OutImportLayerData.Num())
			{
				FLandscapeImportHelper::TransformWeightmapImportData(OutImportLayerData, OutLayerData, ImportResolution, RequiredResolution, ELandscapeImportTransformType::ExpandCentered);
				OutImportLayerData = MoveTemp(OutLayerData);
			}
		}
	}
}

void ULandscapeEditorObject::RefreshImportLayersList(bool bRefreshFromTarget, bool bRegenerateThumbnails)
{
	TArray<FName> LayerNames;
	TArray<ULandscapeLayerInfoObject*> LayerInfoObjs;
	UMaterialInterface* Material = nullptr;
	if (bRefreshFromTarget)
	{
		LayerNames.Reset(ImportLandscape_Layers.Num());
		LayerInfoObjs.Reset(ImportLandscape_Layers.Num());
		Material = ParentMode->GetTargetLandscapeMaterial();
		for (const TSharedRef<FLandscapeTargetListInfo>& TargetListInfo : ParentMode->GetTargetList())
		{
			if ((TargetListInfo->TargetType != ELandscapeToolTargetType::Weightmap) && (TargetListInfo->TargetType != ELandscapeToolTargetType::Visibility))
			{
				continue;
			}

			LayerNames.Add(TargetListInfo->LayerName);
			LayerInfoObjs.Add(TargetListInfo->LayerInfoObj.Get());
		}
	}
	else
	{
		Material = NewLandscape_Material.Get();
		LayerNames = UE::Landscape::RetrieveTargetLayerNamesFromMaterial(Material);
	}

	TArray<FLandscapeImportLayer>& LandscapeLayers = bRefreshFromTarget ? ImportLandscape_Layers : NewLandscape_Layers;
	const TArray<FLandscapeImportLayer> OldLayersList = MoveTemp(LandscapeLayers);
	LandscapeLayers.Reset(LayerNames.Num());

	// Don't recreate the render state of everything, only update the materials context
	FMaterialUpdateContext MaterialUpdateContext(FMaterialUpdateContext::EOptions::Default & ~FMaterialUpdateContext::EOptions::RecreateRenderStates);
	for (int32 i = 0; i < LayerNames.Num(); i++)
	{
		const FName& LayerName = LayerNames[i];

		if (!LayerName.IsNone())
		{
			bool bFound = false;
			FLandscapeImportLayer NewImportLayer;
			NewImportLayer.ImportResult = ELandscapeImportResult::Success;
			NewImportLayer.ErrorMessage = FText();

			for (int32 j = 0; j < OldLayersList.Num(); j++)
			{
				if (OldLayersList[j].LayerName == LayerName)
				{
					NewImportLayer = OldLayersList[j];
					bFound = true;
					break;
				}
			}

			if (bFound && !bRegenerateThumbnails)
			{
				if ((NewImportLayer.ThumbnailMIC != nullptr) && (NewImportLayer.ThumbnailMIC->Parent != Material))
				{
					NewImportLayer.ThumbnailMIC->SetParentEditorOnly(Material);
					MaterialUpdateContext.AddMaterialInterface(NewImportLayer.ThumbnailMIC);
				}
			}
			else
			{
				NewImportLayer.LayerName = LayerName;
				NewImportLayer.ThumbnailMIC = UE::Landscape::CreateLandscapeLayerThumbnailMIC(MaterialUpdateContext, Material, LayerName);
			}

			if (bRefreshFromTarget)
			{
				NewImportLayer.LayerInfo = LayerInfoObjs[i];
			}

			RefreshLayerImport(NewImportLayer);

			LandscapeLayers.Add(MoveTemp(NewImportLayer));
		}
	}
}

void ULandscapeEditorObject::AutoFillTargetLayers(const FString& DefaultTargetLayerAssetPath, const bool bCreateAssetForEmptyLayers)
{
	for (FLandscapeImportLayer& NewLayer : NewLandscape_Layers)
	{
		if (const TOptional<FAssetData> AssetData = LandscapeEditorUtils::FindLandscapeTargetLayerInfoAsset(NewLayer.LayerName, DefaultTargetLayerAssetPath); 
			AssetData.IsSet())
		{
			NewLayer.LayerInfo = CastChecked<ULandscapeLayerInfoObject>(AssetData->GetAsset());
		}
		else if (bCreateAssetForEmptyLayers)
		{
			// Creates a new layer info object, using the default if available, or a new empty one
			ULandscapeLayerInfoObject* LayerInfo = UE::Landscape::CreateTargetLayerInfo(NewLayer.LayerName, DefaultTargetLayerAssetPath);

			NewLayer.LayerInfo = LayerInfo;

			// Show in the content browser
			TArray<UObject*> Objects;
			Objects.Add(LayerInfo);
			GEditor->SyncBrowserToObjects(Objects);
		}
	}
}

void ULandscapeEditorObject::UpdateComponentLayerAllowList()
{
	if (ParentMode && ParentMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		ParentMode->CurrentToolTarget.LandscapeInfo->UpdateComponentLayerAllowList();
	}
}

void ULandscapeEditorObject::UpdateTargetLayerDisplayOrder()
{
	if (ParentMode != nullptr)
	{
		ParentMode->UpdateTargetLayerDisplayOrder(TargetDisplayOrder, bTargetDisplayOrdersAscending);
	}
}

float ULandscapeEditorObject::GetCurrentToolStrength() const
{
	if (IsWeightmapTarget())
	{
		return PaintToolStrength;
	}
	return ToolStrength;
}

void ULandscapeEditorObject::SetCurrentToolStrength(float NewToolStrength)
{
	if (IsWeightmapTarget())
	{
		PaintToolStrength = NewToolStrength;		
	}
	else
	{
		ToolStrength = NewToolStrength;
	}
}

float ULandscapeEditorObject::GetCurrentToolBrushRadius() const
{
	if (IsWeightmapTarget())
	{
		return PaintBrushRadius;
	}
	return BrushRadius;
	
	
}

void ULandscapeEditorObject::SetCurrentToolBrushRadius(float NewBrushStrength)
{
	if (IsWeightmapTarget())
	{
		PaintBrushRadius = NewBrushStrength;
	}
	else
	{
		BrushRadius = NewBrushStrength;
	}
}

float ULandscapeEditorObject::GetCurrentToolBrushFalloff() const
{
	if (IsWeightmapTarget())
	{
		return PaintBrushFalloff;
		
	}
	return BrushFalloff;
	
}

void ULandscapeEditorObject::SetCurrentToolBrushFalloff(float NewBrushFalloff)
{
	if (IsWeightmapTarget())
	{
		PaintBrushFalloff = NewBrushFalloff;
	}
	else
	{
		BrushFalloff = NewBrushFalloff;
	}
}

float ULandscapeEditorObject::GetFlattenTarget(bool bInReturnPreviewValueIfActive) const
{
	return (bFlattenEyeDropperModeActivated && bInReturnPreviewValueIfActive) ? FlattenEyeDropperModeDesiredTarget : FlattenTarget;
}
