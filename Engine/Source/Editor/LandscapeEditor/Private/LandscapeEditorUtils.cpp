// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeEditorUtils.h"
#include "LandscapeSettings.h"

#include "LandscapeEditorModule.h"
#include "LandscapeEdit.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorObject.h"
#include "LandscapeTiledImage.h"
#include "LandscapeStreamingProxy.h"

#include "DesktopPlatformModule.h"
#include "EditorModeManager.h"
#include "EditorModes.h"

#include "Algo/Transform.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Algo/LevenshteinDistance.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LandscapeEditLayerCustomization.h"

namespace LandscapeEditorUtils
{
	int32 GetMaxSizeInComponents()
	{
		const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();
		return Settings->MaxComponents;
	}


	TOptional<FString> GetImportExportFilename(const FString& InDialogTitle, const FString& InStartPath, const FString& InDialogTypeString, bool bInImporting)
	{
		FString Filename;
		IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

		if (DesktopPlatform == nullptr)
		{
			return TOptional<FString>();
		}

		TArray<FString> Filenames;
		ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
		FEdModeLandscape* LandscapeEdMode = static_cast<FEdModeLandscape*>(GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape));

		bool bSuccess;
		
		if (bInImporting)
		{
			bSuccess = DesktopPlatform->OpenFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				InDialogTitle,
				InStartPath,
				TEXT(""),
				InDialogTypeString,
				EFileDialogFlags::None,
				Filenames);
		}
		else
		{
			bSuccess = DesktopPlatform->SaveFileDialog(
				FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
				InDialogTitle,
				InStartPath,
				TEXT(""),
				InDialogTypeString,
				EFileDialogFlags::None,
				Filenames);
		}

		if (bSuccess)
		{
			FString TiledFileNamePattern;

			if (bInImporting && FLandscapeTiledImage::CheckTiledNamePath(Filenames[0], TiledFileNamePattern) && FMessageDialog::Open(EAppMsgType::YesNo, FText::FromString(FString::Format(TEXT("Use '{0}' Tiled Image?"), { TiledFileNamePattern }))) == EAppReturnType::Yes)
			{
				Filename = TiledFileNamePattern;
			}
			else
			{
				Filename = Filenames[0];
			}

		}

		return TOptional<FString>(Filename);

	}

	void SaveLandscapeProxies(UWorld* World, TArrayView<ALandscapeProxy*> Proxies)
	{
		// Save the proxies
		TRACE_CPUPROFILER_EVENT_SCOPE(SaveCreatedActors);
		UWorldPartition::FDisableNonDirtyActorTrackingScope Scope(World->GetWorldPartition(), true);
		LandscapeEditorUtils::SaveObjects(Proxies);
	}

	TArray<FAssetData> GetLandscapeTargetLayerInfoAssets()
	{
		TArray<FAssetData> LayerInfoAssets;

		const UClass* AssetClass = ULandscapeLayerInfoObject::StaticClass();
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		FARFilter Filter;
		const FName PackageName = *AssetClass->GetPackage()->GetName();
		const FName AssetName = AssetClass->GetFName();

		Filter.ClassPaths.Add(FTopLevelAssetPath(PackageName, AssetName));
		AssetRegistryModule.Get().GetAssets(Filter, LayerInfoAssets);

		return LayerInfoAssets;
	}

	TOptional<FAssetData> FindLandscapeTargetLayerInfoAsset(const FName& LayerName, const FString& TargetLayerAssetPackagePath)
	{
		TArray<FAssetData> LayerInfoAssets = GetLandscapeTargetLayerInfoAssets();

		float BestScore = -1.0f;
		TOptional<FAssetData> BestLayerInfoObj;

		for (const FAssetData& LayerInfoAsset : LayerInfoAssets)
		{
			const ULandscapeLayerInfoObject* LayerInfo = CastChecked<ULandscapeLayerInfoObject>(LayerInfoAsset.GetAsset());
			FString CurrentPackagePath = LayerInfoAsset.PackagePath.ToString() + "/";

			// Only include assets in the package or its sub folders
			if (LayerInfo && LayerInfo->GetLayerName() == LayerName && CurrentPackagePath.Contains(TargetLayerAssetPackagePath))
			{
				float WorstCase = static_cast<float>(TargetLayerAssetPackagePath.Len() + CurrentPackagePath.Len());
				WorstCase = FMath::Max(WorstCase, 1.f);

				const float Score = 1.0f - (static_cast<float>(Algo::LevenshteinDistance(TargetLayerAssetPackagePath, CurrentPackagePath)) / WorstCase);
				if (Score > BestScore)
				{
					BestScore = Score;
					BestLayerInfoObj.Emplace(LayerInfoAsset);
				}
			}
		}

		return BestLayerInfoObj;
	}

	void BuildContextMenuFromCategoryEntryMap(const FEditLayerCategoryToEntryMap& InMap, FMenuBuilder& OutMenuBuilder)
	{
		for (const TPair<FName, FEditLayerMenuBlock>& CategoryPair : InMap)
		{
			const FName& CategoryName = CategoryPair.Key;
			const TArray<FMenuEntryParams>& MenuEntries = CategoryPair.Value.Entries;

			if (MenuEntries.IsEmpty())
			{
				continue;
			}

			check(!CategoryPair.Value.SectionLabel.IsEmpty()); 

			OutMenuBuilder.BeginSection(CategoryName, CategoryPair.Value.SectionLabel);
			for (const FMenuEntryParams& Entry : MenuEntries)
			{
				OutMenuBuilder.AddMenuEntry(Entry);
			}
			OutMenuBuilder.EndSection();
		}
	}
}


