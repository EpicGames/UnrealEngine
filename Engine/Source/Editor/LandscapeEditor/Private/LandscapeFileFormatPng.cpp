// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeFileFormatPng.h"

#include "Algo/Transform.h"
#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "ImageCore.h"
#include "ImageUtils.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"


#define LOCTEXT_NAMESPACE "LandscapeEditor.NewLandscape"


FLandscapeHeightmapFileFormat_Png::FLandscapeHeightmapFileFormat_Png()
{
	// @todo : can support all image extensions here
	//	since changing the loader to use FImageUtils
	//	this is no longer png specific, can load all image formats here
	//	 (eg. EXR and DDS might be useful)
	//	ideally FLandscapeHeightmapFileFormat_Png should be renamed too
	//	 (note that only PNG, EXR and DDS support 16-bit write ; TIFF can read 16 but not write)
	FileTypeInfo.Description = LOCTEXT("FileFormatPng_HeightmapDesc", "Heightmap .png files");
	FileTypeInfo.Extensions.Add(".png");
	FileTypeInfo.bSupportsExport = true;
}

FLandscapeFileInfo FLandscapeHeightmapFileFormat_Png::Validate(const TCHAR* HeightmapFilename, FName LayerName) const
{
	FLandscapeFileInfo Result;

	FImage Image;
	if ( !FImageUtils::LoadImage(HeightmapFilename,Image) )
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileReadError", "Error reading heightmap file");
		return Result;
	}

	// SizeXY are S32
	if ( Image.SizeX <= 0 || Image.SizeY <= 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileCorruptPng", "The heightmap file cannot be read (corrupt png?)");
		return Result;
	}

	FLandscapeFileResolution ImportResolution;
	ImportResolution.Width  = Image.SizeX;
	ImportResolution.Height = Image.SizeY;
	Result.PossibleResolutions.Add(ImportResolution);
	
	if ( ERawImageFormat::NumChannels(Image.Format) != 1 )
	{
		// @todo : could run UE::TextureUtilitiesCommon::AutoDetectAndChangeGrayScale here
		//	no need to complain if it's an RGB but actually contains gray colors

		Result.ResultCode = ELandscapeImportResult::Warning;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileColorPng", "The imported layer is not Grayscale. Results in-Editor will not be consistent with the source file.");
	}

	if ( Result.ResultCode != ELandscapeImportResult::Warning && IsU8Channels(Image.Format) )
	{
		Result.ResultCode = ELandscapeImportResult::Warning;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileLowBitDepth", "The heightmap file appears to be an 8-bit png, 16-bit is preferred. The import *can* continue, but the result may be lower quality than desired.");
	}
	

	// possible todo: support sCAL (XY scale) and pCAL (Z scale) png chunks for filling out Result.DataScale
	// I don't know if any heightmap generation software uses these or not
	// if we support their import we should make the exporter write them too

	// @todo : Efficiency : "Image" is just discarded
	//	 could get the image info without actually reading the whole thing?
	//	 or don't discard if we will wind up loading later?
	// the calling code does :
	//const FLandscapeFileInfo FileInfo = FileFormat->Validate(InImageFilename);
	//FLandscapeImportData<T> ImportData = FileFormat->Import(InImageFilename, ExpectedResolution);
	//	so the image is loaded twice

	return Result;
}

FLandscapeImportData<uint16> FLandscapeHeightmapFileFormat_Png::Import(const TCHAR* HeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const
{
	FLandscapeImportData<uint16> Result;
	
	FImage Image;
	if ( !FImageUtils::LoadImage(HeightmapFilename,Image) )
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileReadError", "Error reading heightmap file");
		return Result;
	}

	if (Image.SizeX != ExpectedResolution.Width || Image.SizeY != ExpectedResolution.Height)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapResolutionMismatch", "The heightmap file's resolution does not match the requested resolution");
		return Result;
	}

	// by default 8-bit source images come in as SRGB but we want to treat them as Linear here
	Image.GammaSpace = EGammaSpace::Linear;

	FImageView G16Image;
	G16Image.SizeX = Image.SizeX;
	G16Image.SizeY = Image.SizeY;
	G16Image.NumSlices = 1;
	G16Image.Format = ERawImageFormat::G16;
	G16Image.GammaSpace = EGammaSpace::Linear;
	
	Result.Data.SetNumUninitialized( G16Image.GetNumPixels() );
	G16Image.RawData = Result.Data.GetData();

	// blit into Result.Data with conversion to G16 :
	FImageCore::CopyImage(Image,G16Image);

	return Result;
}

void FLandscapeHeightmapFileFormat_Png::Export(const TCHAR* HeightmapFilename, FName LayerName, TArrayView<const uint16> Data, FLandscapeFileResolution DataResolution, FVector Scale) const
{
	FImageView G16Image;
	G16Image.SizeX = DataResolution.Width;
	G16Image.SizeY = DataResolution.Height;
	G16Image.NumSlices = 1;
	G16Image.Format = ERawImageFormat::G16;
	G16Image.GammaSpace = EGammaSpace::Linear;
	G16Image.RawData = (void *)Data.GetData();
	check( Data.NumBytes() >= (SIZE_T)G16Image.GetImageSizeBytes() );

	FImageUtils::SaveImageByExtension(HeightmapFilename, G16Image);
	// note: no error return
	//	not all image formats support 16-bit, they will convert to 8-bit if necessary
}

//////////////////////////////////////////////////////////////////////////

FLandscapeWeightmapFileFormat_Png::FLandscapeWeightmapFileFormat_Png()
{
	// @todo : support more than .png here
	//	 Weightmps are 8 bit so all formats are fully supported
	FileTypeInfo.Description = LOCTEXT("FileFormatPng_WeightmapDesc", "Layer .png files");
	FileTypeInfo.Extensions.Add(".png");
	FileTypeInfo.bSupportsExport = true;
}

// very similar to FLandscapeHeightmapFileFormat_Png::Validate

FLandscapeFileInfo FLandscapeWeightmapFileFormat_Png::Validate(const TCHAR* WeightmapFilename, FName LayerName) const
{
	FLandscapeFileInfo Result;
	
	FImage Image;
	if ( !FImageUtils::LoadImage(WeightmapFilename,Image) )
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerFileReadError", "Error reading layer file");
		return Result;
	}

	// SizeXY are S32
	if ( Image.SizeX <= 0 || Image.SizeY <= 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerCorruptPng", "The layer file cannot be read (corrupt png?)");
		return Result;
	}

	if ( ERawImageFormat::NumChannels(Image.Format) != 1 )
	{
		// @todo : could run UE::TextureUtilitiesCommon::AutoDetectAndChangeGrayScale here
		//	no need to complain if it's an RGB but actually contains gray colors

		Result.ResultCode = ELandscapeImportResult::Warning;
		Result.ErrorMessage = LOCTEXT("Import_LayerColorPng", "The imported layer is not Grayscale. Results in-Editor will not be consistent with the source file.");
	}

	FLandscapeFileResolution ImportResolution;
	ImportResolution.Width = Image.SizeX;
	ImportResolution.Height = Image.SizeY;
	Result.PossibleResolutions.Add(ImportResolution);
	
	// @todo : Efficiency : "Image" is just discarded
	//	 could get the image info without actually reading the whole thing?
	//	 or don't discard if we will wind up loading later?

	return Result;
}

FLandscapeImportData<uint8> FLandscapeWeightmapFileFormat_Png::Import(const TCHAR* WeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const
{
	FLandscapeImportData<uint8> Result;
	
	FImage Image;
	if ( !FImageUtils::LoadImage(WeightmapFilename,Image) )
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerFileReadError", "Error reading layer file");
		return Result;
	}
	
	if (Image.SizeX != ExpectedResolution.Width || Image.SizeY != ExpectedResolution.Height)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerResolutionMismatch", "The layer file's resolution does not match the requested resolution");
		return Result;
	}
	
	// by default 8-bit source images come in as SRGB but we want to treat them as Linear here
	Image.GammaSpace = EGammaSpace::Linear;

	FImageView G8Image;
	G8Image.SizeX = Image.SizeX;
	G8Image.SizeY = Image.SizeY;
	G8Image.NumSlices = 1;
	G8Image.Format = ERawImageFormat::G8;
	G8Image.GammaSpace = EGammaSpace::Linear;

	Result.Data.SetNumUninitialized( G8Image.GetNumPixels() );
	G8Image.RawData = Result.Data.GetData();
	
	// blit into Result.Data with conversion to G8 :
	FImageCore::CopyImage(Image,G8Image);

	return Result;
}

void FLandscapeWeightmapFileFormat_Png::Export(const TCHAR* WeightmapFilename, FName LayerName, TArrayView<const uint8> Data, FLandscapeFileResolution DataResolution, FVector Scale) const
{
	FImageView G8Image;
	G8Image.SizeX = DataResolution.Width;
	G8Image.SizeY = DataResolution.Height;
	G8Image.NumSlices = 1;
	G8Image.Format = ERawImageFormat::G8;
	G8Image.GammaSpace = EGammaSpace::Linear;
	G8Image.RawData = (void *)Data.GetData();
	check( Data.NumBytes() >= (SIZE_T)G8Image.GetImageSizeBytes() );

	FImageUtils::SaveImageByExtension(WeightmapFilename, G8Image);
	// note: no error return
}

#undef LOCTEXT_NAMESPACE
