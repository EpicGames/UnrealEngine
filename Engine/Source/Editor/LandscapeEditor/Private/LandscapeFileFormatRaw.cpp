// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeFileFormatRaw.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformCrt.h"
#include "HAL/UnrealMemory.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector.h"
#include "Misc/FileHelper.h"
#include "Templates/UnrealTemplate.h"

#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor.NewLandscape"

namespace
{
	void WriteMetaData(const TCHAR* InMapFilename, FLandscapeFileResolution InDataResolution, int32 InBitsPerPixel)
	{
		FString RawMetadataFilename = FPaths::SetExtension(InMapFilename, ".json");
		FString JsonStr;
		
		TSharedRef<TJsonWriter<TCHAR>> JsonWriter = TJsonWriterFactory<TCHAR>::Create(&JsonStr);
		TSharedRef<FJsonObject> JsonObject = MakeShareable(new FJsonObject);;
		JsonObject->SetField(TEXT("width"), MakeShared<FJsonValueNumber>(InDataResolution.Width));
		JsonObject->SetField(TEXT("height"), MakeShared<FJsonValueNumber>(InDataResolution.Height));
		JsonObject->SetField(TEXT("bpp"), MakeShared<FJsonValueNumber>(InBitsPerPixel));

		if (FJsonSerializer::Serialize<TCHAR>(JsonObject, JsonWriter))
		{
			FFileHelper::SaveStringToFile(JsonStr, *RawMetadataFilename);
		}
	}
}

bool GetRawResolution(const TCHAR* InFilename, FLandscapeFileResolution& OutResolution, int32& OutBitsPerPixel)
{
	FString Extension = FPaths::GetExtension(InFilename);
	FString RawMetadataFilename = FPaths::SetExtension(InFilename, ".json");
	
	OutBitsPerPixel = 0;

	if (FPaths::FileExists(RawMetadataFilename))
	{
		FString JsonStr;
		FFileHelper::LoadFileToString(JsonStr, *RawMetadataFilename);
		TSharedRef<TJsonReader<TCHAR>> JsonReader = TJsonReaderFactory<TCHAR>::Create(JsonStr);
		TSharedPtr<FJsonObject> JsonObject;
		if (FJsonSerializer::Deserialize(JsonReader, JsonObject))
		{
			OutResolution.Width = JsonObject->GetIntegerField(TEXT("width"));
			OutResolution.Height = JsonObject->GetIntegerField(TEXT("height"));

			OutBitsPerPixel = JsonObject->GetIntegerField(TEXT("bpp"));
			
			return true;
		}
		
		return false;
	}

	const bool bIs16Bit = Extension == FString(TEXT("r16"));
	const bool bIs8Bit = Extension == FString(TEXT("r8"));

	if (!(bIs16Bit || bIs8Bit))
	{
		return false;
	}

	const int32 BytesPerPixel = bIs16Bit ? 2 : 1;
	int64 FileSize = IFileManager::Get().FileSize(InFilename);
	
	const int64 NumPixels = FileSize / BytesPerPixel;
	const uint32 Dimension = FMath::TruncToInt32(FMath::Sqrt(static_cast<double>(NumPixels)));
	if (Dimension * Dimension == NumPixels)
	{
		OutResolution.Width = Dimension;
		OutResolution.Height = Dimension;

		OutBitsPerPixel = BytesPerPixel * 8;

		return true;
	}

	return false;
}

FLandscapeHeightmapFileFormat_Raw::FLandscapeHeightmapFileFormat_Raw()
{
	FileTypeInfo.Description = LOCTEXT("FileFormatRaw_HeightmapDesc", "Heightmap .r16/.raw files");
	FileTypeInfo.Extensions.Add(".r16");
	FileTypeInfo.Extensions.Add(".raw");
	FileTypeInfo.bSupportsExport = true;
}

FLandscapeFileInfo FLandscapeHeightmapFileFormat_Raw::Validate(const TCHAR* HeightmapFilename, FName LayerName) const
{
	FLandscapeFileInfo Result;

	int64 ImportFileSize = IFileManager::Get().FileSize(HeightmapFilename);

	if (ImportFileSize < 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileReadError", "Error reading heightmap file");
	}
	else if (ImportFileSize == 0 || ImportFileSize % 2 != 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileInvalidSizeNot16bit", "The heightmap file has an invalid size (possibly not 16-bit?)");
	}
	else
	{
		FLandscapeFileResolution Resolution;
		int32 BitsPerPixel;
		if (GetRawResolution(HeightmapFilename, Resolution, BitsPerPixel))
		{
			if (BitsPerPixel != 16)
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_HeightmapFileBitsPerPixel", "Height file has an invalid number of bits per pixel");
			}
			else
			{
				Result.PossibleResolutions = { Resolution };
			}
		}
		else
		{
			if (BitsPerPixel == 0)
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_RawFileInvalidExtension", "The file bit depth unknown bit depth use .r16 (height) or .r8 (weight)");
			}
			else
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_HeightmapFileInvalidSize", "The heightmap file has an invalid size (possibly not 16-bit?)");
			}
			
		}
	}

	return Result;
}

FLandscapeImportData<uint16> FLandscapeHeightmapFileFormat_Raw::Import(const TCHAR* HeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const
{
	FLandscapeImportData<uint16> Result;

	TArray<uint8> TempData;
	if (!FFileHelper::LoadFileToArray(TempData, HeightmapFilename, FILEREAD_Silent))
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapFileReadError", "Error reading heightmap file");
	}
	else if (TempData.Num() != (ExpectedResolution.Width * ExpectedResolution.Height * 2))
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_HeightmapResolutionMismatch", "The heightmap file's resolution does not match the requested resolution");
	}
	else
	{
		Result.Data.Empty(ExpectedResolution.Width * ExpectedResolution.Height);
		Result.Data.AddUninitialized(ExpectedResolution.Width * ExpectedResolution.Height);
		FMemory::Memcpy(Result.Data.GetData(), TempData.GetData(), ExpectedResolution.Width * ExpectedResolution.Height * 2);
	}

	return Result;
}

void FLandscapeHeightmapFileFormat_Raw::Export(const TCHAR* HeightmapFilename, FName LayerName, TArrayView<const uint16> Data, FLandscapeFileResolution DataResolution, FVector Scale) const
{
	TArray<uint8> TempData;
	TempData.Empty(DataResolution.Width * DataResolution.Height * 2);
	TempData.AddUninitialized(DataResolution.Width * DataResolution.Height * 2);
	FMemory::Memcpy(TempData.GetData(), Data.GetData(), DataResolution.Width * DataResolution.Height * 2);

	if (FFileHelper::SaveArrayToFile(TempData, HeightmapFilename))
	{
		FString Extension = FPaths::GetExtension(HeightmapFilename);

		if (Extension == "raw")
		{
			WriteMetaData(HeightmapFilename, DataResolution, 16);
		}
	}
}

//////////////////////////////////////////////////////////////////////////

FLandscapeWeightmapFileFormat_Raw::FLandscapeWeightmapFileFormat_Raw()
{
	FileTypeInfo.Description = LOCTEXT("FileFormatRaw_WeightmapDesc", "Layer .r8/.raw files");
	FileTypeInfo.Extensions.Add(".r8");
	FileTypeInfo.Extensions.Add(".raw");
	FileTypeInfo.bSupportsExport = true;
}

FLandscapeFileInfo FLandscapeWeightmapFileFormat_Raw::Validate(const TCHAR* WeightmapFilename, FName LayerName) const
{
	FLandscapeFileInfo Result;

	int64 ImportFileSize = IFileManager::Get().FileSize(WeightmapFilename);

	if (ImportFileSize < 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerFileReadError", "Error reading layer file");
	}
	else
	{
		FLandscapeFileResolution Resolution;
		int32 BitsPerPixel;
		if (GetRawResolution(WeightmapFilename, Resolution, BitsPerPixel))
		{
			if (BitsPerPixel != 8)
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_WeightmapFileBitsPerPixel", "Weightmap file has an invalid number of bits per pixel");
				
			}
			else
			{
				Result.PossibleResolutions = { Resolution };
			}
		}
		else
		{
			if (BitsPerPixel == 0)
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_RawFileInvalidExtension", "The file bit depth unknown bit depth use .r16 (height) or .r8 (weight)");
			}
			else
			{
				Result.ResultCode = ELandscapeImportResult::Error;
				Result.ErrorMessage = LOCTEXT("Import_WeightmapFileInvalidSize", "The layer file has an invalid size");
			}			
		}
	}

	return Result;
}

FLandscapeImportData<uint8> FLandscapeWeightmapFileFormat_Raw::Import(const TCHAR* WeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const
{
	FLandscapeImportData<uint8> Result;

	TArray<uint8> TempData;
	if (!FFileHelper::LoadFileToArray(TempData, WeightmapFilename, FILEREAD_Silent))
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerFileReadError", "Error reading layer file");
	}
	else if (TempData.Num() != (ExpectedResolution.Width * ExpectedResolution.Height))
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("Import_LayerResolutionMismatch", "The layer file's resolution does not match the requested resolution");
	}
	else
	{
		Result.Data = MoveTemp(TempData);
	}

	return Result;
}

void FLandscapeWeightmapFileFormat_Raw::Export(const TCHAR* WeightmapFilename, FName LayerName, TArrayView<const uint8> Data, FLandscapeFileResolution DataResolution, FVector Scale) const
{
	if (FFileHelper::SaveArrayToFile(Data, WeightmapFilename))
	{
		FString Extension = FPaths::GetExtension(WeightmapFilename);

		if (Extension == "raw")
		{
			WriteMetaData(WeightmapFilename, DataResolution, 8);
		}
	}
}

#undef LOCTEXT_NAMESPACE
