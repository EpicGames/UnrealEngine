// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "LandscapeFileFormatInterface.h"
#include "Math/UnrealMathSSE.h"
#include "UObject/NameTypes.h"

// Implement .raw file format
class FLandscapeHeightmapFileFormat_Raw : public ILandscapeHeightmapFileFormat
{
private:
	FLandscapeFileTypeInfo FileTypeInfo;

public:
	FLandscapeHeightmapFileFormat_Raw();

	virtual const FLandscapeFileTypeInfo& GetInfo() const override
	{
		return FileTypeInfo;
	}

	virtual FLandscapeFileInfo Validate(const TCHAR* HeightmapFilename, FName LayerName) const override;
	virtual FLandscapeImportData<uint16> Import(const TCHAR* HeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const override;
	virtual void Export(const TCHAR* HeightmapFilename, FName LayerName, TArrayView<const uint16> Data, FLandscapeFileResolution DataResolution, FVector Scale) const override;
};

//////////////////////////////////////////////////////////////////////////

class FLandscapeWeightmapFileFormat_Raw : public ILandscapeWeightmapFileFormat
{
private:
	FLandscapeFileTypeInfo FileTypeInfo;

public:
	FLandscapeWeightmapFileFormat_Raw();

	virtual const FLandscapeFileTypeInfo& GetInfo() const override
	{
		return FileTypeInfo;
	}

	virtual FLandscapeFileInfo Validate(const TCHAR* WeightmapFilename, FName LayerName) const override;
	virtual FLandscapeImportData<uint8> Import(const TCHAR* WeightmapFilename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const override;
	virtual void Export(const TCHAR* WeightmapFilename, FName Layername, TArrayView<const uint8> Data, FLandscapeFileResolution DataResolution, FVector Scale) const override;
};
