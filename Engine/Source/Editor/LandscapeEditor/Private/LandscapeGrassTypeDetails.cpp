// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeGrassTypeDetails.h"

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "DetailLayoutBuilder.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Misc/AssertionMacros.h"
#include "PropertyHandle.h"
#include "PropertyRestriction.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "LandscapeGrassType.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "LandscapeProxy.h"
#include "MaterialCachedData.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

class UObject;

static bool GShowBothPerQualityAndPerPlaformProperties = false;
static FAutoConsoleVariableRef CVarShowBothPerQualityAndPerPlaform(
	TEXT("r.grass.ShowBothPerQualityAndPerPlaformProperties"),
	GShowBothPerQualityAndPerPlaformProperties,
	TEXT("Show both per platform and per quality properties in the editor."));

#define LOCTEXT_NAMESPACE "LandscapeGrassTypeDetails"

TSharedRef<IDetailCustomization> FLandscapeGrassTypeDetails::MakeInstance()
{
	return MakeShareable(new FLandscapeGrassTypeDetails);
}

FLandscapeGrassTypeDetails::~FLandscapeGrassTypeDetails()
{
}

void FLandscapeGrassTypeDetails::CustomizeDetails( IDetailLayoutBuilder& DetailLayout )
{
	if (GShowBothPerQualityAndPerPlaformProperties)
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailLayout.GetObjectsBeingCustomized(Objects);

	const TArray<FName> PerPlatformProperties =
	{
		GET_MEMBER_NAME_CHECKED(FGrassVariety, GrassDensity),
		GET_MEMBER_NAME_CHECKED(FGrassVariety, StartCullDistance),
		GET_MEMBER_NAME_CHECKED(FGrassVariety, EndCullDistance),
	};

	const TArray<FName> PerQualityProperties =
	{
		GET_MEMBER_NAME_CHECKED(FGrassVariety, GrassDensityQuality),
		GET_MEMBER_NAME_CHECKED(FGrassVariety, StartCullDistanceQuality),
		GET_MEMBER_NAME_CHECKED(FGrassVariety, EndCullDistanceQuality),
	};

	if (Objects.Num() > 0)
	{
		// get the grass variety array
		TSharedPtr<IPropertyHandle> GrassVariety = DetailLayout.GetProperty(GET_MEMBER_NAME_CHECKED(ULandscapeGrassType, GrassVarieties));
		TSharedPtr<IPropertyHandleArray> GrassVarietyArrayHandle = GrassVariety->AsArray();

		if (GrassVarietyArrayHandle.IsValid())
		{
			uint32 NumElements;
			GrassVarietyArrayHandle->GetNumElements(NumElements);

			for (uint32 Index = 0; Index < NumElements; ++Index)
			{
				TSharedRef<IPropertyHandle> ElementHandle = GrassVarietyArrayHandle->GetElement(Index);

				uint32 NumChildren;
				ElementHandle->GetNumChildren(NumChildren);

				for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
				{
					TSharedRef<IPropertyHandle> ChildHandle = ElementHandle->GetChildHandle(ChildIndex).ToSharedRef();
					const FName& ChildPropertyName = ChildHandle->GetProperty()->GetFName();

					if (PerPlatformProperties.Contains(ChildPropertyName) && GEngine && GEngine->UseGrassVarityPerQualityLevels)
					{
						DetailLayout.HideProperty(ChildHandle);
					}

					if (PerQualityProperties.Contains(ChildPropertyName) && GEngine && !GEngine->UseGrassVarityPerQualityLevels)
					{
						DetailLayout.HideProperty(ChildHandle);
					}
				}
			}
		}
	}
}


// ----------------------------------------------------------------------------------

TSharedRef<IPropertyTypeCustomization> FLandscapeGrassNameCustomization::MakeInstance()
{
	return MakeShareable(new FLandscapeGrassNameCustomization);
}

// This customization allows to show an "editable" combo box, that can be used as a key in a TMap property
void FLandscapeGrassNameCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Grab the property handle for the Name in order to retrieve its value
	TSharedPtr<IPropertyHandle> PropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FLandscapeGrassName, Name));
	check(PropertyHandle.IsValid());
	TSharedPtr<IPropertyHandle> ParentPropertyHandle = PropertyHandle->GetParentHandle();
	
	const bool bIsEditable = PropertyHandle->IsEditable();

	// Gather the grass names from all materials that might be used by the landscape proxies owning this property :
	TArray<UObject*> CustomizedObjects;
	StructPropertyHandle->GetOuterObjects(CustomizedObjects);

	TSet<UMaterialInterface*> LandscapeMaterials;
	for (UObject* CustomizedObject : CustomizedObjects)
	{
		if (ALandscapeProxy* Proxy = Cast<ALandscapeProxy>(CustomizedObject))
		{
			Proxy->RetrieveAllLandscapeMaterials(LandscapeMaterials);
		}
	}

	TSet<FName> AvailableGrassNames;
	for (UMaterialInterface* LandscapeMaterial : LandscapeMaterials)
	{
		for (TPair<FName, TObjectPtr<ULandscapeGrassType>> NamedGrassType : LandscapeMaterial->GetCachedExpressionData().NamedGrassTypes)
		{
			if (NamedGrassType.Key != NAME_None)
			{
				AvailableGrassNames.Add(NamedGrassType.Key);
			}
		}
	}

	ComboBoxOptions = AvailableGrassNames.Array();
	ComboBoxOptions.Sort(FNameLexicalLess());

	auto TextToName = [](const FText& InText) -> FName
		{
			FString String = InText.ToString();
			String.TrimStartAndEndInline();
			return FName(String);
		};

	auto SetPropertyValue = [ParentPropertyHandle, PropertyHandle](const FName& InName)
		{
			// Prefer setting the Name's value through the parent property's handle (FLandscapeGrassName) if possible, so that the TMap's key uniqueness enforcement works as expected
			if (ParentPropertyHandle.IsValid())
			{
				// Since we'll use a formatted string (Name="MyName") to address the property, we need to escape the quote (") character, so that it doesn't interfere with the string quotes 
				FString EscapedName = InName.ToString();
				EscapedName.ReplaceInline(TEXT("\""), TEXT("\\\""));
				// This is a bit of a hack, but to be able to call SetValueFromFormattedString (in order to set the property's inner Name), we need to address it using this property path : 
				ParentPropertyHandle->SetValueFromFormattedString(FString::Format(TEXT("({0}=\"{1}\")"), { GET_MEMBER_NAME_CHECKED(FLandscapeGrassName, Name).ToString(), EscapedName }));

			}
			else if (PropertyHandle.IsValid())
			{
				PropertyHandle->SetValueFromFormattedString(InName.ToString());
			}
		};

	TSharedPtr<SEditableTextBox> EditableTextBox = 
		SNew(SEditableTextBox)
		.Text_Lambda([PropertyHandle]()
			{
				FString CurrentValue;
				FPropertyAccess::Result CurrentValueResult = PropertyHandle->GetValueAsFormattedString(CurrentValue);
				if (CurrentValueResult == FPropertyAccess::MultipleValues)
				{
					return LOCTEXT("MultipleValues", "Multiple Values");
				}
				return FText::FromString(CurrentValue);
			})
		.OnTextCommitted_Lambda([SetPropertyValue, TextToName/*, IsGrassNameUnique*/](const FText& InNewText, ETextCommit::Type)
			{
				// This might seem weird to go from FText to FString to FName and then back to FString but this allows
				//  to transform an empty string into "None" (NAME_None) since, ultimately, this serves to set a FName
				const FName NewName = TextToName(InNewText);
				SetPropertyValue(NewName);
			});

	HeaderRow
		.ValueContent()
		[
			SNew(SComboBox<FName>)
			.OptionsSource(&ComboBoxOptions)
			.OnGenerateWidget_Lambda([](FName InName)
				{
					return SNew(STextBlock)
						.Text(FText::FromName(InName));
				})
			.OnSelectionChanged_Lambda([SetPropertyValue](FName InSelected, ESelectInfo::Type)
				{
					SetPropertyValue(InSelected);
				})
			.Content()
			[
				EditableTextBox.ToSharedRef()
			]
			// Disable this custom widget by hand if the property is disabled (since this is custom, it doesn't automatically support the CanEditChange method)
			.IsEnabled(bIsEditable)
		];
}

// No children to display here, all the customization lies in the header 
void FLandscapeGrassNameCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}
#undef LOCTEXT_NAMESPACE
