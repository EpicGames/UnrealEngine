// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FPropertyRestriction;
class IDetailLayoutBuilder;
class UObject;
class SToolTip;
struct FPropertyChangedEvent;


// ----------------------------------------------------------------------------------

class FLandscapeGrassTypeDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual ~FLandscapeGrassTypeDetails();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailLayout ) override;
};


// ----------------------------------------------------------------------------------

class FLandscapeGrassNameCustomization : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization overrides.
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

	/** Overridden to do nothing */
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	// ~End IPropertyTypeCustomization overrides.

private:
	TArray<FName> ComboBoxOptions;
};