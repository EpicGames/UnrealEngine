// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeImageFileCache.h"
#include "LandscapeSettings.h"

#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Internationalization.h"
#include "IDirectoryWatcher.h"


template <>
FLandscapeImageFileCache::CacheType& FLandscapeImageFileCache::ChooseCache<uint8>()
{
	return CachedImages[Cache8];
}

template <>
FLandscapeImageFileCache::CacheType& FLandscapeImageFileCache::ChooseCache<uint16>()
{
	return CachedImages[Cache16];
}


FLandscapeImageFileCache::FLandscapeImageFileCache()
{
	ULandscapeSettings* Settings = GetMutableDefault<ULandscapeSettings>();
	SettingsChangedHandle = Settings->OnSettingChanged().AddRaw(this, &FLandscapeImageFileCache::OnLandscapeSettingsChanged);
	MaxCacheSize = Settings->MaxImageImportCacheSizeMegaBytes * 1024U * 1024U;
}

FLandscapeImageFileCache::~FLandscapeImageFileCache()
{
	if (UObjectInitialized() && !GExitPurge)
	{
		GetMutableDefault<ULandscapeSettings>()->OnSettingChanged().Remove(SettingsChangedHandle);
	}
}

template<typename T>
FLandscapeFileInfo FLandscapeImageFileCache::FindImage(const TCHAR* InImageFilename, FLandscapeImageDataRef& OutImageData)
{
	FString FullyQualifiedFileName = FPaths::ConvertRelativePathToFull(FString(InImageFilename));

	FCacheEntry* CacheEntry = ChooseCache<T>().Find(FullyQualifiedFileName);
	FLandscapeFileInfo Result;

	if (CacheEntry)
	{
		CacheEntry->UsageCount++;
		OutImageData = CacheEntry->ImageData;
		Result.PossibleResolutions.Add(FLandscapeFileResolution(OutImageData.Resolution.X, OutImageData.Resolution.Y));
		Result.ResultCode = CacheEntry->ImageData.Result;
		Result.ErrorMessage = CacheEntry->ImageData.ErrorMessage;
		return Result;
	}

	FLandscapeImageDataRef NewImageData;
	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const ILandscapeFileFormat<T>* FileFormat = LandscapeEditorModule.GetFormatByExtension<T>(*FPaths::GetExtension(InImageFilename, true));

	if (!FileFormat)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		return Result;
	}

	const FLandscapeFileInfo FileInfo = FileFormat->Validate(InImageFilename);

	if (FileInfo.ResultCode != ELandscapeImportResult::Error && FileInfo.PossibleResolutions.Num() > 0)
	{
		FLandscapeFileResolution ExpectedResolution = FileInfo.PossibleResolutions[0];
		FLandscapeImportData<T> ImportData = FileFormat->Import(InImageFilename, ExpectedResolution);

		if (ImportData.ResultCode == ELandscapeImportResult::Error)
		{
			// Validate didn't error, but Import did.  Return the Import error.
			Result.ResultCode = ELandscapeImportResult::Error;
			Result.ErrorMessage = ImportData.ErrorMessage;
			return Result;
		}
		else if (ImportData.ResultCode == ELandscapeImportResult::Warning)
		{
			// New warning from Import.
			NewImageData.Result = ELandscapeImportResult::Warning;
			NewImageData.ErrorMessage = ImportData.ErrorMessage;
		}
		else
		{
			// No new warning or error, use the warning from the validate step, if there was one.
			NewImageData.Result = FileInfo.ResultCode;
			NewImageData.ErrorMessage = FileInfo.ErrorMessage;
		}

		const int32 BufferSize = ImportData.Data.Num() * sizeof(T);
		NewImageData.Data = TSharedPtr<TArray<uint8>>(new TArray<uint8>());
		NewImageData.Data->SetNumUninitialized(BufferSize);
		FMemory::Memcpy(NewImageData.Data->GetData(), ImportData.Data.GetData(), BufferSize);
		NewImageData.Resolution = FIntPoint(ExpectedResolution.Width, ExpectedResolution.Height);
		NewImageData.BytesPerPixel = BufferSize / (ExpectedResolution.Width * ExpectedResolution.Height);
	}
	else
	{
		return FileInfo;
	}

	Trim();
	OutImageData = NewImageData;
	Add<T>(FullyQualifiedFileName, OutImageData);

	Result.PossibleResolutions.Add(FLandscapeFileResolution(OutImageData.Resolution.X, OutImageData.Resolution.Y));
	Result.ResultCode = NewImageData.Result;
	Result.ErrorMessage = NewImageData.ErrorMessage;

	return Result;
}

// FindImage is a public symbol, so requires explicit instantiation.  Or if its inlined, then everything it references must be inlined
// or explicitly instantiated.
template FLandscapeFileInfo FLandscapeImageFileCache::FindImage<uint8>(const TCHAR* InImageFilename, FLandscapeImageDataRef& OutImageData);
template FLandscapeFileInfo FLandscapeImageFileCache::FindImage<uint16>(const TCHAR* InImageFilename, FLandscapeImageDataRef& OutImageData);


void FLandscapeImageFileCache::MonitorCallback(const TArray<struct FFileChangeData>& Changes)
{
	for (const FFileChangeData& Change : Changes)
	{
		if (Change.Action == FFileChangeData::FCA_Modified || Change.Action == FFileChangeData::FCA_Removed)
		{
			// The file cache expects path to be fully qualified
			FString FullyQualifiedFileName = FPaths::ConvertRelativePathToFull(Change.Filename);
			Remove(FullyQualifiedFileName);
		}
	}
}

bool FLandscapeImageFileCache::MonitorFile(const FString& InFullyQualifiedFileName)
{
	check(!FPaths::IsRelative(InFullyQualifiedFileName));

	FString Directory = FPaths::GetPath(InFullyQualifiedFileName);

	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");

	if (FDirectoryMonitor* Monitor = MonitoredDirs.Find(Directory))
	{
		Monitor->NumFiles++;
		return true;
	}
	else
	{
		FDelegateHandle Handle;
		bool bWatcherResult = DirectoryWatcherModule.Get()->RegisterDirectoryChangedCallback_Handle(Directory, IDirectoryWatcher::FDirectoryChanged::CreateRaw(this, &FLandscapeImageFileCache::MonitorCallback), Handle);
		if (bWatcherResult)
		{
			MonitoredDirs.Add(Directory, FDirectoryMonitor(Handle));
		}
		return bWatcherResult;
	}
}

void FLandscapeImageFileCache::UnmonitorFile(const FString& InFullyQualifiedFileName)
{
	check(!FPaths::IsRelative(InFullyQualifiedFileName));

	FString Directory = FPaths::GetPath(InFullyQualifiedFileName);

	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");

	if (FDirectoryMonitor* Monitor = MonitoredDirs.Find(Directory))
	{
		check(Monitor->NumFiles > 0);

		Monitor->NumFiles--;
		if (Monitor->NumFiles == 0)
		{
			DirectoryWatcherModule.Get()->UnregisterDirectoryChangedCallback_Handle(Directory, Monitor->MonitorHandle);
			MonitoredDirs.Remove(Directory);
		}
	}
}

template <typename T>
void FLandscapeImageFileCache::Add(const FString& InFullyQualifiedFileName, FLandscapeImageDataRef NewImageData)
{
	check(!FPaths::IsRelative(InFullyQualifiedFileName));

	CacheType& Cache = ChooseCache<T>();
	check(Cache.Find(InFullyQualifiedFileName) == nullptr);

	Cache.Add(InFullyQualifiedFileName, FCacheEntry(NewImageData));
	MonitorFile(InFullyQualifiedFileName);
	CacheSize += NewImageData.Data->Num();
}

void FLandscapeImageFileCache::Remove(const FString& InFullyQualifiedFileName)
{
	check(!FPaths::IsRelative(InFullyQualifiedFileName));

	for (CacheType& Cache : CachedImages)
	{
		if (FCacheEntry* CacheEntry = Cache.Find(InFullyQualifiedFileName))
		{
			CacheSize -= CacheEntry->ImageData.Data->Num();
			Cache.Remove(InFullyQualifiedFileName);
			UnmonitorFile(InFullyQualifiedFileName);
		}
	}
}

void FLandscapeImageFileCache::OnLandscapeSettingsChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent)
{
	if (InPropertyChangedEvent.GetPropertyName() == "MaxImageImportCacheSizeMegaBytes")
	{
		ULandscapeSettings* LandscapeSettings = Cast<ULandscapeSettings>(InObject);
		SetMaxSize(LandscapeSettings->MaxImageImportCacheSizeMegaBytes);
	}
}

void FLandscapeImageFileCache::SetMaxSize(uint64 InNewMaxSize)
{
	if (MaxCacheSize != InNewMaxSize)
	{
		MaxCacheSize = InNewMaxSize * 1024U *1024U;
		Trim();
	}
}

void FLandscapeImageFileCache::Clear()
{
	for (CacheType& Cache : CachedImages)
	{
		Cache.Empty();
	}
	FDirectoryWatcherModule& DirectoryWatcherModule = FModuleManager::LoadModuleChecked<FDirectoryWatcherModule>("DirectoryWatcher");
	
	for (const TPair<FString, FDirectoryMonitor>& MonitoredDir : MonitoredDirs)
	{
		DirectoryWatcherModule.Get()->UnregisterDirectoryChangedCallback_Handle(MonitoredDir.Key, MonitoredDir.Value.MonitorHandle);
	}
	MonitoredDirs.Empty();
}


void FLandscapeImageFileCache::Trim()
{
	if (CacheSize < MaxCacheSize)
	{
		return;
	}

	using RemovalPriorityData = TTuple<uint32, FString, uint32>;
	TArray<RemovalPriorityData> AllCacheEntries;

	// Make an array of all cache entries from both maps, sorted by UsageCount.
	for (CacheType& Cache : CachedImages)
	{
		for (const TPair<FString, FCacheEntry>& It : Cache)
		{
			AllCacheEntries.Push(RemovalPriorityData(It.Value.UsageCount, It.Key, It.Value.ImageData.Data->Num()));
		}
	}
	AllCacheEntries.Sort([](const RemovalPriorityData& LHS, const RemovalPriorityData& RHS) { return  LHS.Get<0>() < RHS.Get<0>(); });

	// Starting with the leased used, remove entries until we are under MaxCacheSize.
	TArray<FString> ToRemove;
	uint64 Size = CacheSize;
	for (const RemovalPriorityData& Entry : AllCacheEntries)
	{
		ToRemove.Add(Entry.Get<1>());
		uint64 ImageSize = Entry.Get<2>();
		Size -= ImageSize;
		if (Size <= MaxCacheSize)
		{
			break;
		}
	}

	// Note that the file is removed from both caches, regardless of which entry UsageCount got it on this list.  Other
	// callers of Remove must target all entries of the file name, but this one could go either way.  This rare case might
	// overshoot the MaxCacheSize target.
	for (const FString& Filename : ToRemove)
	{
		Remove(Filename);
	}
}
