// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Math/Vector.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Templates/SharedPointer.h"
#include "DirectoryWatcherModule.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeEditorModule.h"
#include "Containers/StaticArray.h"
#include "Delegates/IDelegateInstance.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"

struct FLandscapeImageDataRef
{
	TSharedPtr<TArray<uint8>> Data;
	FIntPoint Resolution;
	ELandscapeImportResult Result;
	FText ErrorMessage;
	int32 BytesPerPixel;
};

class FLandscapeImageFileCache
{
public:
	FLandscapeImageFileCache();
	~FLandscapeImageFileCache();

	template<typename T>
	FLandscapeFileInfo FindImage(const TCHAR* InImageFilename, FLandscapeImageDataRef& OutImageData);

	void SetMaxSize(uint64 InNewMaxSize);

	void Clear();

private:

	struct FCacheEntry
	{
		FCacheEntry(FLandscapeImageDataRef ImageData) : ImageData(ImageData) {}

		uint32 UsageCount = 1;
		FLandscapeImageDataRef ImageData;
	};

	struct FDirectoryMonitor
	{
		FDirectoryMonitor(FDelegateHandle Handle) { MonitorHandle = Handle; }
		int32 NumFiles = 1;
		FDelegateHandle MonitorHandle;
	};
	
	void OnLandscapeSettingsChanged(UObject* InObject, struct FPropertyChangedEvent& InPropertyChangedEvent);
	 
	bool MonitorFile(const FString& InFullyQualifiedFileName);
	void UnmonitorFile(const FString& InFullyQualifiedFileName);

	void MonitorCallback(const TArray<struct FFileChangeData>& Changes);

	using CacheType = TMap<FString, FCacheEntry>;

	template<typename T>
	CacheType& ChooseCache();

	// Two cache maps.  Hold 8 bit and 16 bit data separately.  Otherwise importing the same file as weightmap and heightmap
	// will use the cached 8bit version for the heightmap.
	enum
	{
		Cache8 = 0,
		Cache16 = 1
	};
	TStaticArray<CacheType, 2> CachedImages;
	
	TMap<FString, FDirectoryMonitor> MonitoredDirs;

	template <typename T>
	void Add(const FString& InFullyQualifiedFileName, FLandscapeImageDataRef NewImageData);
	void Remove(const FString& InFullyQualifiedFileName);

	void Trim();

	uint64 MaxCacheSize = 32 * 1024 * 1024;
	uint64 CacheSize = 0;
	FDelegateHandle SettingsChangedHandle;
};

#undef LOCTEXT_NAMESPACE 
