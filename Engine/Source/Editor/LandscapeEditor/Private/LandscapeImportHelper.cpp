// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeImportHelper.h"
#include "LandscapeEditorModule.h"
#include "LandscapeDataAccess.h"
#include "LandscapeConfigHelper.h"
#include "Modules/ModuleManager.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "LandscapeEditorUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeImportHelper)

#define LOCTEXT_NAMESPACE "LandscapeImportHelper"

bool FLandscapeImportHelper::ExtractCoordinates(const FString& BaseFilename, FIntPoint& OutCoord, FString& OutBaseFilePattern)
{
	//We expect file name in form: <tilename>_x<number>_y<number>
	int32 XPos = BaseFilename.Find(TEXT("_x"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	int32 YPos = BaseFilename.Find(TEXT("_y"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (XPos != INDEX_NONE && YPos != INDEX_NONE && XPos < YPos)
	{
		FString XCoord = BaseFilename.Mid(XPos + 2, YPos - (XPos + 2));
		FString YCoord = BaseFilename.Mid(YPos + 2, BaseFilename.Len() - (YPos + 2));

		if (XCoord.IsNumeric() && YCoord.IsNumeric())
		{
			OutBaseFilePattern = BaseFilename.Mid(0, XPos);
			TTypeFromString<int32>::FromString(OutCoord.X, *XCoord);
			TTypeFromString<int32>::FromString(OutCoord.Y, *YCoord);
			return true;
		}
	}

	return false;
}

void FLandscapeImportHelper::GetMatchingFiles(const FString& FilePathPattern, TArray<FString>& OutFileToImport)
{
	FString BaseFilePathPattern = FPaths::GetBaseFilename(FilePathPattern);
	IFileManager::Get().IterateDirectoryRecursively(*FPaths::GetPath(FilePathPattern), [&OutFileToImport, &FilePathPattern, &BaseFilePathPattern](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (!bIsDirectory)
		{
			FString Filename(FilenameOrDirectory);
			FString BaseFilename = FPaths::GetBaseFilename(Filename);
			bool bIsValidFile = false;
			// The file name must match either exactly (e.g. MyHeightmap.png) :
			if (BaseFilename == BaseFilePathPattern)
			{
				bIsValidFile = true;
			}
			// Or it must exactly match the pattern (e.g. MyHeightmap_x0_y0.png ok, MyHeightmap0.png not ok) :
			else
			{
				FIntPoint Coord;
				FString BaseFilePattern;
				bool bIsValidPatternFileName = FLandscapeImportHelper::ExtractCoordinates(BaseFilename, Coord, BaseFilePattern);
				bIsValidFile = bIsValidPatternFileName && (BaseFilePattern == BaseFilePathPattern);
			}

			if (bIsValidFile)
			{
				OutFileToImport.Add(Filename);
			}
		}
		return true;
	});
}

template<class T>
ELandscapeImportResult GetImportDataInternal(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, FName LayerName, T DefaultValue, TArray<T>& OutData, FText& OutMessage)
{
	if (DescriptorIndex < 0 || DescriptorIndex >= ImportDescriptor.ImportResolutions.Num())
	{
		OutMessage = LOCTEXT("Import_InvalidDescriptorIndex", "Invalid Descriptor Index");
		return ELandscapeImportResult::Error;
	}
		
	if (!ImportDescriptor.FileDescriptors.Num() || ImportDescriptor.ImportResolutions.Num() != ImportDescriptor.FileResolutions.Num())
	{
		OutMessage = LOCTEXT("Import_InvalidDescriptor", "Invalid Descriptor");
		return ELandscapeImportResult::Error;
	}
	
	int64 TotalWidth = ImportDescriptor.ImportResolutions[DescriptorIndex].Width;	// convert from uint32
	int64 TotalHeight = ImportDescriptor.ImportResolutions[DescriptorIndex].Height;

	if (TotalWidth <= 0 || TotalHeight <= 0)
	{
		OutMessage = LOCTEXT("Import_InvalidImportResolution", "Import Resolution is not valid");
		return ELandscapeImportResult::Error;
	}

	if (TotalWidth > MAX_int64 / TotalHeight)	// Total Pixels should fit in an int64
	{
		OutMessage = LOCTEXT("Import_ImageTooLarge", "Landscape image is too large");
		return ELandscapeImportResult::Error;
	}

	int64 TotalPixels = TotalWidth * TotalHeight;

	OutData.Reset();
	OutData.SetNumZeroed(TotalPixels);

	// Initialize All to default value so that non-covered regions have data
	TArray<T> StrideData;
	StrideData.SetNumUninitialized(TotalWidth);
	for (int64 X = 0; X < TotalWidth; ++X)
	{
		StrideData[X] = DefaultValue;
	}
	for (int64 Y = 0; Y < TotalHeight; ++Y)
	{
		FMemory::Memcpy(&OutData[Y * TotalWidth], StrideData.GetData(), sizeof(T) * TotalWidth);
	}

	ELandscapeImportResult Result = ELandscapeImportResult::Success;
	// Import Regions
	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const ILandscapeFileFormat<T>* FileFormat = LandscapeEditorModule.GetFormatByExtension<T>(*FPaths::GetExtension(ImportDescriptor.FileDescriptors[0].FilePath, true));
	check(FileFormat);

	int64 FileWidth = ImportDescriptor.FileResolutions[DescriptorIndex].Width;	// convert from uint32
	int64 FileHeight = ImportDescriptor.FileResolutions[DescriptorIndex].Height;
	
	for (const FLandscapeImportFileDescriptor& FileDescriptor : ImportDescriptor.FileDescriptors)
	{
		FLandscapeImportData<T> ImportData = FileFormat->Import(*FileDescriptor.FilePath, LayerName, ImportDescriptor.FileResolutions[DescriptorIndex]);
		OutMessage = ImportData.ErrorMessage;
		Result = ImportData.ResultCode;
		if (ImportData.ResultCode == ELandscapeImportResult::Error)
		{
			break;
		}
		
		// NOTE: this assumes the same file resolution for all descriptors..
		int64 StartX = FileDescriptor.Coord.X * FileWidth;
		int64 StartY = FileDescriptor.Coord.Y * FileHeight;
		
		for (int64 Y = 0; Y < FileHeight; ++Y)
		{		
			int64 DestY = StartY + Y;
			FMemory::Memcpy(&OutData[DestY * TotalWidth + StartX], &ImportData.Data[Y * FileWidth], FileWidth * sizeof(T));
		}
	}

	return Result;
}

template<class T>
ELandscapeImportResult GetImportDescriptorInternal(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FName LayerName, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage)
{
	OutImportDescriptor.Reset();
	if (FilePath.IsEmpty())
	{
		OutMessage = LOCTEXT("Import_InvalidPath", "Invalid file");
		return ELandscapeImportResult::Error;
	}
		
	FIntPoint OutCoord{};
	FIntPoint MinCoord(INT32_MAX, INT32_MAX); // All coords should be rebased to the min
	FIntPoint MaxCoord(INT32_MIN, INT32_MIN);
	FString OutFileImportPattern;
	FString FilePathPattern;
	TArray<FString> OutFilesToImport;

	// If we are handling multiple files handle the case where Filepath is in the _xN_yM.extention format or if it's a pattern already
	if (!bSingleFile)
	{
		if (FLandscapeImportHelper::ExtractCoordinates(FPaths::GetBaseFilename(FilePath), OutCoord, OutFileImportPattern))
		{
			FilePathPattern = FPaths::GetPath(FilePath) / OutFileImportPattern;
		}
		else
		{
			FilePathPattern = FPaths::GetBaseFilename(FilePath, false);
		}

		FLandscapeImportHelper::GetMatchingFiles(FilePathPattern, OutFilesToImport);
		if (OutFilesToImport.IsEmpty())
		{
			return ELandscapeImportResult::Error;
		}
		
		if (OutFilesToImport.Contains(FilePath))
		{
			// If one of the files found has a name that exactly matches yet we have on or more numbered files matching the pattern, then we have an ambiguity and warn the user about it :
			if (OutFilesToImport.Num() > 1)
			{
				OutMessage = FText::Format(LOCTEXT("Import_AmbiguousImportFile", 
					"Ambiguous import files found :\nThere's a single '{0}.{3}' file and {1} {1}|plural(one=file,other=files) whose {1}|plural(one=name,other=names) {1}|plural(one=matches,other=match) the proper input pattern (ex: '{2}_x0_y0.{3}').\nPlease rename the single file."),
					FText::FromString(FPaths::GetBaseFilename(FilePath)), OutFilesToImport.Num() - 1, FText::FromString(FPaths::GetBaseFilename(FilePathPattern)), FText::FromString(FPaths::GetExtension(FilePath)));
				return ELandscapeImportResult::Error;
			}
			else
			{
				// We have a single file found and it's exactly matching the expected file name, consider we're in single file mode :
				check(OutFilesToImport.Num() == 1);
				bSingleFile = true;
			}
		}
	}
	else
	{
		OutFilesToImport.Add(FilePath);
	}

	TArray<FLandscapeFileResolution> ImportResolutions;

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const ILandscapeFileFormat<T>* FileFormat = nullptr;
	for (const FString& ImportFilename : OutFilesToImport)
	{
		const bool bFirst = FileFormat == nullptr;
		const ILandscapeFileFormat<T>* CurrentFileFormat = LandscapeEditorModule.GetFormatByExtension<T>(*FPaths::GetExtension(ImportFilename, true));
		if (FileFormat != nullptr && FileFormat != CurrentFileFormat)
		{
			OutMessage = LOCTEXT("Import_MismatchFileType", "Not all files have the same file type");
			return ELandscapeImportResult::Error;
		}
		FileFormat = CurrentFileFormat;

		if (FileFormat)
		{
			FLandscapeFileInfo FileInfo = FileFormat->Validate(*ImportFilename);
			if (FileInfo.ResultCode == ELandscapeImportResult::Error)
			{
				OutMessage = FileInfo.ErrorMessage;
				return FileInfo.ResultCode;
			}

			FString OutLocalFileImportPattern;
			if (!bSingleFile && !FLandscapeImportHelper::ExtractCoordinates(FPaths::GetBaseFilename(ImportFilename), OutCoord, OutLocalFileImportPattern))
			{
				OutMessage = FText::Format(LOCTEXT("Import_InvalidFilename", "File '{0}' doesn't have the proper pattern(ex: '{1}_x0_y0.{2})'"),FText::FromString(FPaths::GetBaseFilename(ImportFilename)), FText::FromString(OutFileImportPattern), FText::FromString(FPaths::GetExtension(FilePath)));
				return ELandscapeImportResult::Error;
			}
			MinCoord.X = FMath::Min(OutCoord.X, MinCoord.X);
			MinCoord.Y = FMath::Min(OutCoord.Y, MinCoord.Y);
			MaxCoord.X = FMath::Max(OutCoord.X, MaxCoord.X);
			MaxCoord.Y = FMath::Max(OutCoord.Y, MaxCoord.Y);

			FLandscapeImportFileDescriptor FileDescriptor(ImportFilename, OutCoord);
			OutImportDescriptor.FileDescriptors.Add(FileDescriptor);

			if (bFirst)
			{
				// Resolutions will need to match for all files (keep the first one to compare)
				OutImportDescriptor.FileResolutions = MoveTemp(FileInfo.PossibleResolutions);
								
				if (FileInfo.DataScale.IsSet())
				{
					OutImportDescriptor.Scale = FileInfo.DataScale.GetValue();
					OutImportDescriptor.Scale.Z *= LANDSCAPE_INV_ZSCALE;
				}
			}
			else
			{
				if (OutImportDescriptor.FileResolutions != FileInfo.PossibleResolutions)
				{
					OutMessage = LOCTEXT("Import_MismatchResolution", "Not all files have the same resolution");
					return ELandscapeImportResult::Error;
				}
				else if (FileInfo.DataScale.IsSet())
				{
					FVector CurrentScale = FileInfo.DataScale.GetValue();
					CurrentScale.Z *= LANDSCAPE_INV_ZSCALE;

					if (!OutImportDescriptor.Scale.Equals(CurrentScale))
					{
						OutMessage = LOCTEXT("Import_MismatchScale", "Not all files have the same data scale");
						return ELandscapeImportResult::Error;
					}
				}
			}
		}
		else
		{
			OutMessage = LOCTEXT("Import_UnknownFileType", "File type not recognized");
			return ELandscapeImportResult::Error;
		}
	}

	check(OutImportDescriptor.FileDescriptors.Num());
	// Rebase with MinCoord
	for (FLandscapeImportFileDescriptor& FileDescriptor : OutImportDescriptor.FileDescriptors)
	{
		FileDescriptor.Coord -= MinCoord;
	}
	MaxCoord -= MinCoord;

	// Flip Coordinates on Y axis
	if (bFlipYAxis)
	{
		for (FLandscapeImportFileDescriptor& FileDescriptor : OutImportDescriptor.FileDescriptors)
		{
			FileDescriptor.Coord.Y = MaxCoord.Y - FileDescriptor.Coord.Y;
		}
	}

	// Compute Import Total Size
	for (const FLandscapeFileResolution& Resolution : OutImportDescriptor.FileResolutions)
	{
		OutImportDescriptor.ImportResolutions.Add(FLandscapeImportResolution((MaxCoord.X+1)*Resolution.Width, (MaxCoord.Y+1)*Resolution.Height));
	}

	return ELandscapeImportResult::Success;
}

// Copy the pixels (no resampling) from InSrcRegion of InData into the same size region in OutData, aligned to InDestOffset.  Outside this region will be reset to 0.
// InDestSize is the desired overall dimensions of the dest buffer.
// Note:  InSrcRegion is exlcusive/half-open, possibly unlike the other functions in this class.
// TODO [alan.roach]: move this to LandscapeConfigHelper.h, alongside the old CopySubregion.  Kept out of the public header for hotfix compatibility.
template<typename T>
static void TransformImportDataCopySubregion(TConstArrayView<T> InData, TArray<T>& OutData, FIntRect InSrcRegion, int32 InSrcDataPitch, FIntPoint InDestOffset, FIntPoint InDestSize)
{
	FIntRect DestTotalRegion(0, 0, InDestSize.X, InDestSize.Y);
	FIntRect DestRegion(InDestOffset, InDestOffset + InSrcRegion.Size());  // copy region in dest buffer
	DestRegion.Clip(DestTotalRegion);  // limit copy region to overall bounds of dest buffer
	FIntPoint SrcOffsetAdjustment = DestRegion.Min - InDestOffset + InSrcRegion.Min;  // after clipping, that might affect the source region
	int32 DestDataPitch = InDestSize.X;
	int32 DestDataSize = InDestSize.X * InDestSize.Y;

	// Set the output buffer size and initialize to 0.
	check(InDestSize.X > 0 && InDestSize.Y > 0);
	OutData.Empty(DestDataSize);
	OutData.AddZeroed(DestDataSize);

	if (DestRegion.Area() <= 0)
	{
		return;  // Nothing to copy
	}

	check(InSrcRegion.Min.X >= 0 && InSrcRegion.Min.Y >= 0);
	check(InSrcRegion.Max.X <= InSrcDataPitch);
	check(InData.Num() >= ((InSrcRegion.Max.Y - 1) * InSrcDataPitch) + InSrcRegion.Max.X);  // input buffer must be large enough to contain InSrcRegion
	check(DestDataSize >= ((DestRegion.Max.Y - 1) * DestDataPitch) + DestRegion.Max.X);


	const int32 XEnd = DestRegion.Width();
	const int32 YEnd = DestRegion.Height();

	for (int32 Y = 0; Y < YEnd; ++Y)
	{
		for (int32 X = 0; X < XEnd; ++X)
		{
			FIntPoint CopyCoords = FIntPoint(X, Y);
			FIntPoint SrcCoords = CopyCoords + SrcOffsetAdjustment;
			FIntPoint DestCoords = CopyCoords + DestRegion.Min;

			const int32 SrcIndex = SrcCoords.X + SrcCoords.Y * InSrcDataPitch;
			const int32 DstIndex = DestCoords.X + DestCoords.Y * DestDataPitch;

			OutData[DstIndex] = InData[SrcIndex];
		}
	}
}

template<class T>
void TransformImportDataInternal(const TArray<T>& InData, TArray<T>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType, FIntPoint Offset)
{
	// TODO [alan.roach]:  the (-1) applied to Width and Height here is fishy.
	check(InData.Num() == CurrentResolution.Width * CurrentResolution.Height);
	if (TransformType == ELandscapeImportTransformType::Resample)
	{
		const FIntRect SrcRegion(0, 0, CurrentResolution.Width - 1, CurrentResolution.Height - 1);
		const FIntRect DestRegion(0, 0, RequiredResolution.Width - 1, RequiredResolution.Height - 1);
		FLandscapeConfigHelper::ResampleData<T>(InData, OutData, SrcRegion, DestRegion);
	}
	else if(TransformType == ELandscapeImportTransformType::ExpandCentered || TransformType == ELandscapeImportTransformType::ExpandOffset)
	{
		int32 OffsetX = 0;
		int32 OffsetY = 0;
		if (TransformType == ELandscapeImportTransformType::ExpandCentered)
		{
			OffsetX = (int32)(RequiredResolution.Width - CurrentResolution.Width) / 2;
			OffsetY = (int32)(RequiredResolution.Height - CurrentResolution.Height) / 2;
		}
		else if (TransformType == ELandscapeImportTransformType::ExpandOffset)
		{
			OffsetX = Offset.X;
			OffsetY = Offset.Y;
		}
				
		const FIntRect SrcRegion(0, 0, CurrentResolution.Width - 1, CurrentResolution.Height - 1);
		const FIntRect DestRegion(-OffsetX, -OffsetY, RequiredResolution.Width - OffsetX - 1, RequiredResolution.Height - OffsetY - 1);
		FLandscapeConfigHelper::ExpandData<T>(InData, OutData, SrcRegion, DestRegion, OffsetX != 0 || OffsetY != 0);
	}
	else if (TransformType == ELandscapeImportTransformType::Subregion)
	{
		const FIntRect SrcRegion(0, 0, CurrentResolution.Width, CurrentResolution.Height);
		const FIntPoint DestSize(RequiredResolution.Width, RequiredResolution.Height);
		TransformImportDataCopySubregion<T>(InData, OutData, SrcRegion, CurrentResolution.Width, Offset, DestSize);
	}
	else
	{
		OutData = InData;
	}
}

ELandscapeImportResult FLandscapeImportHelper::GetHeightmapImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage)
{
	return GetImportDescriptorInternal<uint16>(FilePath, bSingleFile, bFlipYAxis, NAME_None, OutImportDescriptor, OutMessage);
}

ELandscapeImportResult FLandscapeImportHelper::GetWeightmapImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FName LayerName, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage)
{
	return GetImportDescriptorInternal<uint8>(FilePath, bSingleFile, bFlipYAxis, LayerName, OutImportDescriptor, OutMessage);
}

ELandscapeImportResult FLandscapeImportHelper::GetHeightmapImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, TArray<uint16>& OutData, FText& OutMessage)
{
	return GetImportDataInternal<uint16>(ImportDescriptor, DescriptorIndex, NAME_None, static_cast<uint16>(LandscapeDataAccess::MidValue), OutData, OutMessage);
}

ELandscapeImportResult FLandscapeImportHelper::GetWeightmapImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, FName LayerName, TArray<uint8>& OutData, FText& OutMessage)
{
	return GetImportDataInternal<uint8>(ImportDescriptor, DescriptorIndex, LayerName, 0, OutData, OutMessage);
}

void FLandscapeImportHelper::TransformWeightmapImportData(const TArray<uint8>& InData, TArray<uint8>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType, FIntPoint Offset)
{
	TransformImportDataInternal<uint8>(InData, OutData, CurrentResolution, RequiredResolution, TransformType, Offset);
}

void FLandscapeImportHelper::TransformHeightmapImportData(const TArray<uint16>& InData, TArray<uint16>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType, FIntPoint Offset)
{
	TransformImportDataInternal<uint16>(InData, OutData, CurrentResolution, RequiredResolution, TransformType, Offset);
}

void FLandscapeImportHelper::ChooseBestComponentSizeForImport(int32 Width, int32 Height, int32& InOutQuadsPerSection, int32& InOutSectionsPerComponent, FIntPoint& OutComponentCount)
{
	bool bValidSubsectionSizeParam = false;
	bool bValidQuadsPerSectionParam = false;
	check(Width > 0 && Height > 0);
		
	const int32 MaxComponents = LandscapeEditorUtils::GetMaxSizeInComponents();
	bool bFoundMatch = false;
	// Try to find a section size and number of sections that exactly matches the dimensions of the heightfield
	for (int32 SectionSizesIdx = UE_ARRAY_COUNT(FLandscapeConfig::SubsectionSizeQuadsValues) - 1; SectionSizesIdx >= 0; SectionSizesIdx--)
	{
		for (int32 NumSectionsIdx = 0; NumSectionsIdx < UE_ARRAY_COUNT(FLandscapeConfig::NumSectionValues); NumSectionsIdx++)
		{
			int32 ss = FLandscapeConfig::SubsectionSizeQuadsValues[SectionSizesIdx];
			int32 ns = FLandscapeConfig::NumSectionValues[NumSectionsIdx];

			// Check if the passed in values are found in the array of valid values
			bValidSubsectionSizeParam |= (InOutSectionsPerComponent == ns);
			bValidQuadsPerSectionParam |= (InOutQuadsPerSection == ss);

			if (((Width - 1) % (ss * ns)) == 0 && ((Width - 1) / (ss * ns)) <= MaxComponents &&
				((Height - 1) % (ss * ns)) == 0 && ((Height - 1) / (ss * ns)) <= MaxComponents)
			{
				bFoundMatch = true;
				InOutQuadsPerSection = ss;
				InOutSectionsPerComponent = ns;
				OutComponentCount.X = (Width - 1) / (ss * ns);
				OutComponentCount.Y = (Height - 1) / (ss * ns);
				break;
			}
		}
		if (bFoundMatch)
		{
			break;
		}
	}

	if (!bFoundMatch)
	{
		if (!bValidSubsectionSizeParam)
		{
			InOutSectionsPerComponent = FLandscapeConfig::NumSectionValues[0];
		}

		if (!bValidQuadsPerSectionParam)
		{
			InOutQuadsPerSection = FLandscapeConfig::SubsectionSizeQuadsValues[0];
		}

		// if there was no exact match, try increasing the section size until we encompass the whole heightmap
		const int32 CurrentSectionSize = InOutQuadsPerSection;
		const int32 CurrentNumSections = InOutSectionsPerComponent;
		for (int32 SectionSizesIdx = 0; SectionSizesIdx < UE_ARRAY_COUNT(FLandscapeConfig::SubsectionSizeQuadsValues); SectionSizesIdx++)
		{
			if (FLandscapeConfig::SubsectionSizeQuadsValues[SectionSizesIdx] < CurrentSectionSize)
			{
				continue;
			}

			const int32 ComponentsX = FMath::DivideAndRoundUp((Width - 1), FLandscapeConfig::SubsectionSizeQuadsValues[SectionSizesIdx] * CurrentNumSections);
			const int32 ComponentsY = FMath::DivideAndRoundUp((Height - 1), FLandscapeConfig::SubsectionSizeQuadsValues[SectionSizesIdx] * CurrentNumSections);
			if (ComponentsX <= 32 && ComponentsY <= 32)
			{
				bFoundMatch = true;
				InOutQuadsPerSection = FLandscapeConfig::SubsectionSizeQuadsValues[SectionSizesIdx];
				OutComponentCount.X = ComponentsX;
				OutComponentCount.Y = ComponentsY;
				break;
			}
		}
	}

	if (!bFoundMatch)
	{
		// if the heightmap is very large, fall back to using the largest values we support
		const int32 MaxSectionSize = FLandscapeConfig::SubsectionSizeQuadsValues[UE_ARRAY_COUNT(FLandscapeConfig::SubsectionSizeQuadsValues) - 1];
		const int32 MaxNumSubSections = FLandscapeConfig::NumSectionValues[UE_ARRAY_COUNT(FLandscapeConfig::NumSectionValues) - 1];
		const int32 ComponentsX = FMath::DivideAndRoundUp((Width - 1), MaxSectionSize * MaxNumSubSections);
		const int32 ComponentsY = FMath::DivideAndRoundUp((Height - 1), MaxSectionSize * MaxNumSubSections);

		bFoundMatch = true;
		InOutQuadsPerSection = MaxSectionSize;
		InOutSectionsPerComponent = MaxNumSubSections;
		OutComponentCount.X = ComponentsX;
		OutComponentCount.Y = ComponentsY;
	}

	check(bFoundMatch);
}

#undef LOCTEXT_NAMESPACE
