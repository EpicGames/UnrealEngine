// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeProxyUIDetails.h"

#include "Algo/AnyOf.h"
#include "Algo/Count.h"
#include "Algo/Find.h"
#include "Algo/Transform.h"
#include "Components/RuntimeVirtualTextureComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/PlatformCrt.h"
#include "IDetailPropertyRow.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Landscape.h"
#include "LandscapeEditTypes.h"
#include "LandscapeInfo.h"
#include "LandscapeProxy.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeSubsystem.h"
#include "Layout/Margin.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "RuntimeVirtualTextureSetBounds.h"
#include "ScopedTransaction.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/LazyObjectPtr.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "VT/RuntimeVirtualTextureVolume.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/IToolTip.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/STextComboBox.h"

class IPropertyHandle;
class SWidget;
class UObject;
class URuntimeVirtualTexture;

#define LOCTEXT_NAMESPACE "FLandscapeProxyUIDetails"

FLandscapeProxyUIDetails::FLandscapeProxyUIDetails()
{
	// Position Precision options are copied from StaticMeshEditorTools.cpp 
	auto PositionPrecisionValueToDisplayString = [](int32 Value)
	{
		if(Value <= 0)
		{
			return FString::Printf(TEXT("%dcm"), 1 << (-Value));
		}
		else
		{
			const float fValue = static_cast<float>(FMath::Exp2((double)-Value));
			return FString::Printf(TEXT("1/%dcm (%.3gcm)"), 1 << Value, fValue);
		}
	};
	
	for (int32 i = MinNanitePrecision; i <= MaxNanitePrecision; i++)
	{
		TSharedPtr<FString> Option = MakeShared<FString>(PositionPrecisionValueToDisplayString(i));
		
		PositionPrecisionOptions.Add(Option);
	}
}

TSharedRef<IDetailCustomization> FLandscapeProxyUIDetails::MakeInstance()
{
	return MakeShareable( new FLandscapeProxyUIDetails);
}

void FLandscapeProxyUIDetails::CustomizeDetails( IDetailLayoutBuilder& DetailBuilder )
{
	DetailLayoutBuilder = &DetailBuilder;

	TArray<TWeakObjectPtr<UObject>> EditingObjects;
	DetailBuilder.GetObjectsBeingCustomized(EditingObjects);

	auto GenerateTextWidget = [](const FText& InText, bool bInBold = false) -> TSharedRef<SWidget>
	{
		return SNew(STextBlock)
				.Font(bInBold? IDetailLayoutBuilder::GetDetailFontBold() : IDetailLayoutBuilder::GetDetailFont())
				.Text(InText);
	};

	TArray<TWeakObjectPtr<ALandscapeProxy>> EditingProxies;
	Algo::Transform(EditingObjects, EditingProxies, [](TWeakObjectPtr<UObject> InObject) { return TWeakObjectPtr<ALandscapeProxy>(Cast<ALandscapeProxy>(InObject.Get())); });

	TArray<TWeakObjectPtr<ALandscape>> LandscapeActors;
	TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> EditingStreamingProxies;
	for (TWeakObjectPtr<ALandscapeProxy> EditingProxy : EditingProxies)
	{
		if (EditingProxy.IsValid())
		{
			if (ALandscape* LandscapeActor = EditingProxy->GetLandscapeActor())
			{
				LandscapeActors.AddUnique(LandscapeActor);
			}

			if (ALandscapeStreamingProxy* LandscapeStreamingProxy = Cast<ALandscapeStreamingProxy>(EditingProxy.Get()))
			{
				EditingStreamingProxies.Add(LandscapeStreamingProxy);
			}
		}
	}

	ALandscapeStreamingProxy* LandscapeStreamingProxy = EditingStreamingProxies.IsEmpty() ? nullptr : EditingStreamingProxies[0].Get();

	// Hide World Partition specific properties in non WP levels
	const bool bShouldDisplayWorldPartitionProperties = Algo::AnyOf(EditingProxies, [](const TWeakObjectPtr<ALandscapeProxy> InProxy)
	{
		UWorld* World = InProxy.IsValid() ? InProxy->GetTypedOuter<UWorld>() : nullptr;
		return UWorld::IsPartitionedWorld(World);
	});

	if (!bShouldDisplayWorldPartitionProperties)
	{
		DetailBuilder.HideProperty(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ALandscape, bAreNewLandscapeActorsSpatiallyLoaded), ALandscape::StaticClass()));
	}

	if (LandscapeActors.Num() == 1)
	{
		TWeakObjectPtr<ALandscape> LandscapeActor = LandscapeActors[0];
		if (ULandscapeInfo* LandscapeInfo = LandscapeActor->GetLandscapeInfo())
		{
			IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("Information", FText::GetEmpty(), ECategoryPriority::Important);

			FText CreateRVTVolumesTooltip = LOCTEXT("Button_CreateVolumes_Tooltip", "Create volumes for the selected Runtime Virtual Textures.");
				
			FText RowDisplayText = LOCTEXT("LandscapeComponentResolution", "Component Resolution (Verts)");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeComponentResolution"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("LandscapeComponentResolutionValue", "{0} x {0}"), LandscapeActor->ComponentSizeQuads+1), true) // Verts
			];

			RowDisplayText = LOCTEXT("LandscapeComponentCount", "Component Count");
			int32 NumComponents = LandscapeActor->LandscapeComponents.Num();
			// If displaying streaming proxies, don't show the main landscape actor's component count, which is always 0 : 
			if (!EditingStreamingProxies.IsEmpty())
			{
				NumComponents = EditingStreamingProxies[0]->LandscapeComponents.Num();
				if (Algo::AnyOf(EditingStreamingProxies, [NumComponents](const TWeakObjectPtr<ALandscapeStreamingProxy> InStreamingProxy) { return InStreamingProxy.Get()->LandscapeComponents.Num() != NumComponents; }))
				{
					NumComponents = -1;
				}
			}
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeComponentCount"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget((NumComponents == -1) ? LOCTEXT("MultipleValues", "Multiple Values") : FText::Format(LOCTEXT("LandscapeComponentCountValue", "{0}"), NumComponents), true)
			];

			RowDisplayText = LOCTEXT("LandscapeComponentSubsections", "Component Subsections");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeComponentSubsections"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("LandscapeComponentSubSectionsValue", "{0} x {0}"), LandscapeActor->NumSubsections), true)
			];

			FIntRect Rect = LandscapeActor->GetBoundingRect();
			FIntPoint Size = Rect.Size();
			RowDisplayText = LOCTEXT("LandscapeResolution", "Resolution (Verts)");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeResolution"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("LandscapeResolutionValue", "{0} x {1}"), Size.X+1, Size.Y+1), true)
			];

			int32 LandscapeCount = LandscapeInfo->GetSortedStreamingProxies().Num() + (LandscapeInfo->LandscapeActor.Get() ? 1 : 0);
			RowDisplayText = LOCTEXT("LandscapeCount", "Landscape Proxy Count");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeCount"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("LandscapeProxyCountValue", "{0}"), LandscapeCount), true)
			];

			int32 TotalComponentCount = LandscapeInfo->XYtoComponentMap.Num();
			RowDisplayText = LOCTEXT("TotalLandscapeComponentCount", "Total Component Count");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("TotalLandscapeComponentCount"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("TotalLandscapeComponentCountValue", "{0}"), TotalComponentCount), true)
			];

			LandscapeInfo->GetLandscapeExtent(Rect.Min.X, Rect.Min.Y, Rect.Max.X, Rect.Max.Y);
			Size = Rect.Size();
			RowDisplayText = LOCTEXT("LandscapeOverallResolution", "Overall Resolution (Verts)");
			CategoryBuilder.AddCustomRow(RowDisplayText)
			.RowTag(TEXT("LandscapeOverallResolution"))
			.NameContent()
			[
				GenerateTextWidget(RowDisplayText)
			]
			.ValueContent()
			[
				GenerateTextWidget(FText::Format(LOCTEXT("LandscapeOverallResolutionValue", "{0} x {1}"), Size.X + 1, Size.Y + 1), true)
			];

			// Apply custom widget for CreateVolume.
			TSharedRef<IPropertyHandle> CreateVolumePropertyHandle = DetailBuilder.GetProperty(TEXT("bSetCreateRuntimeVirtualTextureVolumes"));
			DetailBuilder.EditDefaultProperty(CreateVolumePropertyHandle)->CustomWidget()
			.NameContent()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text(LOCTEXT("Button_CreateVolumes", "Create Volumes"))
				.ToolTipText(CreateRVTVolumesTooltip)
			]
			.ValueContent()
			.MinDesiredWidth(125.f)
			[
				SNew(SButton)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ContentPadding(2.f)
				.Text(LOCTEXT("Button_CreateVolumes", "Create Volumes"))
				.OnClicked_Lambda([LandscapeActor, this]() { return CreateRuntimeVirtualTextureVolume(LandscapeActor.Get()); })
				.IsEnabled_Lambda([LandscapeActor, this]() { return IsCreateRuntimeVirtualTextureVolumeEnabled(LandscapeActor.Get()); })
				.ToolTipText_Lambda([LandscapeActor, this, CreateRVTVolumesTooltip]()
				{
					return IsCreateRuntimeVirtualTextureVolumeEnabled(LandscapeActor.Get())
						? CreateRVTVolumesTooltip
						: LOCTEXT("Button_CreateVolumes_Tooltip_Invalid", "No Runtime Virtual Textures found in the world.");
				})
			];
		}
		
		TSharedRef<IPropertyHandle> NanitePositionPrecisionHandle = DetailBuilder.GetProperty(TEXT("NanitePositionPrecision"));
		IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(NanitePositionPrecisionHandle);

		// todo don.boogert : this is the handling of disabling of inherited properties on Landscape Proxies. 
		// todo don.boogert : be able to handle override & inherited properties which also have custom UI.
		if (LandscapeStreamingProxy->IsPropertyInherited(NanitePositionPrecisionHandle->GetProperty()))
		{
			if (LandscapeStreamingProxy != nullptr)
			{
				if (DetailRow != nullptr)
				{
					// Extend the tool tip to indicate this property is inherited
					FText ToolTipText = NanitePositionPrecisionHandle->GetToolTipText();
					DetailRow->ToolTip(FText::Format(LOCTEXT("InheritedProperty", "{0} This property is inherited from the parent Landscape proxy."), ToolTipText));
			
					// Disable the property editing
					DetailRow->IsEnabled(false);
				}
			}
		}
		
		auto GetPositionPrecision = [MinNanitePrecision = this->MinNanitePrecision, &PositionOptions = this->PositionPrecisionOptions, LandscapeActor]()
		{
			return PositionOptions[LandscapeActor->GetNanitePositionPrecision() - MinNanitePrecision] ;
		};

		auto SetPositionPrecision = [NanitePositionPrecisionHandle, MinNanitePrecision = this->MinNanitePrecision, &PositionOptions = this->PositionPrecisionOptions, LandscapeActor](TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
		{
			if (!LandscapeActor.IsValid())
			{
				return;
			}
					
			if (const int32 Index = PositionOptions.Find(NewValue); Index != INDEX_NONE)
			{
				NanitePositionPrecisionHandle->SetValue(Index + MinNanitePrecision);
			}
		};
		
		DetailRow->CustomWidget()
		.NameContent()
		[
			SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(LOCTEXT("LandscapeNanitePositionPrecision", "Nanite Position Precision"))
			.ToolTipText(LOCTEXT("LandscapeNanitePositionPrecisionTooltip", "Precision of Nanite vertex positions in World Space."))
		]
		.ValueContent()
		.VAlign(VAlign_Center)
		[
			SNew(STextComboBox)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.OptionsSource(&PositionPrecisionOptions)
			.InitiallySelectedItem(GetPositionPrecision())
			.OnSelectionChanged_Lambda(SetPositionPrecision)
		];
	}

	// Add Nanite buttons :
	if (!EditingProxies.IsEmpty())
	{
		auto BuildNaniteData = [EditingProxies](UE::Landscape::EBuildFlags InBuildFlags) -> FReply
		{
			TArray<ALandscapeProxy*> ProxiesToBuild;
			Algo::TransformIf(EditingProxies, ProxiesToBuild, [](const TWeakObjectPtr<ALandscapeProxy>& InProxy) { return InProxy.IsValid(); }, [](const TWeakObjectPtr<ALandscapeProxy>& InProxy) { return InProxy.Get(); });
			if (ProxiesToBuild.IsEmpty())
			{
				return FReply::Unhandled();
			}

			if (UWorld* World = ProxiesToBuild[0]->GetWorld())
			{
				if (ULandscapeSubsystem* LandscapeSubsystem = World->GetSubsystem<ULandscapeSubsystem>())
				{
					LandscapeSubsystem->BuildNanite(InBuildFlags, MakeArrayView(ProxiesToBuild));
				}
			}

			return FReply::Handled();
		};

		auto GetNumProxiesNeedingRebuild = [EditingProxies](UE::Landscape::EBuildFlags InBuildFlags) -> int32
		{
			const bool bForceRebuild = EnumHasAnyFlags(InBuildFlags, UE::Landscape::EBuildFlags::ForceRebuild);
			TSet<TWeakObjectPtr<ALandscapeProxy>> ProxiesToBuild;
			for (TWeakObjectPtr<ALandscapeProxy> EditingProxy : EditingProxies)
			{
				if (EditingProxy.IsValid())
				{
					ProxiesToBuild.Add(EditingProxy);
					// Build all streaming proxies in the case of a ALandscape :
					if (ALandscape* Landscape = Cast<ALandscape>(EditingProxy.Get()))
					{
						ULandscapeInfo* LandscapeInfo = Landscape->GetLandscapeInfo();
						if (LandscapeInfo != nullptr)
						{
							Algo::Transform(LandscapeInfo->GetSortedStreamingProxies(), ProxiesToBuild, [](const TWeakObjectPtr<ALandscapeStreamingProxy>& InStreamingProxy) { return InStreamingProxy; });
						}
					}
				}
			}

			return static_cast<int32>(Algo::CountIf(ProxiesToBuild, [bForceRebuild](TWeakObjectPtr<ALandscapeProxy> InProxy) { return (InProxy.IsValid() && (bForceRebuild || !InProxy->IsNaniteMeshUpToDate())); }));
		};

		auto HasAtLeastOneNaniteLandscape = [LandscapeActors]() -> bool
		{
			return Algo::FindByPredicate(LandscapeActors, [](TWeakObjectPtr<ALandscape> InLandscape) { return (InLandscape.IsValid() && InLandscape->IsNaniteEnabled()); }) != nullptr;
		};

		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("Nanite", FText::GetEmpty(), ECategoryPriority::Default);
		CategoryBuilder.AddCustomRow(FText::FromString(TEXT("Rebuild Nanite Data")))
		.RowTag("RebuildNaniteData")
		[
			SNew(SHorizontalBox)
			.IsEnabled_Lambda([HasAtLeastOneNaniteLandscape] { return HasAtLeastOneNaniteLandscape(); })
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SButton)
				.Text(LOCTEXT("BuildNaniteData", "Build Data"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ToolTipText_Lambda([GetNumProxiesNeedingRebuild]()
				{ 
					return FText::Format(LOCTEXT("BuildNaniteDataTooltip", "Builds the Nanite mesh representation from the Landscape data if it's not up to date ({0} {0}|plural(one=actors, other=actors) to build)"), GetNumProxiesNeedingRebuild(UE::Landscape::EBuildFlags::None));
				})
				.OnClicked_Lambda([BuildNaniteData]() { return BuildNaniteData(UE::Landscape::EBuildFlags::WriteFinalLog); })
				.IsEnabled_Lambda([GetNumProxiesNeedingRebuild]() { return (GetNumProxiesNeedingRebuild(UE::Landscape::EBuildFlags::None) > 0); })
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SButton)
				.Text(LOCTEXT("RebuildNaniteData", "Rebuild Data"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ToolTipText_Lambda([GetNumProxiesNeedingRebuild]()
				{ 
					return FText::Format(LOCTEXT("RebuildNaniteDataTooltip", "Rebuilds the Nanite mesh representation from the Landscape data ({0} {0}|plural(one=actors, other=actors) to build)"), GetNumProxiesNeedingRebuild(UE::Landscape::EBuildFlags::ForceRebuild));
				})
				.OnClicked_Lambda([BuildNaniteData]() { return BuildNaniteData(UE::Landscape::EBuildFlags::WriteFinalLog | UE::Landscape::EBuildFlags::ForceRebuild); })
			]
		];
	}

	if (LandscapeStreamingProxy != nullptr)
	{
		for (TFieldIterator<FProperty> PropertyIterator(LandscapeStreamingProxy->GetClass()); PropertyIterator; ++PropertyIterator)
		{
			FProperty* Property = *PropertyIterator;

			if (Property == nullptr)
			{
				continue;
			}

			if (LandscapeStreamingProxy->IsPropertyInherited(Property))
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(Property->GetFName());
				
				if (PropertyHandle->IsValidHandle())
				{
					IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(PropertyHandle);

					// Extend the tool tip to indicate this property is inherited
					FText ToolTipText = PropertyHandle->GetToolTipText();
					DetailRow->ToolTip(FText::Format(LOCTEXT("InheritedProperty", "{0} This property is inherited from the parent Landscape proxy."), ToolTipText));

					// Disable the property editing
					DetailRow->IsEnabled(false);
				}
			}
			else if (LandscapeStreamingProxy->IsPropertyOverridable(Property))
			{
				TSharedPtr<IPropertyHandle> PropertyHandle = DetailBuilder.GetProperty(Property->GetFName());
				
				if (PropertyHandle->IsValidHandle())
				{
					const FText TooltipText = LOCTEXT("OverriddenProperty", "Check this box to override the parent landscape's property.");
					FName PropertyName = Property->GetFName();
					IDetailPropertyRow* DetailRow = DetailBuilder.EditDefaultProperty(PropertyHandle);
					TSharedPtr<SWidget> NameWidget = nullptr;
					TSharedPtr<SWidget> ValueWidget = nullptr;
					
					DetailRow->GetDefaultWidgets(NameWidget, ValueWidget);

					TAttribute<bool> EnabledAttribute = TAttribute<bool>::Create([LandscapeStreamingProxy, PropertyName]() -> bool
					{
						return LandscapeStreamingProxy->IsSharedPropertyOverridden(PropertyName);
					});

					DetailRow->CustomWidget(/*bShowChildren = */true)
					.NameContent()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SCheckBox)
							.ToolTipText(TooltipText)
							.IsChecked(this, &FLandscapeProxyUIDetails::IsOverriddenPropertyChecked, EditingStreamingProxies, PropertyName)
							.OnCheckStateChanged(this, &FLandscapeProxyUIDetails::OnCheckStateChanged, EditingStreamingProxies, PropertyName)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						[
							NameWidget->AsShared()
						]
					]
					.ValueContent()
					[
						ValueWidget->AsShared()
					];
					
					// Hook the EnabledAttribute on the 2 sub-widgets by hand, so that we don't disable the entire custom widget
					//  when the property is not overridden (since that would also disable the override checkbox...)
					NameWidget->SetEnabled(EnabledAttribute);
					ValueWidget->SetEnabled(EnabledAttribute);

					// The Reset To Default button corresponds to the overridden state of the overridable property : 
					DetailRow->OverrideResetToDefault(FResetToDefaultOverride::Create(
						FIsResetToDefaultVisible::CreateLambda([EditingStreamingProxies, PropertyName](TSharedPtr<IPropertyHandle> InPropertyHandle) -> bool
							{
								return Algo::AnyOf(EditingStreamingProxies, [PropertyName](TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxy) { return StreamingProxy.IsValid() && StreamingProxy->IsSharedPropertyOverridden(PropertyName); });
							}),
						FResetToDefaultHandler::CreateLambda([EditingStreamingProxies, PropertyName](TSharedPtr<IPropertyHandle> InPropertyHandle)
							{
								const FScopedTransaction Transaction(LOCTEXT("ResetSetSharedPropertyOverride", "Reset Property Override"));
								for (const TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxy : EditingStreamingProxies)
								{
									if (!StreamingProxy.IsValid())
									{
										continue;
									}

									StreamingProxy->SetSharedPropertyOverride(PropertyName, false);
								}
							})));
				}
			}
		}
	}
}

ECheckBoxState FLandscapeProxyUIDetails::IsOverriddenPropertyChecked(TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> InEditingStreamingProxies, FName InPropertyName) const
{
	bool bContainsOverriddenProperty = false;
	bool bContainsDefaultProperty = false;

	for (const TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxy : InEditingStreamingProxies)
	{
		if (!StreamingProxy.IsValid())
		{
			continue;
		}

		const bool bIsPropertyOverridden = StreamingProxy->IsSharedPropertyOverridden(InPropertyName);

		bContainsOverriddenProperty |= bIsPropertyOverridden;
		bContainsDefaultProperty |= !bIsPropertyOverridden;
	}

	if (bContainsOverriddenProperty && bContainsDefaultProperty)
	{
		return ECheckBoxState::Undetermined;
	}

	return bContainsOverriddenProperty ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void FLandscapeProxyUIDetails::OnCheckStateChanged(ECheckBoxState InNewState, TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> InEditingStreamingProxies, FName InPropertyName)
{
	if (InNewState == ECheckBoxState::Undetermined)
	{
		return;
	}

	const bool bChecked = (InNewState == ECheckBoxState::Checked);

	const FScopedTransaction Transaction(LOCTEXT("SetSharedPropertyOverride", "Change Property Override"));

	for (const TWeakObjectPtr<ALandscapeStreamingProxy> StreamingProxy : InEditingStreamingProxies)
	{
		if (!StreamingProxy.IsValid())
		{
			continue;
		}

		StreamingProxy->SetSharedPropertyOverride(InPropertyName, bChecked);
	}

	check(DetailLayoutBuilder != nullptr);
	// Refresh the details to disable/enable the widgets that should be : we need this to force CanEditChange to be re-evaluated after the property was overridden/de-overridden
	DetailLayoutBuilder->ForceRefreshDetails();
}

static void GetMissingRuntimeVirtualTextureVolumes(ALandscape* InLandscapeActor, TArray<URuntimeVirtualTexture*>& OutVirtualTextures)
{
	UWorld* World = InLandscapeActor != nullptr ? InLandscapeActor->GetWorld() : nullptr;
	if (World == nullptr)
	{
		return;
	}

	TArray<URuntimeVirtualTexture*> FoundVolumes;
	for (TObjectIterator<URuntimeVirtualTextureComponent> It(RF_ClassDefaultObject, false, EInternalObjectFlags::Garbage); It; ++It)
	{
		if (It->GetWorld() == World)
		{
			if (URuntimeVirtualTexture* VirtualTexture = It->GetVirtualTexture())
			{
				FoundVolumes.Add(VirtualTexture);
			}
		}
	}

	for (URuntimeVirtualTexture* VirtualTexture : InLandscapeActor->RuntimeVirtualTextures)
	{
		if (VirtualTexture != nullptr && FoundVolumes.Find(VirtualTexture) == INDEX_NONE)
		{
			OutVirtualTextures.Add(VirtualTexture);
		}
	}
}

bool FLandscapeProxyUIDetails::IsCreateRuntimeVirtualTextureVolumeEnabled(ALandscape* InLandscapeActor) const
{
	if (InLandscapeActor == nullptr)
	{
		return false;
	}

	TArray<URuntimeVirtualTexture*> VirtualTextureVolumesToCreate;
	GetMissingRuntimeVirtualTextureVolumes(InLandscapeActor, VirtualTextureVolumesToCreate);
	return VirtualTextureVolumesToCreate.Num() > 0;
}

FReply FLandscapeProxyUIDetails::CreateRuntimeVirtualTextureVolume(ALandscape* InLandscapeActor)
{
	if (InLandscapeActor == nullptr)
	{
		return FReply::Unhandled();
	}

	TArray<URuntimeVirtualTexture*> VirtualTextureVolumesToCreate;
	GetMissingRuntimeVirtualTextureVolumes(InLandscapeActor, VirtualTextureVolumesToCreate);
	if (VirtualTextureVolumesToCreate.Num() == 0)
	{
		return FReply::Unhandled();
	}

	const FScopedTransaction Transaction(LOCTEXT("Transaction_CreateVolumes", "Create Runtime Virtual Texture Volumes"));
	
	for (URuntimeVirtualTexture* VirtualTexture : VirtualTextureVolumesToCreate)
	{
		ARuntimeVirtualTextureVolume* NewVolume = InLandscapeActor->GetWorld()->SpawnActor<ARuntimeVirtualTextureVolume>();
		NewVolume->VirtualTextureComponent->SetVirtualTexture(VirtualTexture);
		NewVolume->VirtualTextureComponent->SetBoundsAlignActor(InLandscapeActor);
		RuntimeVirtualTexture::SetBounds(NewVolume->VirtualTextureComponent);
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
