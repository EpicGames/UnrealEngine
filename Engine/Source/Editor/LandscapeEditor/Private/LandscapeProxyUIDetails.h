// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"

class ALandscape;
class ALandscapeStreamingProxy;
class IDetailLayoutBuilder;
enum class ECheckBoxState : uint8;

class FLandscapeProxyUIDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails( IDetailLayoutBuilder& DetailBuilder ) override;
	/** End IDetailCustomization interface */

private:
	/** Use MakeInstance to create an instance of this class */
	FLandscapeProxyUIDetails();

	/** Returns true if SetBounds button is enabled */
	bool IsCreateRuntimeVirtualTextureVolumeEnabled(ALandscape* InLandscapeActor) const;

	/** Callbacks for the override property checkbox */
	ECheckBoxState IsOverriddenPropertyChecked(TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> InEditingStreamingProxies, FName InPropertyName) const;
	void OnCheckStateChanged(ECheckBoxState InNewState, TArray<TWeakObjectPtr<ALandscapeStreamingProxy>> InEditingStreamingProxies, FName InPropertyName);
	
	/** Callback for Set Bounds button */
	FReply CreateRuntimeVirtualTextureVolume(ALandscape* InLandscapeActor);
	
	IDetailLayoutBuilder* DetailLayoutBuilder = nullptr;

	/** Nanite Position Precision Options */
	TArray<TSharedPtr<FString>> PositionPrecisionOptions;
	static constexpr int MinNanitePrecision = -6;
	static constexpr int MaxNanitePrecision = 13;
};
