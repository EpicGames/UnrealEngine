// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeRegionUtils.h"

#include "Landscape.h"

#include "SourceControlHelpers.h"
#include "UObject/SavePackage.h"
#include "Builders/CubeBuilder.h"
#include "ActorFactories/ActorFactory.h"
#include "LandscapeProxy.h"
#include "LocationVolume.h"
#include "LandscapeStreamingProxy.h"
#include "LandscapeEditorUtils.h"
#include "LandscapeEditorPrivate.h"
#include "Engine/Level.h"

namespace LandscapeRegionUtils
{

ALocationVolume* CreateLandscapeRegionVolume(UWorld* InWorld, ALandscapeProxy* InParentLandscapeActor, const FIntPoint& InRegionCoordinate, double InRegionSize)
{
	const double Shrink = 0.95;

	FVector ParentLocation = InParentLandscapeActor->GetActorLocation();
	FVector Location = ParentLocation + FVector(InRegionCoordinate.X * InRegionSize, InRegionCoordinate.Y * InRegionSize, 0.0) + FVector(InRegionSize / 2.0, InRegionSize / 2.0, 0.0);
	FRotator Rotation = FRotator::ZeroRotator;
	FActorSpawnParameters SpawnParameters;

	const FString Label = FString::Printf(TEXT("LandscapeRegion_%i_%i"), InRegionCoordinate.X, InRegionCoordinate.Y);
	SpawnParameters.Name = FName(*Label);
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

	ALocationVolume* LocationVolume = InWorld->SpawnActor<ALocationVolume>(Location, Rotation, SpawnParameters);
	if (LocationVolume == nullptr)
	{
		return nullptr;
	}
	LocationVolume->SetActorLabel(Label);

	LocationVolume->AttachToActor(InParentLandscapeActor, FAttachmentTransformRules::KeepWorldTransform);
	FVector Scale{ InRegionSize * Shrink,  InRegionSize * Shrink, InRegionSize * 0.5f };
	LocationVolume->SetActorScale3D(Scale);

	// Specify a cube shape for the LocationVolume
	UCubeBuilder* Builder = NewObject<UCubeBuilder>();
	Builder->X = 1.0f;
	Builder->Y = 1.0f;
	Builder->Z = 1.0f;
	UActorFactory::CreateBrushForVolumeActor(LocationVolume, Builder);

	LocationVolume->Tags.Add(FName("LandscapeRegion"));
	return LocationVolume;
}

void ForEachComponentByRegion(int32 RegionSize, const TArray<FIntPoint>& ComponentCoordinates, TFunctionRef<bool(const FIntPoint&, const TArray<FIntPoint>&)> RegionFn)
{
	if (RegionSize <= 0)
	{
		UE_LOGF(LogLandscapeTools, Error, "RegionSize must be greater than zero");
		return;
	}

	TMap<FIntPoint, TArray< FIntPoint>> Regions;
	for (FIntPoint ComponentCoordinate : ComponentCoordinates)
	{
		FIntPoint RegionCoordinate;
		RegionCoordinate.X = (ComponentCoordinate.X) / RegionSize;
		RegionCoordinate.Y = (ComponentCoordinate.Y) / RegionSize;

		if (TArray<FIntPoint>* ComponentIndices = Regions.Find(RegionCoordinate))
		{
			ComponentIndices->Add(ComponentCoordinate);
		}
		else
		{
			Regions.Add(RegionCoordinate, TArray<FIntPoint> {ComponentCoordinate});
		}
	}

	auto Sorter = [](const FIntPoint& A, const FIntPoint& B)
	{
		if (A.Y == B.Y)
			return A.X < B.X;

		return A.Y < B.Y;
	};

	Regions.KeySort(Sorter);

	for (const TPair< FIntPoint, TArray< FIntPoint>>& RegionIt : Regions)
	{
		if (!RegionFn(RegionIt.Key, RegionIt.Value))
		{
			break;
		}
	}
}

void ForEachRegion_LoadProcessUnload(ULandscapeInfo* InLandscapeInfo, UWorld* InWorld, TFunctionRef<bool(const FBox&, const TArray<ALandscapeProxy*>)> InRegionFn)
{
	TArray<AActor*> Children;
	InLandscapeInfo->LandscapeActor->GetAttachedActors(Children);
	TArray<ALocationVolume*> LandscapeRegions;

	for (AActor* Child : Children)
	{
		if (Child->IsA<ALocationVolume>())
		{
			LandscapeRegions.Add(Cast<ALocationVolume>(Child));
		}
	}

	for (ALocationVolume* Region : LandscapeRegions)
	{
		Region->Load();

		FBox RegionBounds = Region->GetComponentsBoundingBox();

		TArray<AActor*> AllActors = InWorld->GetLevel(0)->Actors;
		TArray<ALandscapeProxy*> LandscapeProxies;
		for (AActor* Actor : AllActors)
		{
			if (ALandscapeStreamingProxy* Proxy = Cast<ALandscapeStreamingProxy>(Actor))
			{
				LandscapeProxies.Add(Proxy);
			}
		}

		// Save the actor
		bool bShouldExit = !InRegionFn(RegionBounds, LandscapeProxies);

		InLandscapeInfo->ForceLayersFullUpdate();

		LandscapeEditorUtils::SaveLandscapeProxies(InWorld, MakeArrayView(LandscapeProxies));

		Region->Unload();

		if (bShouldExit)
		{
			break;
		}
	}
}

int32 NumLandscapeRegions(ULandscapeInfo* InLandscapeInfo)
{
	int32 NumRegions = 0;
	TArray<AActor*> Children;
	InLandscapeInfo->LandscapeActor->GetAttachedActors(Children);
	TArray<ALocationVolume*> LandscapeRegions;

	for (AActor* Child : Children)
	{
		NumRegions += Child->IsA<ALocationVolume>() ? 1 : 0;
	}

	return NumRegions;
}
} // LandscapeEditor