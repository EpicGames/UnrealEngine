// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h"
#include "Containers/Array.h"
#include "Templates/Function.h"

class AActor;
class UWorld;
class ALandscapeProxy;
class ULandscapeInfo;
class ALandscapeStreamingProxy;
class ALandscapeProxy;
class ALocationVolume;

namespace LandscapeRegionUtils
{
	// Create a LocationVolume for a Region located at RegionCoordinate with a Box shape of dimension RegionSize
	ALocationVolume* CreateLandscapeRegionVolume(UWorld* World, ALandscapeProxy* ParentLandscapeActor, const FIntPoint& RegionCoordinate, double RegionSize);

	// Iterate over all Components grouping by Region
	void ForEachComponentByRegion(int32 RegionSize, const TArray<FIntPoint>& ComponentCoordinates, TFunctionRef<bool(const FIntPoint&, const TArray<FIntPoint>&)> RegionFn);

	// Load, Process (call RegionFn) and Unload the Region
	void ForEachRegion_LoadProcessUnload(ULandscapeInfo* LandscapeInfo, UWorld* World, TFunctionRef<bool(const FBox&, const TArray<ALandscapeProxy*>)> RegionFn);

	// Number of landscape regions in this landscape.
	int32 NumLandscapeRegions(ULandscapeInfo* InLandscapeInfo);
}
