// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineDetails.h"
#include "LandscapeSplineSelection.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "LandscapeEdMode.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Input/SButton.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "DetailWidgetRow.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "ILandscapeSplineInterface.h"

#define LOCTEXT_NAMESPACE "LandscapeSplineDetails"


TSharedRef<IDetailCustomization> FLandscapeSplineDetails::MakeInstance()
{
	return MakeShareable(new FLandscapeSplineDetails);
}

void FLandscapeSplineDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& LandscapeSplineCategory = DetailBuilder.EditCategory("LandscapeSpline", FText::GetEmpty(), ECategoryPriority::Transform);

	LandscapeSplineCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0, 0, 2, 0)
		.VAlign(VAlign_Center)
		.FillWidth(1)
		[
			SNew(STextBlock)
			.Text_Raw(this, &FLandscapeSplineDetails::OnGetSplineOwningLandscapeText)
		]
	];

	FMargin ButtonPadding(2.f, 0.f);

	LandscapeSplineCategory.AddCustomRow(LOCTEXT("SelectSplineElements", "Select Spline Elements"))
	.RowTag("SelectSplineElements")
	.NameContent()
	[
		SNew(STextBlock)
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectSplineSegments", "Select Spline Segments") : LOCTEXT("SelectSplineControlPoints", "Select Spline Points"))
	]
	.ValueContent()
	.VAlign(VAlign_Fill)
	.MaxDesiredWidth(170.f)
	.MinDesiredWidth(170.f)
	[
		SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds)

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectFirst")
				.ContentPadding(2.0f)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectFirstSplineSegmentToolTip", "Select first spline segment.") : LOCTEXT("SelectFirstSplinePointToolTip", "Select first spline point."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectEndLinearSplineElementButtonClicked, ESplineNavigationFlags::DirectionBackward)
				.IsEnabled(this, &FLandscapeSplineDetails::HasEndLinearSplineElement, ESplineNavigationFlags::DirectionBackward)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.AddPrev")
				.ContentPadding(2.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectAddPrevSplineSegmentToolTip", "Add previous segment to current selection.") : LOCTEXT("SelectAddPrevSplinePointToolTip", "Add previous point to current selection."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectAdjacentLinearSplineElementButtonClicked, (ESplineNavigationFlags::DirectionBackward | ESplineNavigationFlags::AddToSelection))
				.IsEnabled(this, &FLandscapeSplineDetails::HasAdjacentLinearSplineElement, (ESplineNavigationFlags::DirectionBackward | ESplineNavigationFlags::AddToSelection))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectPrev")
				.ContentPadding(2.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectPrevSplineSegmentToolTip", "Select previous segment.") : LOCTEXT("SelectPrevPointToolTip", "Select previous point."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectAdjacentLinearSplineElementButtonClicked, ESplineNavigationFlags::DirectionBackward)
				.IsEnabled(this, &FLandscapeSplineDetails::HasAdjacentLinearSplineElement, ESplineNavigationFlags::DirectionBackward)

		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectAll")
				.ContentPadding(2.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectAllSplineSegmentToolTip", "Select all segments.") : LOCTEXT("SelectAllSplinePointsToolTip", "Select all points."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectAllConnectedSplineElementsButtonClicked)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectNext")
				.ContentPadding(2.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectNextSplineSegmentToolTip", "Select next segment.") : LOCTEXT("SelectNextSplinePointToolTip", "Select next point."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectAdjacentLinearSplineElementButtonClicked, ESplineNavigationFlags::DirectionForward)
				.IsEnabled(this, &FLandscapeSplineDetails::HasAdjacentLinearSplineElement, ESplineNavigationFlags::DirectionForward)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.AddNext")
				.ContentPadding(2.f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectAddNextSplineSegmentToolTip", "Add next segment to current selection.") : LOCTEXT("SelectAddNextSplinePointToolTip", "Add next point to current selection."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectAdjacentLinearSplineElementButtonClicked, (ESplineNavigationFlags::DirectionForward | ESplineNavigationFlags::AddToSelection))
				.IsEnabled(this, &FLandscapeSplineDetails::HasAdjacentLinearSplineElement, (ESplineNavigationFlags::DirectionForward | ESplineNavigationFlags::AddToSelection))
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(ButtonPadding)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SplineComponentDetails.SelectLast")
				.ContentPadding(2.0f)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectLastSplineSegmentToolTip", "Select last spline segment.") : LOCTEXT("SelectLastSplinePointToolTip", "Select last spline point."))
				.OnClicked(this, &FLandscapeSplineDetails::OnSelectEndLinearSplineElementButtonClicked, ESplineNavigationFlags::DirectionForward)
				.IsEnabled(this, &FLandscapeSplineDetails::HasEndLinearSplineElement, ESplineNavigationFlags::DirectionForward)
		]

		// Add a vertical divider to separate navigation with conversion
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(ButtonPadding * 3)
		[
			SNew(SSeparator)
			.Orientation(EOrientation::Orient_Vertical)
			.Thickness(2.f)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), IsSegmentSelectModeEnabled() ? "SplineComponentDetails.ConvertToPoints" : "SplineComponentDetails.ConvertToSegments")
				.ContentPadding(ButtonPadding)
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				.ToolTipText(IsSegmentSelectModeEnabled() ? LOCTEXT("SelectConvertSplineSegmentsToPoints", "Switch selected segments to points.") : LOCTEXT("SelectConvertSplinePointsToSegments", "Switch selected points to segments."))
				.OnClicked(this, &FLandscapeSplineDetails::OnToggleSplineSelectionTypeButtonClicked)
		]
	];

	LandscapeSplineCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1)
		[
			SNew(SButton)
			.Text(LOCTEXT("Move Selected ControlPnts+Segs to Current level", "Move to current level"))
			.HAlign(HAlign_Center)
			.OnClicked(this, &FLandscapeSplineDetails::OnMoveToCurrentLevelButtonClicked)
			.IsEnabled(this, &FLandscapeSplineDetails::IsMoveToCurrentLevelButtonEnabled)
		]
	];
	LandscapeSplineCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.FillWidth(1)
		[
			SNew(SButton)
			.Text(LOCTEXT("Move Spline Mesh Components to Proper level", "Update Spline Mesh Levels"))
			.HAlign(HAlign_Center)
			.OnClicked(this, &FLandscapeSplineDetails::OnUpdateSplineMeshLevelsButtonClicked)
			.IsEnabled(this, &FLandscapeSplineDetails::IsUpdateSplineMeshLevelsButtonEnabled)
		]
	];

	IDetailCategoryBuilder& LandscapeSplineSegmentCategory = DetailBuilder.EditCategory("LandscapeSplineSegment", FText::GetEmpty(), ECategoryPriority::Default);
	LandscapeSplineSegmentCategory.AddCustomRow(FText::GetEmpty())
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.Padding(0, 0, 2, 0)
		.VAlign(VAlign_Center)
		.FillWidth(1)
		[
			SNew(SButton)
			.Text(LOCTEXT("FlipSegment", "Flip Selected Segment(s)"))
			.HAlign(HAlign_Center)
			.OnClicked(this, &FLandscapeSplineDetails::OnFlipSegmentButtonClicked)
			.IsEnabled(this, &FLandscapeSplineDetails::IsFlipSegmentButtonEnabled)
		]
	];
}

class FEdModeLandscape* FLandscapeSplineDetails::GetEditorMode() const
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

bool FLandscapeSplineDetails::IsSegmentSelectModeEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		return LandscapeEdMode->HasSelectedSplineSegments();
	}

	return false;
}

FReply FLandscapeSplineDetails::OnFlipSegmentButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		LandscapeEdMode->FlipSelectedSplineSegments();
	}
	return FReply::Handled();
}

bool FLandscapeSplineDetails::HasAdjacentLinearSplineElement(ESplineNavigationFlags Flags) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		// check flags have one direction set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::DirectionMask)) == 1);
		return LandscapeEdMode->HasAdjacentLinearSplineConnection(Flags | (IsSegmentSelectModeEnabled() ? ESplineNavigationFlags::SegmentSelectModeEnabled : ESplineNavigationFlags::ControlPointSelectModeEnabled));
	}
	return false;
}

bool FLandscapeSplineDetails::HasEndLinearSplineElement(ESplineNavigationFlags Flags) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		// check flags have one direction set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::DirectionMask)) == 1);
		return LandscapeEdMode->HasEndLinearSplineConnection(Flags | (IsSegmentSelectModeEnabled() ? ESplineNavigationFlags::SegmentSelectModeEnabled : ESplineNavigationFlags::ControlPointSelectModeEnabled));
	}
	return false;
}

bool FLandscapeSplineDetails::IsFlipSegmentButtonEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		return LandscapeEdMode->HasSelectedSplineSegments();
	}
	return false;
}

FText FLandscapeSplineDetails::OnGetSplineOwningLandscapeText() const
{
	TSet<AActor*> SplineOwners;
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		LandscapeEdMode->GetSelectedSplineOwners(SplineOwners);
	}

	FString SplineOwnersStr;
	for (AActor* Owner : SplineOwners)
	{
		if (Owner)
		{
			if (!SplineOwnersStr.IsEmpty())
			{
				SplineOwnersStr += ", ";
			}

			SplineOwnersStr += Owner->GetActorLabel();
		}
	}
	
	return FText::Format(LOCTEXT("SplineOwner", "Owner: {0}"), FText::FromString(SplineOwnersStr));
}

FReply FLandscapeSplineDetails::OnSelectAdjacentLinearSplineElementButtonClicked(ESplineNavigationFlags Flags) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		// check flags have one direction set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::DirectionMask)) == 1);
		LandscapeEdMode->SelectAdjacentLinearSplineElement(Flags | (IsSegmentSelectModeEnabled() ? ESplineNavigationFlags::SegmentSelectModeEnabled : ESplineNavigationFlags::ControlPointSelectModeEnabled));
	}

	return FReply::Handled();
}

FReply FLandscapeSplineDetails::OnSelectEndLinearSplineElementButtonClicked(ESplineNavigationFlags Flags) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		// check flags have one direction set
		check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::DirectionMask)) == 1);
		LandscapeEdMode->SelectEndLinearSplineElement(Flags | (IsSegmentSelectModeEnabled() ? ESplineNavigationFlags::SegmentSelectModeEnabled : ESplineNavigationFlags::ControlPointSelectModeEnabled));
	}

	return FReply::Handled();
}

FReply FLandscapeSplineDetails::OnToggleSplineSelectionTypeButtonClicked() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		IsSegmentSelectModeEnabled() ? LandscapeEdMode->SelectSplineControlPointsFromCurrentSegmentSelection() : LandscapeEdMode->SelectSplineSegmentsFromCurrentControlPointSelection();
	}

	return FReply::Handled();
}

FReply FLandscapeSplineDetails::OnSelectAllConnectedSplineElementsButtonClicked() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		IsSegmentSelectModeEnabled() ? LandscapeEdMode->SelectAllConnectedSplineSegments() : LandscapeEdMode->SelectAllConnectedSplineControlPoints();
	}

	return FReply::Handled();
}

FReply FLandscapeSplineDetails::OnMoveToCurrentLevelButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		LandscapeEdMode->SplineMoveToCurrentLevel();
	}

	return FReply::Handled();
}

bool FLandscapeSplineDetails::IsMoveToCurrentLevelButtonEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() && LandscapeEdMode->CurrentToolTarget.LandscapeInfo->GetCurrentLevelLandscapeProxy(true))
	{
		return LandscapeEdMode->CanMoveSplineToCurrentLevel();
	}

	return false;
}

FReply FLandscapeSplineDetails::OnUpdateSplineMeshLevelsButtonClicked()
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		LandscapeEdMode->UpdateSplineMeshLevels();
	}
	
	return FReply::Handled();
}

bool FLandscapeSplineDetails::IsUpdateSplineMeshLevelsButtonEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid())
	{
		return LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
