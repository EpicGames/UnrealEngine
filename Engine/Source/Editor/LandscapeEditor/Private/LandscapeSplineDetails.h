// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "IDetailCustomization.h"

class FEdModeLandscape;
class IDetailLayoutBuilder;
enum class ESplineNavigationFlags : uint8;

class FLandscapeSplineDetails : public IDetailCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IDetailCustomization> MakeInstance();

	/** IDetailCustomization interface */
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	FEdModeLandscape* GetEditorMode() const;
	FText OnGetSplineOwningLandscapeText() const;
	FReply OnMoveToCurrentLevelButtonClicked();
	bool IsMoveToCurrentLevelButtonEnabled() const;
	bool IsSegmentSelectModeEnabled() const;

	FReply OnUpdateSplineMeshLevelsButtonClicked();
	bool IsUpdateSplineMeshLevelsButtonEnabled() const;
	
	FReply OnFlipSegmentButtonClicked();
	bool IsFlipSegmentButtonEnabled() const;

	bool HasAdjacentLinearSplineElement(ESplineNavigationFlags Flags) const;
	bool HasEndLinearSplineElement(ESplineNavigationFlags Flags) const;
	FReply OnSelectAdjacentLinearSplineElementButtonClicked(ESplineNavigationFlags Flags) const;
	FReply OnSelectEndLinearSplineElementButtonClicked(ESplineNavigationFlags Flags) const;

	FReply OnSelectAllConnectedSplineElementsButtonClicked() const;
	FReply OnToggleSplineSelectionTypeButtonClicked() const;
};
