// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineImportExport.h"
#include "LandscapeSplineSegment.h"
#include "LandscapeSplineControlPoint.h"

#define LOCTEXT_NAMESPACE "Landscape"

const FString FLandscapeSplineTextObjectFactory::SplineLocationTag("SPLINE LOCATION: ");
const FString FLandscapeSplineTextObjectFactory::SplineBeginTag("BEGIN_SPLINES");
const FString FLandscapeSplineTextObjectFactory::SplineEndTag("END_SPLINES");

FLandscapeSplineTextObjectFactory::FLandscapeSplineTextObjectFactory(FFeedbackContext* InWarningContext /*= GWarn*/)
	: FCustomizableTextObjectFactory(InWarningContext)
{
}

TArray<UObject*> FLandscapeSplineTextObjectFactory::ImportSplines(UObject* InParent, const TCHAR* TextBuffer)
{
	if (FParse::Command(&TextBuffer, *SplineBeginTag))
	{
		ProcessBuffer(InParent, RF_Transactional, TextBuffer);

		check(bProcessedEnd);
	}

	return OutObjects;
}

void FLandscapeSplineTextObjectFactory::ProcessConstructedObject(UObject* CreatedObject)
{
	OutObjects.Add(CreatedObject);

	CreatedObject->PostEditImport();
}

bool FLandscapeSplineTextObjectFactory::CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const
{
	if (ObjectClass == ULandscapeSplineControlPoint::StaticClass() ||
		ObjectClass == ULandscapeSplineSegment::StaticClass())
	{
		return true;
	}

	return false;
}

void FLandscapeSplineTextObjectFactory::ProcessUnidentifiedLine(const FString& StrLine)
{
	if (StrLine.StartsWith(SplineEndTag))
	{
		bProcessedEnd = true;
	}
	else if (StrLine.StartsWith(SplineLocationTag))
	{
		SplineLocation.InitFromString(StrLine.Mid(SplineLocationTag.Len()));
	}
}

#undef LOCTEXT_NAMESPACE
