// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories.h"

class FLandscapeSplineTextObjectFactory : protected FCustomizableTextObjectFactory
{
public:
	FLandscapeSplineTextObjectFactory(FFeedbackContext* InWarningContext = GWarn);

	TArray<UObject*> ImportSplines(UObject* InParent, const TCHAR* TextBuffer);

	static const FString SplineLocationTag;
	static const FString SplineBeginTag;
	static const FString SplineEndTag;

	FVector SplineLocation;
protected:
	TArray<UObject*> OutObjects;
	bool bProcessedEnd = false;
	
	virtual void ProcessConstructedObject(UObject* CreatedObject) override;
	virtual bool CanCreateClass(UClass* ObjectClass, bool& bOmitSubObjs) const override;
	virtual void ProcessUnidentifiedLine(const FString& StrLine) override;
};
