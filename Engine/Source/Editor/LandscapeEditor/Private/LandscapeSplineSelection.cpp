// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeSplineSelection.h"

#include "LandscapeSplineControlPoint.h"
#include "LandscapeSplineSegment.h"
#include "LandscapeSplinesComponent.h"

#include "UnrealEdGlobals.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "UObject/ConstructorHelpers.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LandscapeSplineSelection)

#define LOCTEXT_NAMESPACE "LandscapeSplineSelection"

ULandscapeSplineSelection::ULandscapeSplineSelection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void ULandscapeSplineSelection::SelectSegment(ULandscapeSplineSegment* Segment, ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows | ESplineNavigationFlags::AddToSelection | ESplineNavigationFlags::SegmentSelectModeEnabled));

	// Do nothing if the given point is already selected
	if (IsSegmentSelected(Segment))
	{
		check(Segment->IsSplineSelected());
		return;
	}

	// Before entering segment mode, save the control point mode for persistence
	if (SelectedSplineSegments.Num() == 0)
	{
		ControlPointWidgetMode = GLevelEditorModeTools().GetWidgetMode();
	}

	// Clear all previous selection
	if (!EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
	{
		ClearSelection();
	}

	Modify(false);
	SelectedSplineSegments.Add(Segment);

	SelectNavigationSegment(Segment);

	Segment->Modify(false);
	Segment->SetSplineSelected(true);
	GLevelEditorModeTools().SetWidgetMode(UE::Widget::WM_Scale);

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows))
	{
		UpdatePropertiesWindows();
	}
}

void ULandscapeSplineSelection::SelectControlPoint(ULandscapeSplineControlPoint* ControlPoint, ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows | ESplineNavigationFlags::AddToSelection | ESplineNavigationFlags::ControlPointSelectModeEnabled));

	// Do nothing if the given point is already selected
	if (IsControlPointSelected(ControlPoint))
	{
		check(ControlPoint->IsSplineSelected());
		return;
	}

	// When switching from segment mode, restore the last used control point mode
	if (SelectedSplineSegments.Num() > 0 && ControlPointWidgetMode.IsSet())
	{
		GLevelEditorModeTools().SetWidgetMode(ControlPointWidgetMode.GetValue());
	}

	// Clear all previous selection
	if (!EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
	{
		ClearSelection();
	}

	Modify(false);
	SelectedSplineControlPoints.Add(ControlPoint);

	SelectNavigationControlPoint(ControlPoint);

	ControlPoint->Modify(false);
	ControlPoint->SetSplineSelected(true);

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows))
	{
		UpdatePropertiesWindows();
	}
}

void ULandscapeSplineSelection::ClearSelectedControlPoints()
{
	for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
	{
		check(ControlPoint->IsSplineSelected());
		ControlPoint->Modify(false);
		ControlPoint->SetSplineSelected(false);
	}
	
	Modify(false);
	SelectedSplineControlPoints.Empty();
}

void ULandscapeSplineSelection::ClearSelectedSegments()
{
	for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
	{
		check(Segment->IsSplineSelected());
		Segment->Modify(false);
		Segment->SetSplineSelected(false);
	}

	Modify(false);
	SelectedSplineSegments.Empty();
}

void ULandscapeSplineSelection::ClearSelection()
{
	ClearSelectedControlPoints();
	ClearSelectedSegments();

	UpdatePropertiesWindows();
}

void ULandscapeSplineSelection::DeselectControlPoint(ULandscapeSplineControlPoint* ControlPoint, ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows));
	check(ControlPoint->IsSplineSelected());

	ControlPoint->Modify(false);
	ControlPoint->SetSplineSelected(false);

	Modify(false);

	// Update the Spline Selection state. Selection was previously disabled if multiple splines/linear paths were selected 
	// After deselecting, update the linear path to see if the selection is now valid
	SelectedSplineControlPoints.Remove(ControlPoint);
	if (!SelectedSplineControlPoints.IsEmpty())
	{
		BuildLinearPathFromLastSelectedPointInternal();
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows))
	{
		UpdatePropertiesWindows();
	}
}

void ULandscapeSplineSelection::DeSelectSegment(ULandscapeSplineSegment* Segment, ESplineNavigationFlags Flags)
{
	check(EnumOnlyContainsFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows));
	check(Segment->IsSplineSelected());

	Segment->Modify(false);
	Segment->SetSplineSelected(false);

	Modify(false);

	// Update the Spline Selection state. Selection was previously disabled if multiple splines/linear paths were selected
	// After deselecting, update the linear path to see if the selection is now valid
	SelectedSplineSegments.Remove(Segment);
	if (!SelectedSplineSegments.IsEmpty())
	{
		BuildLinearPathFromLastSelectedPointInternal();
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::UpdatePropertiesWindows))
	{
		UpdatePropertiesWindows();
	}
}

void ULandscapeSplineSelection::SelectConnected()
{
	TArray<ULandscapeSplineControlPoint*> ControlPointsToProcess = ObjectPtrDecay(SelectedSplineControlPoints);

	while (ControlPointsToProcess.Num() > 0)
	{
		const ULandscapeSplineControlPoint* ControlPoint = ControlPointsToProcess.Pop();

		for (const FLandscapeSplineConnection& Connection : ControlPoint->ConnectedSegments)
		{
			ULandscapeSplineControlPoint* OtherEnd = Connection.GetFarConnection().ControlPoint;

			if (!OtherEnd->IsSplineSelected())
			{
				SelectControlPoint(OtherEnd, ESplineNavigationFlags::AddToSelection);
				ControlPointsToProcess.Add(OtherEnd);
			}
		}
	}

	TArray<ULandscapeSplineSegment*> SegmentsToProcess = ObjectPtrDecay(SelectedSplineSegments);

	while (SegmentsToProcess.Num() > 0)
	{
		const ULandscapeSplineSegment* Segment = SegmentsToProcess.Pop();

		for (const FLandscapeSplineSegmentConnection& SegmentConnection : Segment->Connections)
		{
			for (const FLandscapeSplineConnection& Connection : SegmentConnection.ControlPoint->ConnectedSegments)
			{
				if (Connection.Segment != Segment && !Connection.Segment->IsSplineSelected())
				{
					SelectSegment(Connection.Segment, ESplineNavigationFlags::AddToSelection);
					SegmentsToProcess.Add(Connection.Segment);
				}
			}
		}
	}
}

void ULandscapeSplineSelection::SelectAllSplineSegments(const ULandscapeInfo& InLandscapeInfo)
{
	TArray<TScriptInterface<ILandscapeSplineInterface>> SplineActors(InLandscapeInfo.GetSplineActors());
	for (TScriptInterface<ILandscapeSplineInterface> SplineActor : SplineActors)
	{
		if (ULandscapeSplinesComponent* SplineComponent = SplineActor->GetSplinesComponent())
		{
			SplineComponent->ForEachControlPoint([this](ULandscapeSplineControlPoint* ControlPoint)
				{
					for (const FLandscapeSplineConnection& Connection : ControlPoint->ConnectedSegments)
					{
						SelectSegment(Connection.Segment, ESplineNavigationFlags::AddToSelection);
					}
				});
		}
	}
}

void ULandscapeSplineSelection::SelectAllControlPoints(const ULandscapeInfo& InLandscapeInfo)
{
	TArray<TScriptInterface<ILandscapeSplineInterface>> SplineActors(InLandscapeInfo.GetSplineActors());
	for (TScriptInterface<ILandscapeSplineInterface> SplineActor : SplineActors)
	{
		if (ULandscapeSplinesComponent* SplineComponent = SplineActor->GetSplinesComponent())
		{
			SplineComponent->ForEachControlPoint([this](ULandscapeSplineControlPoint* ControlPoint) { SelectControlPoint(ControlPoint, ESplineNavigationFlags::AddToSelection); });
		}
	}
}

void ULandscapeSplineSelection::SelectAdjacentControlPoints()
{
	for (const ULandscapeSplineSegment* Segment : SelectedSplineSegments)
	{
		SelectControlPoint(Segment->Connections[0].ControlPoint, ESplineNavigationFlags::AddToSelection);
		SelectControlPoint(Segment->Connections[1].ControlPoint, ESplineNavigationFlags::AddToSelection);
	}
}

void ULandscapeSplineSelection::SelectAdjacentSegments()
{
	for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
	{
		for (const FLandscapeSplineConnection& Connection : ControlPoint->ConnectedSegments)
		{
			SelectSegment(Connection.Segment, ESplineNavigationFlags::AddToSelection);
		}
	}
}

const bool ULandscapeSplineSelection::IsSegmentSelected(ULandscapeSplineSegment* Segment) const
{
	return SelectedSplineSegments.Contains(Segment);
}

const bool ULandscapeSplineSelection::IsControlPointSelected(ULandscapeSplineControlPoint* ControlPoint) const
{
	return SelectedSplineControlPoints.Contains(ControlPoint);
}

void ULandscapeSplineSelection::UpdatePropertiesWindows() const
{
	TArray<UObject*> Objects;
	Objects.Reset(SelectedSplineControlPoints.Num() + SelectedSplineSegments.Num());
	Algo::Copy(SelectedSplineControlPoints, Objects);
	Algo::Copy(SelectedSplineSegments, Objects);

	FPropertyEditorModule& PropertyModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	PropertyModule.UpdatePropertyViews(Objects);

	GUnrealEd->RedrawLevelEditingViewports();
}

ULandscapeSplineControlPoint* ULandscapeSplineSelection::GetLinearEndControlPointInternal(ULandscapeSplineControlPoint* SelectedControlPoint)
{
	TSet<ULandscapeSplineControlPoint*> ControlPointsVisited;
	TArray<ULandscapeSplineControlPoint*> ControlPointsToProcess;
	ControlPointsToProcess.Add(SelectedControlPoint);

	Modify(false);
	// Reset the linear path before it gets rebuilt below
	LinearControlPoints.Empty();

	// Control points with zero/multi node connections are not part of a valid linear path
	if (SelectedControlPoint->ConnectedSegments.IsEmpty() || SelectedControlPoint->ConnectedSegments.Num() > 2)
	{
		LinearControlPoints.Add(SelectedControlPoint);
		return SelectedControlPoint;
	}

	while (ControlPointsToProcess.Num() > 0)
	{
		ULandscapeSplineControlPoint* ControlPoint = ControlPointsToProcess.Pop();
		ControlPointsVisited.Add(ControlPoint);

		// Store nodes in order of visiting to create the linear path
		if (!LinearControlPoints.Contains(ControlPoint))
		{
			LinearControlPoints.Add(ControlPoint);
		}

		// Valid end points only have one connection
		if (ControlPoint->ConnectedSegments.Num() == 1 && ControlPoint != SelectedControlPoint)
		{
			return ControlPoint;
		}

		for (const FLandscapeSplineConnection& Connection : ControlPoint->ConnectedSegments)
		{
			ULandscapeSplineControlPoint* OtherEnd = Connection.GetFarConnection().ControlPoint;
			check(OtherEnd != nullptr);

			if (OtherEnd->ConnectedSegments.IsEmpty() || OtherEnd->ConnectedSegments.Num() > 2)
			{
				continue;
			}

			if (!ControlPointsVisited.Contains(OtherEnd))
			{
				ControlPointsToProcess.Add(OtherEnd);
			}
		}
	}

	return SelectedControlPoint;
}

void ULandscapeSplineSelection::ResetNavigationPath()
{
	Modify(false);
	// Clear cached path
	LinearControlPoints.Empty();
	LinearSegments.Empty();
}

void ULandscapeSplineSelection::BuildLinearPathFromLastSelectedPointInternal()
{
	Modify(false);
	// Calculate the last selected point
	// If a control point is selected, use it directly. Otherwise, if a segment is selected, retrieve the control point from its connection
	TObjectPtr<ULandscapeSplineSegment> LastSelectedSegment = SelectedSplineSegments.IsEmpty() ? nullptr : SelectedSplineSegments.Last();
	TObjectPtr<ULandscapeSplineControlPoint> LastSelectedControlPoint = SelectedSplineControlPoints.IsEmpty() && LastSelectedSegment ? LastSelectedSegment->Connections[0].ControlPoint : SelectedSplineControlPoints.Last();

	check(LastSelectedControlPoint != nullptr);

	LinearControlPoints.Empty();
	LinearSegments.Empty();

	ULandscapeSplineControlPoint* StartControlPoint = GetLinearEndControlPointInternal(LastSelectedControlPoint);
	ULandscapeSplineControlPoint* EndControlPoint = GetLinearEndControlPointInternal(StartControlPoint);

	for (const ULandscapeSplineControlPoint* Point : LinearControlPoints)
	{
		for (const FLandscapeSplineConnection& Connection : Point->ConnectedSegments)
		{
			if (!LinearSegments.Contains(Connection.Segment))
			{
				LinearSegments.Add(Connection.Segment);
			}
		}
	}
}

void ULandscapeSplineSelection::SelectNavigationSegment(ULandscapeSplineSegment* Segment)
{
	Modify(false);

	if (!LinearSegments.Contains(Segment))
	{
		BuildLinearPathFromLastSelectedPointInternal();
	}
}

void ULandscapeSplineSelection::SelectNavigationControlPoint(ULandscapeSplineControlPoint* ControlPoint)
{
	Modify(false);

	if (!LinearControlPoints.Contains(ControlPoint))
	{
		BuildLinearPathFromLastSelectedPointInternal();
	}
}

bool ULandscapeSplineSelection::IsSelectionValidForNavigation() const
{
	// If any elements in selection are part of different linear paths, disable navigation
	for (ULandscapeSplineControlPoint* ControlPoint : SelectedSplineControlPoints)
	{
		if (!LinearControlPoints.Contains(ControlPoint))
		{
			return false;
		}
	}

	for (ULandscapeSplineSegment* Segment : SelectedSplineSegments)
	{
		if (!LinearSegments.Contains(Segment))
		{
			return false;
		}
	}

	return true;
}

ULandscapeSplineSegment* ULandscapeSplineSelection::GetEndSegmentInLinearPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::DirectionMask)) == 1);

	if (LinearSegments.IsEmpty())
	{
		return nullptr;
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::DirectionForward))
	{
		return LinearSegments.Top();
	}

	return LinearSegments[0];
}

ULandscapeSplineControlPoint* ULandscapeSplineSelection::GetEndControlPointInLinearPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::ControlPointSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	if (LinearControlPoints.IsEmpty())
	{
		return nullptr;
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::DirectionForward))
	{
		return LinearControlPoints.Top();
	}

	return LinearControlPoints[0];
}

ULandscapeSplineSegment* ULandscapeSplineSelection::GetAdjacentSegmentInLinearPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	const int32 LastSelectedSegmentIndex = LinearSegments.Find(SelectedSplineSegments.Last());
	const int32 NextIndexOffset = EnumHasAllFlags(Flags, ESplineNavigationFlags::DirectionForward) ? 1 : -1;
	check(LastSelectedSegmentIndex != INDEX_NONE); // last selected should always be part of the linear path

	if (LinearSegments.IsValidIndex(LastSelectedSegmentIndex + NextIndexOffset))
	{
		return LinearSegments[LastSelectedSegmentIndex + NextIndexOffset];
	}
	return nullptr;
}

ULandscapeSplineControlPoint* ULandscapeSplineSelection::GetAdjacentControlPointInPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::ControlPointSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	const int32 LastSelectedControlPointIndex = LinearControlPoints.Find(SelectedSplineControlPoints.Last());
	const int32 NextIndexOffset = EnumHasAllFlags(Flags, ESplineNavigationFlags::DirectionForward) ? 1 : -1;
	check(LastSelectedControlPointIndex != INDEX_NONE); // last selected should always be part of the linear path

	if (LinearControlPoints.IsValidIndex(LastSelectedControlPointIndex + NextIndexOffset))
	{
		return LinearControlPoints[LastSelectedControlPointIndex + NextIndexOffset];
	}
	return nullptr;
}

bool ULandscapeSplineSelection::HasAdjacentSegmentInLinearPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::SegmentSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);

	// Early out when the current selection disables navigation/selection (GetAdjacentSegmentInLinearPath reqiures a valid linear path)
	if (!IsSelectionValidForNavigation())
	{
		return false;
	}

	const ULandscapeSplineSegment* AdjacentSegment = GetAdjacentSegmentInLinearPath(Flags);
	if (!AdjacentSegment)
	{
		return false;
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
	{
		return !AdjacentSegment->IsSplineSelected();
	}

	return true;
}

bool ULandscapeSplineSelection::HasAdjacentControlPointInLinearPath(ESplineNavigationFlags Flags) const
{
	check(EnumHasAllFlags(Flags, ESplineNavigationFlags::ControlPointSelectModeEnabled));
	check(FMath::CountBits(static_cast<uint8>(Flags & ESplineNavigationFlags::SelectModeMask)) == 1);
	
	// Early out when the current selection disables navigation/selection (GetAdjacentControlPointInPath reqiures a valid linear path)
	if (!IsSelectionValidForNavigation())
	{
		return false;
	}

	const ULandscapeSplineControlPoint* AdjacentPoint = GetAdjacentControlPointInPath(Flags);
	if (!AdjacentPoint)
	{
		return false;
	}

	if (EnumHasAllFlags(Flags, ESplineNavigationFlags::AddToSelection))
	{
		return !AdjacentPoint->IsSplineSelected();
	}

	return true;
}

#undef LOCTEXT_NAMESPACE
