// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"

#include "LandscapeSplineSelection.generated.h"

class ULandscapeSplineControlPoint;
class ULandscapeSplineSegment;
class ULandscapeInfo;

enum class ESplineNavigationFlags : uint8
{
	None = 0,
	DirectionForward = 1 << 0,
	DirectionBackward = 1 << 1,

	SegmentSelectModeEnabled = 1 << 2,
	ControlPointSelectModeEnabled = 1 << 3,

	AddToSelection = 1 << 4,
	UpdatePropertiesWindows = 1 << 5,

	SelectModeMask = SegmentSelectModeEnabled | ControlPointSelectModeEnabled,
	DirectionMask = DirectionForward | DirectionBackward
};
ENUM_CLASS_FLAGS(ESplineNavigationFlags);

//
// ULandscapeSplineSelection
// 
UCLASS(Transient, MinimalAPI)
class ULandscapeSplineSelection : public UObject
{
	GENERATED_UCLASS_BODY()

	// Spline Selection
	void SelectSegment(ULandscapeSplineSegment* Segment, ESplineNavigationFlags Flags = ESplineNavigationFlags::None);
	void SelectControlPoint(ULandscapeSplineControlPoint* ControlPoint, ESplineNavigationFlags Flags = ESplineNavigationFlags::None);

	void ClearSelectedControlPoints();
	void ClearSelectedSegments();
	void ClearSelection();
	void DeselectControlPoint(ULandscapeSplineControlPoint* ControlPoint, ESplineNavigationFlags Flags);
	void DeSelectSegment(ULandscapeSplineSegment* Segment, ESplineNavigationFlags Flags);

	void SelectConnected();
	void SelectAllSplineSegments(const ULandscapeInfo& InLandscapeInfo);
	void SelectAllControlPoints(const ULandscapeInfo& InLandscapeInfo);
	void SelectAdjacentControlPoints();
	void SelectAdjacentSegments();

	const bool IsSegmentSelected(ULandscapeSplineSegment* Segment) const;
	const bool IsControlPointSelected(ULandscapeSplineControlPoint* ControlPoint) const;

	void UpdatePropertiesWindows() const;

	// Spline Navigation (path traversal based on current selection)
	void ResetNavigationPath();

	void SelectNavigationSegment(ULandscapeSplineSegment* Segment);
	void SelectNavigationControlPoint(ULandscapeSplineControlPoint* ControlPoint);
	bool IsSelectionValidForNavigation() const;

	ULandscapeSplineSegment* GetEndSegmentInLinearPath(ESplineNavigationFlags Flags) const;
	ULandscapeSplineControlPoint* GetEndControlPointInLinearPath(ESplineNavigationFlags Flags) const;
	ULandscapeSplineSegment* GetAdjacentSegmentInLinearPath(ESplineNavigationFlags Flags) const;
	ULandscapeSplineControlPoint* GetAdjacentControlPointInPath(ESplineNavigationFlags Flags) const;

	bool HasAdjacentSegmentInLinearPath(ESplineNavigationFlags Flags) const;
	bool HasAdjacentControlPointInLinearPath(ESplineNavigationFlags Flags) const;

	TArray<ULandscapeSplineControlPoint*> GetSelectedSplineControlPoints() const
	{
		return SelectedSplineControlPoints;
	};

	TArray<ULandscapeSplineSegment*> GetSelectedSplineSegments() const
	{
		return SelectedSplineSegments;
	};

private:
	// Spline navigation
	void BuildLinearPathFromLastSelectedPointInternal();
	ULandscapeSplineControlPoint* GetLinearEndControlPointInternal(ULandscapeSplineControlPoint* SelectedControlPoint);

	UPROPERTY(Transient)
	TArray<TObjectPtr<ULandscapeSplineControlPoint>> SelectedSplineControlPoints;
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULandscapeSplineSegment>> SelectedSplineSegments;

	// Linear representation of the current spline based on the last selected point/segment
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULandscapeSplineControlPoint>> LinearControlPoints;
	UPROPERTY(Transient)
	TArray<TObjectPtr<ULandscapeSplineSegment>> LinearSegments;

	// Persistent control point mode
	TOptional<UE::Widget::EWidgetMode> ControlPointWidgetMode;
};
