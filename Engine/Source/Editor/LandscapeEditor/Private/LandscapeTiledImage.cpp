// Copyright Epic Games, Inc. All Rights Reserved.

#include "LandscapeTiledImage.h"

#include "HAL/FileManager.h"
#include "Internationalization/Regex.h"
#include "LandscapeImageFileCache.h"

#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"


TMap<FString, FString> FLandscapeTiledImage::Tokens = {
{ TEXT("u"), TEXT("<u>") },
{ TEXT("v"), TEXT("<v>") },
{ TEXT("x"), TEXT("<x>") },
{ TEXT("y"), TEXT("<y>") }
};

FString FLandscapeTiledImage::GetTokenRegex(const FString& Prefix)
{
	return Prefix + TEXT("(-?[0-9]+)");
}


FLandscapeTiledImage::FLandscapeTiledImage()
{
}

template <typename T>
FLandscapeFileInfo FLandscapeTiledImage::Load(const TCHAR* Filename)
{
	FLandscapeFileInfo Result;

	TArray<FString> FoundFiles;
	FindFiles(Filename, FoundFiles);

	FString Path = FPaths::GetPath(Filename);
	FString FilenameNoPath = FPaths::GetCleanFilename(Filename);
	FString RegexFilename(FilenameNoPath);
	TMap<int32, FString> Groups;

	for (const TPair<FString, FString>& It : Tokens)
	{
		if (int32 StartIndex = RegexFilename.Find(It.Value); StartIndex != INDEX_NONE)
		{
			Groups.Add(StartIndex,  It.Value);
		}
	}

	Groups.KeySort([](int32 A, int32 B) { return A < B; });

	for (const TPair<FString, FString>& It : Tokens)
	{
		if (int32 start = RegexFilename.Find(It.Value); start != INDEX_NONE)
		{
			RegexFilename.RemoveAt(start, It.Value.Len());
			RegexFilename.InsertAt(start, GetTokenRegex(""));
		}
	}

	const bool bExactFilename = RegexFilename == FilenameNoPath;

	SizeInTiles = FIntPoint::NoneValue;
	TileResolution = FIntPoint::NoneValue;
	
	if (FoundFiles.Num() == 1 && bExactFilename)
	{
		FIntPoint TileCoord(0, 0);
		TileFilenames.Add(TileCoord, FString(Filename));
	}
	else
	{
		for (const FString& FoundFilname : FoundFiles)
		{
			FRegexPattern Pattern(RegexFilename);
			// Regex test the filename without the path so that conflicting symbols (e.g. brackets) in the path don't confuse it.
			FRegexMatcher Matcher(Pattern, FoundFilname);

			if (Matcher.FindNext())
			{
				int32 X = -1;
				int32 Y = -1;

				int32 Group = 1;
				for (const TPair<int32, FString> &it : Groups)
				{
					if (it.Value == "<u>")
					{
						X = FCString::Atoi(*Matcher.GetCaptureGroup(Group++)) - 1;
					}
					else if (it.Value == "<x>")
					{
						X = FCString::Atoi(*Matcher.GetCaptureGroup(Group++));
					}
					else if (it.Value == "<v>")
					{
						Y = FCString::Atoi(*Matcher.GetCaptureGroup(Group++)) - 1;
					}
					else if (it.Value == "<y>")
					{
						Y = FCString::Atoi(*Matcher.GetCaptureGroup(Group++));
					}
				}

				if (X >= 0 && Y >= 0)
				{
					FIntPoint TileCoord(X, Y);
					FString FullPath = FPaths::Combine(Path, FoundFilname);
					TileFilenames.Add(TileCoord, FullPath);
				}			
			}
		}
	}
	
	for (const TPair<FIntPoint, FString> & Tile : TileFilenames)
	{
		FLandscapeImageDataRef ImageData;
		FLandscapeImageFileCache& LandscapeImageFileCache = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor").GetImageFileCache();
		FLandscapeFileInfo TileResult = LandscapeImageFileCache.FindImage<T>(*Tile.Value, ImageData);
		if (TileResult.ResultCode == ELandscapeImportResult::Error)
		{
			return TileResult;
		}
		else if (TileResult.ResultCode == ELandscapeImportResult::Warning)
		{
			Result.ResultCode = TileResult.ResultCode;
			Result.ErrorMessage = TileResult.ErrorMessage;
		}

		SizeInTiles.X = FMath::Max(SizeInTiles.X, Tile.Key.X + 1);
		SizeInTiles.Y = FMath::Max(SizeInTiles.Y, Tile.Key.Y + 1);

		uint32 Width = ImageData.Resolution.X;
		uint32 Height = ImageData.Resolution.Y;

		if (TileResolution == FIntPoint::NoneValue)
		{
			TileResolution.X = Width;
			TileResolution.Y = Height;
		}
		else if (Width != TileResolution.X && Height != TileResolution.Y)
		{
			Result.ResultCode = ELandscapeImportResult::Error;
			Result.ErrorMessage = LOCTEXT("FileReadErrorTiledResolutionMismatch", "Mismatched resolution found in tiled image");
			return Result;
		}
	}

	if (TileFilenames.Num() == 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("FileReadErrorNoFilesFound", "No files found");
		return Result;
	}

	// Check for int overflows due to too large SizeInTiles values from the filenames
	if (SizeInTiles.X > std::numeric_limits<int32>::max() / TileResolution.X ||
		SizeInTiles.Y > std::numeric_limits<int32>::max() / TileResolution.Y ||
		SizeInTiles.X <= 0 || SizeInTiles.Y <= 0)
	{
		Result.ResultCode = ELandscapeImportResult::Error;
		Result.ErrorMessage = LOCTEXT("FileReadErrorTileCoordsInvalid", "Invalid tiled image coordinates");
		return Result;
	}

	Result.PossibleResolutions.Add(FLandscapeFileResolution(GetResolution().X, GetResolution().Y));

	return Result;
}

template FLandscapeFileInfo FLandscapeTiledImage::Load<uint8>(const TCHAR* Filename);
template FLandscapeFileInfo FLandscapeTiledImage::Load<uint16>(const TCHAR* Filename);


void FLandscapeTiledImage::FindFiles(const TCHAR* FilenamePattern, TArray<FString>& OutPaths)
{
	FString GlobFilename(FilenamePattern);

	for (const TPair<FString, FString>& It : Tokens)
	{
		GlobFilename = GlobFilename.Replace(*It.Value, TEXT("*"));
	}
	
	OutPaths.Empty();
	IFileManager::Get().FindFiles(OutPaths, *GlobFilename, true, false);
}

bool FLandscapeTiledImage::CheckTiledNamePath(const FString& Filename, FString& OutTiledFilenamePattern)
{
	const FString Extension = FPaths::GetExtension(Filename);
	const FString Root = FPaths::GetPath(Filename);
	const FString BaseFilename = FPaths::GetBaseFilename(Filename);

	bool HasMatch = true;
	FString CurrentFilename = BaseFilename;
	int32 TokenIndex = 0;

	for (const TPair<FString, FString>& It : Tokens)
	{
		while (true)
		{
			FRegexMatcher Matcher(FRegexPattern(GetTokenRegex(It.Key)), CurrentFilename);
			if (!Matcher.FindNext())
			{
				break;
			}

			int32 MatchBegin = Matcher.GetMatchBeginning();
			int32 MatchEnd = Matcher.GetMatchEnding();

			CurrentFilename.RemoveAt(MatchBegin, MatchEnd - MatchBegin);
			CurrentFilename.InsertAt(MatchBegin, It.Key + It.Value);
		}
	}

	const FString PatternFilename = CurrentFilename + FString(".") + Extension;
	FString TiledFilenamePattern = FPaths::Combine(Root, PatternFilename);

	TArray<FString> FoundFiles;
	FLandscapeTiledImage::FindFiles(*TiledFilenamePattern, FoundFiles);
	const bool bFoundTiledFiles = FoundFiles.Num() > 0 && TiledFilenamePattern != Filename;

	if (bFoundTiledFiles )
	{
		OutTiledFilenamePattern = TiledFilenamePattern;
	}
	return bFoundTiledFiles;
}

#undef LOCTEXT_NAMESPACE