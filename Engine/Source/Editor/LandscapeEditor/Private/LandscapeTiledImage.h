// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "LandscapeImageFileCache.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeDataAccess.h"

template<typename T>
class LandscapeImageTraits
{
public:
	static T DefaultValue() { return 0; }
};

template<>
class LandscapeImageTraits<uint16>
{
public:
	static uint16 DefaultValue() { return static_cast<uint16>(LandscapeDataAccess::MidValue); }
};

class FLandscapeTiledImage
{
public:
	FLandscapeTiledImage();

	template<typename T>
	FLandscapeFileInfo Load(const TCHAR* Filename);
	
	FIntPoint GetTileResolution() const { return TileResolution;  }
	FIntPoint GetSizeInTiles() const { return SizeInTiles; }

	FIntPoint GetResolution() const {
		return FIntPoint(TileResolution.X * SizeInTiles.X, TileResolution.Y * SizeInTiles.Y); }

	static void FindFiles(const TCHAR* FilenamePattern, TArray<FString>& OutPaths);

	template<typename T>
	void Read(TArray<T>& OutImageData, bool bFlipYAxis = false) const
	{
		const FIntPoint Resolution = GetResolution();
		ReadRegion(FIntRect(0, 0, Resolution.X, Resolution.Y), OutImageData, bFlipYAxis);
	}

	static bool CheckTiledNamePath(const FString& Filename, FString& OutTiledFilenamePattern);

	template<typename T>
	FLandscapeFileInfo ReadRegion(const FIntRect& Region, TArray<T>& OutImageData, bool bFlipYAxis = false, T DefaultValue = LandscapeImageTraits<T>::DefaultValue()) const
	{
		OutImageData.Init(DefaultValue, Region.Height() * Region.Width());
		
		int32 MinTileX = Region.Min.X / TileResolution.X;
		int32 MinTileY = Region.Min.Y / TileResolution.Y;

		int32 MaxTileX = Region.Max.X / TileResolution.X;
		int32 MaxTileY = Region.Max.Y / TileResolution.Y;
		
		for (int32 TileY = MinTileY; TileY <= MaxTileY; ++TileY)
		{
			for (int32 TileX = MinTileX; TileX <= MaxTileX; ++TileX)
			{
				const FIntPoint TileMin(TileX * TileResolution.X, TileY * TileResolution.Y);

				const FString* TileFilename = TileFilenames.Find(FIntPoint(TileX, TileY));			
				if (TileFilename)
				{
					FLandscapeImageDataRef ImageData;
					FLandscapeImageFileCache& LandscapeImageFileCache = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor").GetImageFileCache();
					FLandscapeFileInfo TileResult = LandscapeImageFileCache.FindImage<T>(**TileFilename, ImageData);

					if (TileResult.ResultCode == ELandscapeImportResult::Error)
					{
						return TileResult;
					}

					const uint8* TileData = ImageData.Data->GetData();

					FIntRect TileRegion = Region;
					TileRegion -= TileMin;
					TileRegion.Clip(FIntRect(0, 0, TileResolution.X, TileResolution.Y));

					for (int32 Y = 0; Y < TileRegion.Height(); ++Y)
					{
						for (int32 X = 0; X < TileRegion.Width(); ++X)
						{
							const FIntPoint TileCoord = FIntPoint(X, Y) + TileRegion.Min;
							const FIntPoint OutCoord = TileCoord + TileMin - Region.Min;
							
							const int32 SrcY = bFlipYAxis ? TileResolution.Y - TileCoord.Y - 1 : TileCoord.Y;

							T& Dest = OutImageData[OutCoord.Y * Region.Width() + OutCoord.X];
							const uint8* Src = &TileData[ImageData.BytesPerPixel * (SrcY * TileResolution.X + TileCoord.X)];

							if (sizeof(T) == ImageData.BytesPerPixel)
							{
								Dest = *(reinterpret_cast<const T*>(Src));
							}
							else if (ImageData.BytesPerPixel == 2 && sizeof(T) == 1)
							{
								Dest = *(reinterpret_cast<const uint16*>(Src)) >> 8;
							}
							else
							{
								Dest = *Src;
							}
						}
					}
				}
			}
		}

		return FLandscapeFileInfo();
	}

private:

	static TMap<FString, FString> Tokens;
	static FString GetTokenRegex(const FString& Prefix);

	TMap<FIntPoint, FString> TileFilenames;

	FIntPoint TileResolution = FIntPoint::NoneValue;
	FIntPoint SizeInTiles = FIntPoint::NoneValue;
};


