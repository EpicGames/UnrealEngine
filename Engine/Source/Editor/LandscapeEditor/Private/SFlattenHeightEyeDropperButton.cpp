// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFlattenHeightEyeDropperButton.h"
#include "Widgets/SOverlay.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "LandscapeEdMode.h"
#include "EditorModeManager.h"
#include "EditorModes.h"

#define LOCTEXT_NAMESPACE "FlattenHeightEyeDropperButton"

void SFlattenHeightEyeDropperButton::Construct(const FArguments& InArgs)
{
	OnBegin = InArgs._OnBegin;
	OnComplete = InArgs._OnComplete;
	bWasClickActivated = false;
	bIsOverButton = false;

	// This is a button containing a dropper image and text that tells the user to hit Esc.
	// Their visibility are changed according to whether dropper mode is active or not.
	SButton::Construct(
		SButton::FArguments()
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.ContentPadding(1.0f)
		.OnClicked(this, &SFlattenHeightEyeDropperButton::OnClicked)
		.OnHovered(this, &SFlattenHeightEyeDropperButton::OnMouseHovered)
		.OnUnhovered(this, &SFlattenHeightEyeDropperButton::OnMouseUnhovered)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			.Padding(FMargin(1.0f, 0.0f))
			[
				SNew(SImage)
				.Image(FCoreStyle::Get().GetBrush("ColorPicker.EyeDropper"))
				.ToolTipText(LOCTEXT("EyeDropperButton_ToolTip", "Activates the eye dropper for selecting a landscape height."))
			]

			+ SOverlay::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("EscapeCue", "Esc"))
				.ToolTipText(LOCTEXT("EyeDropperEscapeCue_ToolTip", "Hit Escape key to stop the eye dropper"))
				.Visibility(this, &SFlattenHeightEyeDropperButton::GetEscapeTextVisibility)
			]
		]
	);

	FSlateApplication::Get().OnApplicationPreInputKeyDownListener().AddRaw(this, &SFlattenHeightEyeDropperButton::OnApplicationPreInputKeyDownListener);
	FSlateApplication::Get().OnApplicationMousePreInputButtonDownListener().AddRaw(this, &SFlattenHeightEyeDropperButton::OnApplicationMousePreInputButtonDownListener);
}

SFlattenHeightEyeDropperButton::~SFlattenHeightEyeDropperButton()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnApplicationPreInputKeyDownListener().RemoveAll(this);
		FSlateApplication::Get().OnApplicationMousePreInputButtonDownListener().RemoveAll(this);
	}
}

void SFlattenHeightEyeDropperButton::OnApplicationPreInputKeyDownListener(const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.GetKey() == EKeys::Escape && bWasClickActivated)
	{
		bWasClickActivated = false;

		// This is needed to switch the dropper cursor off immediately so the user can see the Esc key worked
		FSlateApplication::Get().QueryCursor();

		bool bCancelled = true;
		OnComplete.ExecuteIfBound(bCancelled);
	}
}

void SFlattenHeightEyeDropperButton::OnApplicationMousePreInputButtonDownListener(const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && bWasClickActivated && !bIsOverButton)
	{
		bWasClickActivated = false;

		// This is needed to switch the dropper cursor off immediately so the user can see the Esc key worked
		FSlateApplication::Get().QueryCursor();

		bool bCancelled = false;
		OnComplete.ExecuteIfBound(bCancelled);
	}
}

FReply SFlattenHeightEyeDropperButton::OnClicked()
{
	if (bWasClickActivated)
	{
		bWasClickActivated = false;
		const bool bCancelled = false;
		OnComplete.ExecuteIfBound(bCancelled);
	}
	else
	{
		bWasClickActivated = true;
		OnBegin.ExecuteIfBound();
	}
	
	return FReply::Handled();
}

void SFlattenHeightEyeDropperButton::OnMouseHovered()
{
	bIsOverButton = true;
}

void SFlattenHeightEyeDropperButton::OnMouseUnhovered()
{
	bIsOverButton = false;
}

EVisibility SFlattenHeightEyeDropperButton::GetEscapeTextVisibility() const
{
	// Show the Esc key message in the button when dropper mode is active
	if (bWasClickActivated)
	{
		return EVisibility::Visible;
	}
	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
