// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Input/CursorReply.h"
#include "Input/Reply.h"
#include "Layout/Visibility.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"

DECLARE_DELEGATE_OneParam(FOnDropperComplete, bool)

class SFlattenHeightEyeDropperButton : public SButton
{
public:
	virtual ~SFlattenHeightEyeDropperButton();

	SLATE_BEGIN_ARGS(SFlattenHeightEyeDropperButton)
		: _OnBegin()
		, _OnComplete()
		{}

		/** Invoked when the dropper goes from inactive to active */
		SLATE_EVENT(FSimpleDelegate, OnBegin)

		/** Invoked when the dropper goes from active to inactive */
		SLATE_EVENT(FOnDropperComplete, OnComplete)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	FReply OnClicked();
	void OnMouseHovered();
	void OnMouseUnhovered();

	EVisibility GetEscapeTextVisibility() const;

	void OnApplicationPreInputKeyDownListener(const FKeyEvent& InKeyEvent);
	void OnApplicationMousePreInputButtonDownListener(const FPointerEvent& MouseEvent);

private:

	/** Invoked when the dropper goes from inactive to active */
	FSimpleDelegate OnBegin;

	/** Invoked when the dropper goes from active to inactive - can be used to commit colors by the owning picker */
	FOnDropperComplete OnComplete;

	bool bWasClickActivated;
	bool bIsOverButton;
};
