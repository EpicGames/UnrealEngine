// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLandscapeEditor.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Materials/Material.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Styling/AppStyle.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LandscapeEditTypes.h"
#include "LandscapeEditorCommands.h"
#include "LandscapeEditorObject.h"
#include "IDetailsView.h"
#include "PropertyEditorModule.h"
#include "LandscapeSettings.h"
#include "HAL/IConsoleManager.h"
#include "Toolkits/AssetEditorModeUILayer.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"

void SLandscapeAssetThumbnail::Construct(const FArguments& InArgs, UObject* Asset, TSharedRef<FAssetThumbnailPool> ThumbnailPool, const FName& ClassThumbnailBrushOverride)
{
	FIntPoint ThumbnailSize = InArgs._ThumbnailSize;

	AssetThumbnail = MakeShareable(new FAssetThumbnail(Asset, ThumbnailSize.X, ThumbnailSize.Y, ThumbnailPool));
	OnAccessAsset = InArgs._OnAccessAsset;

	FAssetThumbnailConfig AssetThumbnailConfig;
	AssetThumbnailConfig.ShowAssetColor = false;

	// If the asset is null, then we purposefully don't want this layer to have a thumbnail. Display a generic icon in that case :
	if (Asset == nullptr)
	{
		AssetThumbnailConfig.bForceGenericThumbnail = true;
		AssetThumbnailConfig.ClassThumbnailBrushOverride = ClassThumbnailBrushOverride;
		AssetThumbnailConfig.bAllowHintText = false;
		AssetThumbnailConfig.AlwaysExpandTooltip = false;
		AssetThumbnailConfig.AllowAssetStatusThumbnailOverlay = false;
		AssetThumbnailConfig.ThumbnailLabel = EThumbnailLabel::Type::NoLabel;
	}

	TSharedRef<SWidget> AssetThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(AssetThumbnailConfig);

	// Disable the asset tooltip altogether, the calling code will add its own if needed
	AssetThumbnailWidget->SetToolTip(TSharedPtr<IToolTip>());

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(static_cast<float>(ThumbnailSize.X))
		.HeightOverride(static_cast<float>(ThumbnailSize.Y))
		[
			AssetThumbnailWidget
		]
	];

	if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(Asset))
	{
		UMaterial::OnMaterialCompilationFinished().AddSP(this, &SLandscapeAssetThumbnail::OnMaterialCompilationFinished);
	}
}

SLandscapeAssetThumbnail::~SLandscapeAssetThumbnail()
{
	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
}

void SLandscapeAssetThumbnail::OnMaterialCompilationFinished(UMaterialInterface* MaterialInterface)
{
	UMaterialInterface* MaterialAsset = Cast<UMaterialInterface>(AssetThumbnail->GetAsset());
	if (MaterialAsset)
	{
		if (MaterialAsset->IsDependent(MaterialInterface))
		{
			// Refresh thumbnail
			AssetThumbnail->SetAsset(AssetThumbnail->GetAsset());
		}
	}
}

void SLandscapeAssetThumbnail::SetAsset(UObject* Asset)
{
	AssetThumbnail->SetAsset(Asset);
}

FReply SLandscapeAssetThumbnail::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMyGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition()))
	{
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton && OnAccessAsset.IsBound())
		{
			if (OnAccessAsset.Execute(AssetThumbnail->GetAsset()))
			{
				return FReply::Handled();
			}
		}
	}
	return FReply::Unhandled();
}

//////////////////////////////////////////////////////////////////////////



void FLandscapeToolKit::Init(const TSharedPtr<IToolkitHost>& InitToolkitHost)
{
	auto NameToCommandMap = FLandscapeEditorCommands::Get().NameToCommandMap;

	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	TSharedRef<FUICommandList> CommandList = LandscapeEdMode->GetUICommandList();

#define MAP_MODE(ModeName) CommandList->MapAction(NameToCommandMap.FindChecked(ModeName), FUIAction(FExecuteAction::CreateSP(this, &FLandscapeToolKit::OnChangeMode, FName(ModeName)), FCanExecuteAction::CreateSP(this, &FLandscapeToolKit::IsModeEnabled, FName(ModeName)), FIsActionChecked::CreateSP(this, &FLandscapeToolKit::IsModeActive, FName(ModeName))));
	MAP_MODE("ToolMode_Manage");
	MAP_MODE("ToolMode_Sculpt");
	MAP_MODE("ToolMode_Paint");
#undef MAP_MODE

#define MAP_TOOL(ToolName) CommandList->MapAction(NameToCommandMap.FindChecked("Tool_" ToolName), FUIAction(FExecuteAction::CreateSP(this, &FLandscapeToolKit::OnChangeTool, FName(ToolName)), FCanExecuteAction::CreateSP(this, &FLandscapeToolKit::IsToolEnabled, FName(ToolName)), FIsActionChecked::CreateSP(this, &FLandscapeToolKit::IsToolActive, FName(ToolName))));
	MAP_TOOL("NewLandscape");
	MAP_TOOL("ResizeLandscape");
	MAP_TOOL("ImportExport");

	MAP_TOOL("Sculpt");
	MAP_TOOL("Erase");
	MAP_TOOL("Paint");
	MAP_TOOL("Smooth");
	MAP_TOOL("Flatten");
	MAP_TOOL("Ramp");
	MAP_TOOL("Erosion");
	MAP_TOOL("HydraErosion");
	MAP_TOOL("Noise");
	MAP_TOOL("Visibility");
	MAP_TOOL("BlueprintBrush");

	MAP_TOOL("Select");
	MAP_TOOL("AddComponent");
	MAP_TOOL("DeleteComponent");
	MAP_TOOL("MoveToLevel");

	MAP_TOOL("Mask");
	MAP_TOOL("CopyPaste");
	MAP_TOOL("Mirror");

	MAP_TOOL("Splines");
#undef MAP_TOOL

#define MAP_BRUSH_SET(BrushSetName) CommandList->MapAction(NameToCommandMap.FindChecked(BrushSetName), FUIAction(FExecuteAction::CreateSP(this, &FLandscapeToolKit::OnChangeBrushSet, FName(BrushSetName)), FCanExecuteAction::CreateSP(this, &FLandscapeToolKit::IsBrushSetEnabled, FName(BrushSetName)), FIsActionChecked::CreateSP(this, &FLandscapeToolKit::IsBrushSetActive, FName(BrushSetName))));
	MAP_BRUSH_SET("BrushSet_Circle");
	MAP_BRUSH_SET("BrushSet_Alpha");
	MAP_BRUSH_SET("BrushSet_Pattern");
	MAP_BRUSH_SET("BrushSet_Component");
	MAP_BRUSH_SET("BrushSet_Gizmo");
#undef MAP_BRUSH_SET

#define MAP_BRUSH(BrushName) CommandList->MapAction(NameToCommandMap.FindChecked(BrushName), FUIAction(FExecuteAction::CreateSP(this, &FLandscapeToolKit::OnChangeBrush, FName(BrushName)), FCanExecuteAction(), FIsActionChecked::CreateSP(this, &FLandscapeToolKit::IsBrushActive, FName(BrushName))));
	MAP_BRUSH("Circle_Smooth");
	MAP_BRUSH("Circle_Linear");
	MAP_BRUSH("Circle_Spherical");
	MAP_BRUSH("Circle_Tip");
#undef MAP_BRUSH

	LandscapeEditorWidgets = SNew(SLandscapeEditor, SharedThis(this));
	BrushesWidgets = StaticCastSharedRef<FLandscapeEditorDetails>(FLandscapeEditorDetails::MakeInstance());

	FModeToolkit::Init(InitToolkitHost);
}

FName FLandscapeToolKit::GetToolkitFName() const
{
	return FName("LandscapeEditor");
}

FText FLandscapeToolKit::GetBaseToolkitName() const
{
	return LOCTEXT("ToolkitName", "Landscape");
}

FEdModeLandscape* FLandscapeToolKit::GetEditorMode() const
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

TSharedPtr<SWidget> FLandscapeToolKit::GetInlineContent() const
{
	return LandscapeEditorWidgets;
}

const TArray<FName> FLandscapeToolKit::PaletteNames = { LandscapeEditorNames::Manage, LandscapeEditorNames::Sculpt, LandscapeEditorNames::Paint };

void FLandscapeToolKit::GetToolPaletteNames(TArray<FName>& InPaletteName) const
{
	InPaletteName = PaletteNames;
}

FText FLandscapeToolKit::GetToolPaletteDisplayName(FName PaletteName) const
{

	if (PaletteName == LandscapeEditorNames::Manage)
	{
		return LOCTEXT("Mode.Manage", "Manage");
	}
	else if (PaletteName == LandscapeEditorNames::Sculpt)
	{
		return LOCTEXT("Mode.Sculpt", "Sculpt");
	}
	else if (PaletteName == LandscapeEditorNames::Paint)
	{
		return LOCTEXT("Mode.Paint",  "Paint");
	}

	return FText();
}

void FLandscapeToolKit::BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolBarBuilder)
{
	auto Commands = FLandscapeEditorCommands::Get();
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	const ULandscapeSettings* Settings = GetDefault<ULandscapeSettings>();

	if (PaletteName == LandscapeEditorNames::Manage)
	{
		ToolBarBuilder.BeginSection("Manage");

		ToolBarBuilder.AddToolBarButton(Commands.NewLandscape);
		ToolBarBuilder.AddToolBarButton(Commands.ImportExportTool);
		ToolBarBuilder.AddToolBarButton(Commands.SelectComponentTool);
		ToolBarBuilder.AddToolBarButton(Commands.AddComponentTool);
		ToolBarBuilder.AddToolBarButton(Commands.DeleteComponentTool);
		ToolBarBuilder.AddToolBarButton(Commands.MoveToLevelTool);
		ToolBarBuilder.AddToolBarButton(Commands.ResizeLandscape);
		ToolBarBuilder.AddToolBarButton(Commands.SplineTool);
		if (Settings->AreBlueprintToolsAllowed())
		{
			ToolBarBuilder.AddToolBarButton(Commands.BlueprintBrushTool);
		}
	}

	else if (PaletteName == LandscapeEditorNames::Sculpt)
	{

		ToolBarBuilder.AddToolBarButton(Commands.SculptTool);
		ToolBarBuilder.AddToolBarButton(Commands.EraseTool);
		ToolBarBuilder.AddToolBarButton(Commands.SmoothTool);
		ToolBarBuilder.AddToolBarButton(Commands.FlattenTool);
		ToolBarBuilder.AddToolBarButton(Commands.RampTool);
		ToolBarBuilder.AddToolBarButton(Commands.ErosionTool);
		ToolBarBuilder.AddToolBarButton(Commands.HydroErosionTool);
		ToolBarBuilder.AddToolBarButton(Commands.NoiseTool);
		ToolBarBuilder.AddToolBarButton(Commands.VisibilityTool);

		ToolBarBuilder.AddToolBarButton(Commands.MirrorTool);
		ToolBarBuilder.AddToolBarButton(Commands.RegionCopyPasteTool);

		ToolBarBuilder.AddToolBarButton(Commands.RegionSelectTool);
	}

	else if (PaletteName == LandscapeEditorNames::Paint)
	{
		ToolBarBuilder.AddToolBarButton(Commands.PaintTool);
		ToolBarBuilder.AddToolBarButton(Commands.SmoothTool);
		ToolBarBuilder.AddToolBarButton(Commands.FlattenTool);
		ToolBarBuilder.AddToolBarButton(Commands.NoiseTool);
		ToolBarBuilder.AddToolBarButton(Commands.RegionSelectTool);
	}

}

void FLandscapeToolKit::OnToolPaletteChanged(FName PaletteName)
{
	if (PaletteName == LandscapeEditorNames::Manage && !IsModeActive(LandscapeEditorNames::Manage ))
	{
		OnChangeMode(LandscapeEditorNames::Manage);
	}
	else if (PaletteName == LandscapeEditorNames::Sculpt && !IsModeActive(LandscapeEditorNames::Sculpt ))
	{
		OnChangeMode(LandscapeEditorNames::Sculpt); 
	}
	else if (PaletteName == LandscapeEditorNames::Paint  && !IsModeActive(LandscapeEditorNames::Paint ))
	{
		OnChangeMode(LandscapeEditorNames::Paint); 
	}
}

FText FLandscapeToolKit::GetActiveToolDisplayName() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		return LandscapeEdMode->CurrentTool->GetDisplayName();
	}
	return FText::GetEmpty();
}

FText FLandscapeToolKit::GetActiveToolMessage() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentTool)
	{
		return LandscapeEdMode->CurrentTool->GetDisplayMessage();
	}
	return FText::GetEmpty();
}

void FLandscapeToolKit::NotifyToolChanged()
{
	LandscapeEditorWidgets->NotifyToolChanged();
}

void FLandscapeToolKit::NotifyBrushChanged()
{
	LandscapeEditorWidgets->NotifyBrushChanged();
}

void FLandscapeToolKit::RefreshDetailPanel()
{
	LandscapeEditorWidgets->RefreshDetailPanel();
}

void FLandscapeToolKit::RefreshInspectedObjectsDetailPanel()
{
	// Focus or re-open the inspected objects details view
	if (TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		TSharedPtr<FTabManager> TabManagerPtr = ModeUILayerPtr->GetTabManager();
		if (!TabManagerPtr)
		{
			return;
		}
		InspectedObjectsTab = TabManagerPtr->TryInvokeTab(UAssetEditorUISubsystem::BottomLeftTabID);

		// It's possible we fail to create a tab (e.g. permission issue)
		if (InspectedObjectsTab.IsValid())
		{
			// If we managed to create a tab, the callback should have assigned a InspectedObjectsDetailsView, though 
			check(InspectedObjectsDetailsView != nullptr);
			// Refresh the selected objects
			InspectedObjectsDetailsView->RefreshDetailPanel();
		}
	}
}

void FLandscapeToolKit::OnChangeMode(FName ModeName)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		LandscapeEdMode->SetCurrentToolMode(ModeName);
	}
}

bool FLandscapeToolKit::IsModeEnabled(FName ModeName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		// Manage is the only mode enabled if we have no landscape
		if (ModeName == LandscapeEditorNames::Manage || (LandscapeEdMode->GetLandscapeList().Num() > 0 && LandscapeEdMode->CanEditCurrentTarget()))
		{
			return true;
		}
	}

	return false;
}

bool FLandscapeToolKit::IsModeActive(FName ModeName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentToolMode)
	{
		return LandscapeEdMode->CurrentToolMode->ToolModeName == ModeName;
	}

	return false;
}

void FLandscapeToolKit::OnChangeTool(FName ToolName)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->SetCurrentTool(ToolName);
	}
}

bool FLandscapeToolKit::IsToolEnabled(FName ToolName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		// When using World Partition:
		// MoveToLevel isn't supported because we don't support Proxies in different levels.
		// Resize isn't supported and instead should be done via a user provided Commandlet.
		if (LandscapeEdMode->IsGridBased() &&
			(ToolName == "MoveToLevel" || ToolName == "ResizeLandscape"))
		{
			return false;
		}

		// NewLandscape is always available.
		if (ToolName == "NewLandscape")
		{
			return true;
		}

		// Other tools are available if there is an existing landscape.
		if (LandscapeEdMode->GetLandscapeList().Num() > 0)
		{
			return true;
		}
	}

	return false;
}

bool FLandscapeToolKit::IsToolActive(FName ToolName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr && LandscapeEdMode->CurrentTool != nullptr)
	{
		const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();
		return CurrentToolName == ToolName;
	}

	return false;
}

void FLandscapeToolKit::OnChangeBrushSet(FName BrushSetName)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->SetCurrentBrushSet(BrushSetName);
	}
}

bool FLandscapeToolKit::IsBrushSetEnabled(FName BrushSetName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode 
		&& LandscapeEdMode->IsEditingEnabled()
		&& LandscapeEdMode->CurrentTool )
	{
		return LandscapeEdMode->CurrentTool->ValidBrushes.Contains(BrushSetName);
	}

	return false;
}

bool FLandscapeToolKit::IsBrushSetActive(FName BrushSetName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->LandscapeBrushSets.IsValidIndex(LandscapeEdMode->CurrentBrushSetIndex))
	{
		const FName CurrentBrushSetName = LandscapeEdMode->LandscapeBrushSets[LandscapeEdMode->CurrentBrushSetIndex].BrushSetName;
		return CurrentBrushSetName == BrushSetName;
	}

	return false;
}

void FLandscapeToolKit::OnChangeBrush(FName BrushName)
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode != nullptr)
	{
		LandscapeEdMode->SetCurrentBrush(BrushName);
	}
}

bool FLandscapeToolKit::IsBrushActive(FName BrushName) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode && LandscapeEdMode->CurrentBrush)
	{
		const FName CurrentBrushName = LandscapeEdMode->CurrentBrush->GetBrushName();
		return CurrentBrushName == BrushName;
	}

	return false;
}

void FLandscapeToolKit::RequestModeUITabs()
{
	FModeToolkit::RequestModeUITabs();
	if (TSharedPtr<FAssetEditorModeUILayer> ModeUILayerPtr = ModeUILayer.Pin())
	{
		TSharedPtr<FWorkspaceItem> MenuModeCategoryPtr = ModeUILayerPtr->GetModeMenuCategory();

		if (!MenuModeCategoryPtr)
		{
			return;
		}

		InspectedObjectsTabInfo.OnSpawnTab = FOnSpawnTab::CreateSP(SharedThis(this), &FLandscapeToolKit::CreateInspectedObjectsDetailsViewTab);
		InspectedObjectsTabInfo.TabLabel = LOCTEXT("LandscapeModeDetailsViewToolboxTabLabel", "Landscape Details");
		InspectedObjectsTabInfo.TabTooltip = LOCTEXT("LandscapeModeDetailsViewToolboxTabTooltipText", "Open the Landscape Details tab, which contains the landscape editor mode's inspected object details.");
		InspectedObjectsTabInfo.TabIcon = GetEditorModeIcon();
		ModeUILayerPtr->SetModePanelInfo(UAssetEditorUISubsystem::BottomLeftTabID, InspectedObjectsTabInfo);
	}
}

void FLandscapeToolKit::InvokeUI()
{
	FModeToolkit::InvokeUI();
}

TSharedRef<SDockTab> FLandscapeToolKit::CreateInspectedObjectsDetailsViewTab(const FSpawnTabArgs& Args)
{
	TSharedPtr<SDockTab> CreatedTab = SNew(SDockTab)
		.Label(LOCTEXT("LandscapeDetailsTab", "Landscape Details"))
		.ToolTipText(LOCTEXT("LandscapeDetailsTabToolTip", "Landscape Details"))
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(2.0)
				[
					SAssignNew(InspectedObjectsDetailsView, SLandscapeEditorInspectedDetailsView)
				]
		];

	InspectedObjectsTab = CreatedTab;
	return CreatedTab.ToSharedRef();
}

//////////////////////////////////////////////////////////////////////////

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SLandscapeEditor::Construct(const FArguments& InArgs, TSharedRef<FLandscapeToolKit> InParentToolkit)
{
	ParentToolkit = InParentToolkit;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bAllowSearch = false;
	// Minimum size to allow the ResetToDefault button to be hit testable.
	DetailsViewArgs.RightColumnMinWidth = 35;

	DetailsPanel = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsPanel->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateSP(this, &SLandscapeEditor::GetIsPropertyVisible));

	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		DetailsPanel->SetObject(LandscapeEdMode->UISettings);
	}

	ChildSlot
	[
		SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 5)
			[
				SAssignNew(Error, SErrorText)
			]
			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			.Padding(0)
			[
				SNew(SVerticalBox)
					.IsEnabled(this, &SLandscapeEditor::GetLandscapeEditorIsEnabled)
					+ SVerticalBox::Slot()
					.Padding(0)
					[
						DetailsPanel.ToSharedRef()
					]
			]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FEdModeLandscape* SLandscapeEditor::GetEditorMode() const
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

FText SLandscapeEditor::GetErrorText() const
{
	const FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode == nullptr)
	{
		return FText::GetEmpty();
	}

	ELandscapeEditingState EditState = LandscapeEdMode->GetEditingState();
	switch (EditState)
	{
		case ELandscapeEditingState::SIEWorld:
		{

			if (LandscapeEdMode->NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
			{
				return LOCTEXT("IsSimulatingError_create", "Can't create landscape while simulating!");
			}
			else
			{
				return LOCTEXT("IsSimulatingError_edit", "Can't edit landscape while simulating!");
			}
		}
		case ELandscapeEditingState::PIEWorld:
		{
			if (LandscapeEdMode->NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
			{
				return LOCTEXT("IsPIEError_create", "Can't create landscape in PIE!");
			}
			else
			{
				return LOCTEXT("IsPIEError_edit", "Can't edit landscape in PIE!");
			}
		}
		case ELandscapeEditingState::BadFeatureLevel:
		{
			if (LandscapeEdMode->NewLandscapePreviewMode != ENewLandscapePreviewMode::None)
			{
				return LOCTEXT("IsFLError_create", "Can't create landscape with a feature level less than SM5!");
			}
			else
			{
				return LOCTEXT("IsFLError_edit", "Can't edit landscape with a feature level less than SM5!");
			}
		}
		case ELandscapeEditingState::NoLandscape:
		{
			return LOCTEXT("NoLandscapeError", "No Landscape!");
		}
		case ELandscapeEditingState::Enabled:
		{
			return FText::GetEmpty();
		}
		default:
			checkNoEntry();
	}

	return FText::GetEmpty();
}

bool SLandscapeEditor::GetLandscapeEditorIsEnabled() const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	if (LandscapeEdMode)
	{
		Error->SetError(GetErrorText());
		return LandscapeEdMode->GetEditingState() == ELandscapeEditingState::Enabled;
	}
	return false;
}

bool SLandscapeEditor::GetInspectedObjectsDetailsIsVisible() const
{
	if (FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		return !LandscapeEdMode->GetInspectedObjects().IsEmpty();
	}
	return false;
}

bool SLandscapeEditor::GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const
{
	return ParentToolkit.Pin()->GetIsPropertyVisibleFromProperty(PropertyAndParent.Property);
}

bool FLandscapeToolKit::GetIsPropertyVisibleFromProperty(const FProperty& Property) const
{
	FEdModeLandscape* LandscapeEdMode = GetEditorMode();
	
	if (LandscapeEdMode != nullptr && LandscapeEdMode->CurrentTool != nullptr)
	{
	    // Hide all properties if the current target can't be edited. Except in New Landscape tool
		if (LandscapeEdMode->CurrentTool->GetToolName() != FName("NewLandscape") &&
			!LandscapeEdMode->CanEditCurrentTarget())
		{
			return false;
		}

		if (Property.HasMetaData("ShowForMask"))
		{
			const bool bMaskEnabled = LandscapeEdMode->CurrentTool &&
				LandscapeEdMode->CurrentTool->SupportsMask() &&
				LandscapeEdMode->CurrentToolTarget.LandscapeInfo.IsValid() &&
				LandscapeEdMode->CurrentToolTarget.LandscapeInfo->SelectedRegion.Num() > 0;

			if (bMaskEnabled)
			{
				return true;
			}
		}
		if (Property.HasMetaData("ShowForTools"))
		{
			const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();

			TArray<FString> ShowForTools;
			Property.GetMetaData("ShowForTools").ParseIntoArray(ShowForTools, TEXT(","), true);
			if (!ShowForTools.Contains(CurrentToolName.ToString()))
			{
				return false;
			}
		}
		if (Property.HasMetaData("ShowForBrushes") && LandscapeEdMode->LandscapeBrushSets.IsValidIndex(LandscapeEdMode->CurrentBrushSetIndex))
		{
			const FName CurrentBrushSetName = LandscapeEdMode->LandscapeBrushSets[LandscapeEdMode->CurrentBrushSetIndex].BrushSetName;
			// const FName CurrentBrushName = LandscapeEdMode->CurrentBrush->GetBrushName();

			TArray<FString> ShowForBrushes;
			Property.GetMetaData("ShowForBrushes").ParseIntoArray(ShowForBrushes, TEXT(","), true);
			if (!ShowForBrushes.Contains(CurrentBrushSetName.ToString()))
				//&& !ShowForBrushes.Contains(CurrentBrushName.ToString())
			{
				return false;
			}
		}
		if (Property.HasMetaData("ShowForTargetTypes"))
		{
			static const TCHAR* TargetTypeNames[] = { TEXT("Heightmap"), TEXT("Weightmap"), TEXT("Visibility") };

			TArray<FString> ShowForTargetTypes;
			Property.GetMetaData("ShowForTargetTypes").ParseIntoArray(ShowForTargetTypes, TEXT(","), true);

			const ELandscapeToolTargetType CurrentTargetType = LandscapeEdMode->CurrentToolTarget.TargetType;
			// ELandscapeToolTargetType::Invalid means "weightmap with no valid paint layer" so we still want to display that property if it has been marked to be displayed in Weightmap target type, to be consistent 
			//  with other paint brush properties (that don't use ShowForTargetTypes), which are still displayed in that case, even if they are ineffective :
			if ((CurrentTargetType == ELandscapeToolTargetType::Invalid) 
				&& (ShowForTargetTypes.FindByKey(TargetTypeNames[static_cast<uint8>(ELandscapeToolTargetType::Weightmap)]) != nullptr))
			{ 
				return true;
			}
			// Otherwise, hide it, if ShowForTargetTypes was used on this property but doesn't correspond to the current target type :
			else if ((CurrentTargetType == ELandscapeToolTargetType::Invalid)
				|| (ShowForTargetTypes.FindByKey(TargetTypeNames[static_cast<uint8>(CurrentTargetType)]) == nullptr))
			{
				return false;
			}
		}
		if (Property.HasMetaData("ShowForBlueprintBrushTool"))
		{
			const FName CurrentToolName = LandscapeEdMode->CurrentTool->GetToolName();

			if (CurrentToolName != TEXT("BlueprintBrush"))
			{
				return false;
			}
		}
		return true;
	}

	return false;
}

void SLandscapeEditor::NotifyToolChanged()
{
	RefreshDetailPanel();
}

void SLandscapeEditor::NotifyBrushChanged()
{
	RefreshDetailPanel();
}

void SLandscapeEditor::RefreshDetailPanel()
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		// Refresh details panel
		DetailsPanel->SetObject(LandscapeEdMode->UISettings, true);
	}
}

//////////////////////////////////////////////////////////////////////////

void SLandscapeEditorInspectedDetailsView::Construct(const FArguments& InArgs)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ObjectsUseNameArea;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = true;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	// Minimum size to allow the ResetToDefault button to be hit testable.
	DetailsViewArgs.RightColumnMinWidth = 35;

	DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);

	ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(10.f, 6.f, 0.f, 4.f)
				.AutoHeight()
				[
					DetailsView->GetNameAreaWidget().ToSharedRef()
				]
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				[
					DetailsView.ToSharedRef()
				]
		];

	// Ensure that the details view calls SetObjects on construction
	// If its not called the panel will not show the "Select an object" message
	RefreshDetailPanel();
}

FEdModeLandscape* SLandscapeEditorInspectedDetailsView::GetEditorMode() const
{
	return (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
}

void SLandscapeEditorInspectedDetailsView::RefreshDetailPanel()
{
	if (const FEdModeLandscape* LandscapeEdMode = GetEditorMode())
	{
		DetailsView->SetObjects(LandscapeEdMode->GetInspectedObjects());
	}
}

#undef LOCTEXT_NAMESPACE
