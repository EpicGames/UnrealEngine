// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "LandscapeEdMode.h"
#include "LandscapeEditorDetails.h"
#include "Toolkits/IToolkitHost.h"
#include "AssetThumbnail.h"
#include "Toolkits/BaseToolkit.h"
#include "Framework/SlateDelegates.h"

class Error;
class IDetailsView;
class SErrorText;
class SLandscapeEditor;
class SLandscapeEditorInspectedDetailsView;
struct FPropertyAndParent;

/**
 * Slate widget wrapping an FAssetThumbnail and Viewport
 */
class SLandscapeAssetThumbnail : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SLandscapeAssetThumbnail )
		: _ThumbnailSize( 64,64 ) {}
		SLATE_ARGUMENT(FIntPoint, ThumbnailSize)
		SLATE_EVENT(FAccessAsset, OnAccessAsset)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UObject* Asset, TSharedRef<FAssetThumbnailPool> ThumbnailPool, const FName& ClassThumbnailBrushOverride);
	~SLandscapeAssetThumbnail();

	void SetAsset(UObject* Asset);
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:
	void OnMaterialCompilationFinished(UMaterialInterface* MaterialInterface);

	FAccessAsset OnAccessAsset;
	TSharedPtr<FAssetThumbnail> AssetThumbnail;
};

namespace LandscapeEditorNames
{
	static const FName Manage(TEXT("ToolMode_Manage")); 
	static const FName Sculpt(TEXT("ToolMode_Sculpt")); 
	static const FName Paint(TEXT("ToolMode_Paint"));
}

/**
 * Mode Toolkit for the Landscape Editor Mode
 */
class FLandscapeToolKit : public FModeToolkit
{
public:
	/** Initializes the landscape mode toolkit */
	virtual void Init(const TSharedPtr< class IToolkitHost >& InitToolkitHost) override;

	/** IToolkit interface */
	virtual FName GetToolkitFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FEdModeLandscape* GetEditorMode() const override;
	virtual TSharedPtr<SWidget> GetInlineContent() const override;

	void NotifyToolChanged();
	void NotifyBrushChanged();
	void RefreshDetailPanel();
	void RefreshInspectedObjectsDetailPanel();

	/** Mode Toolbar Palettes **/
	virtual void GetToolPaletteNames(TArray<FName>& InPaletteName) const;
	virtual FText GetToolPaletteDisplayName(FName PaletteName) const; 
	virtual void BuildToolPalette(FName PaletteName, class FToolBarBuilder& ToolbarBuilder);
	virtual void OnToolPaletteChanged(FName PaletteName) override;

	/** Modes Panel Header Information **/
	virtual FText GetActiveToolDisplayName() const;
	virtual FText GetActiveToolMessage() const;

	bool GetIsPropertyVisibleFromProperty(const FProperty& Property) const;

protected:
	void OnChangeMode(FName ModeName);
	bool IsModeEnabled(FName ModeName) const;
	bool IsModeActive(FName ModeName) const;

	void OnChangeTool(FName ToolName);
	bool IsToolEnabled(FName ToolName) const;
	bool IsToolActive(FName ToolName) const;

	void OnChangeBrushSet(FName BrushSetName);
	bool IsBrushSetEnabled(FName BrushSetName) const;
	bool IsBrushSetActive(FName BrushSetName) const;

	void OnChangeBrush(FName BrushName);
	bool IsBrushActive(FName BrushName) const;

	/** FModeToolkit interface */
	virtual void RequestModeUITabs() override;
	virtual void InvokeUI() override;

	TSharedRef<SDockTab> CreateInspectedObjectsDetailsViewTab(const FSpawnTabArgs& Args);

private:
	TSharedPtr<SLandscapeEditor> LandscapeEditorWidgets;
	TSharedPtr<FLandscapeEditorDetails> BrushesWidgets;

	TWeakPtr<SDockTab> InspectedObjectsTab;
	FMinorTabConfig InspectedObjectsTabInfo;
	TSharedPtr<SLandscapeEditorInspectedDetailsView> InspectedObjectsDetailsView;

	const static TArray<FName> PaletteNames;
};

/**
 * Slate widgets for the Landscape Editor Mode
 */
class SLandscapeEditor : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SLandscapeEditor ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FLandscapeToolKit> InParentToolkit);

	void NotifyToolChanged();
	void NotifyBrushChanged();
	void RefreshDetailPanel();

protected:
	class FEdModeLandscape* GetEditorMode() const;

	FText GetErrorText() const;

	bool GetLandscapeEditorIsEnabled() const;
	bool GetInspectedObjectsDetailsIsVisible() const;

	bool GetIsPropertyVisible(const FPropertyAndParent& PropertyAndParent) const;

protected:
	TSharedPtr<SErrorText> Error;
	TSharedPtr<IDetailsView> DetailsPanel;
	TWeakPtr<FLandscapeToolKit> ParentToolkit;
};

/**
 * Slate widget for the Landscape Editor Inspected Objects Details View
 */
class SLandscapeEditorInspectedDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SLandscapeEditorInspectedDetailsView)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	void RefreshDetailPanel();

protected:
	class FEdModeLandscape* GetEditorMode() const;

private:
	TSharedPtr<IDetailsView> DetailsView;
};
