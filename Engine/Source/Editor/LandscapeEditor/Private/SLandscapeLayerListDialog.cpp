// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLandscapeLayerListDialog.h"
#include "Landscape.h"
#include "LandscapeEditLayer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

#define LOCTEXT_NAMESPACE "LandscapeEditor"

/* File scope utility classes */

struct FWidgetLayerListItem
{
	FWidgetLayerListItem(const ULandscapeEditLayerBase* InEditLayer, const TFunction<void(void)>& InOnLayerListUpdated, TArray<TSharedPtr<FWidgetLayerListItem>>* InWidgetLayerList)
	: LayerName(InEditLayer->GetName())
	, LayerGuid(InEditLayer->GetGuid())
	, OnLayerListUpdated(InOnLayerListUpdated)
	, WidgetLayerList(InWidgetLayerList)
	{ 
	}

	// Duplicated Layer Info
	FName LayerName;
	FGuid LayerGuid;
	
	// Metadata for UI
	bool bAllowedToDrag = false;
	TFunction<void(void)> OnLayerListUpdated;
	TArray<TSharedPtr<FWidgetLayerListItem>>* WidgetLayerList = nullptr;
};

class FWidgetLayerListDragDropOp : public FDragDropOperation
{
public:
	DRAG_DROP_OPERATOR_TYPE(FWidgetLayerListDragDropOp , FDragDropOperation)

	/** The template to create an instance */
	TSharedPtr<FWidgetLayerListItem> ListItem;

	/** Constructs the drag drop operation */
	static TSharedRef<FWidgetLayerListDragDropOp> New(const TSharedPtr<FWidgetLayerListItem>& InListItem, FText InDragText)
	{
		TSharedRef<FWidgetLayerListDragDropOp> Operation = MakeShared<FWidgetLayerListDragDropOp>();
		Operation->ListItem = InListItem;
		Operation->Construct();

		return Operation;
	}
};

class SWidgetLayerListItem : public STableRow<TSharedPtr<FWidgetLayerListItem>>
{
public:
	SLATE_BEGIN_ARGS( SWidgetLayerListItem ){}
	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedPtr<FWidgetLayerListItem> InListItem )
	{
		ListItem = InListItem;
		
		STableRow<TSharedPtr<FWidgetLayerListItem>>::Construct(
			STableRow<TSharedPtr<FWidgetLayerListItem>>::FArguments()
			.OnDragDetected(this, &SWidgetLayerListItem::OnDragDetected)
			.OnCanAcceptDrop(this, &SWidgetLayerListItem::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SWidgetLayerListItem::OnAcceptDrop)
			.Padding(FMargin(0, 0, 30, 0))
			.Content()
			[
				SNew(SBox)
				.Padding(FMargin(0, 2))
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(5, 0, 0, 0)
					[
						SNew(SImage)
						.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
						.Visibility_Lambda([this, InListItem]()
						{
							return IsHovered() && InListItem->bAllowedToDrag ? EVisibility::Visible : EVisibility::Hidden;
						})
					]

					+ SHorizontalBox::Slot()
					.Padding(FMargin(10, 0, 0, 0))
					[
						SAssignNew(TextBlock, STextBlock)
						.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
						.MinDesiredWidth(150)
						.Text(this, &SWidgetLayerListItem::GetLayerName)
						.Justification(ETextJustify::Left)
						.ColorAndOpacity(InListItem->bAllowedToDrag ? FLinearColor::White : FLinearColor(0.25, 0.25, 0.25))
					]
				]
			],
			InOwnerTableView);
	}

private:
	FText GetLayerName() const
	{
		return ListItem.IsValid() ? FText::FromName(ListItem.Pin()->LayerName) : FText::GetEmpty();
	}

	/** Called whenever a drag is detected by the tree view. */
	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
	{
		const TSharedPtr<FWidgetLayerListItem> ListItemPinned = ListItem.Pin();
		if (ListItemPinned.IsValid())
		{
			if (ListItemPinned->bAllowedToDrag == false)
			{
				return FReply::Unhandled();
			}
			
			const FText DefaultText = FText::Format(LOCTEXT("DefaultDragDropText", "Move {0}"), GetLayerName());
			return FReply::Handled().BeginDragDrop(FWidgetLayerListDragDropOp::New(ListItemPinned, DefaultText));
		}
		return FReply::Unhandled();
	}

	/** Called to determine whether a current drag operation is valid for this row. */
	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FWidgetLayerListItem> InListItem)
	{
		const TSharedPtr<FWidgetLayerListDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FWidgetLayerListDragDropOp>();
		
		if (DragDropOp.IsValid() && InListItem.Get()->LayerGuid != DragDropOp.Get()->ListItem.Get()->LayerGuid)
		{
			if (InItemDropZone == EItemDropZone::OntoItem)
			{
				return TOptional<EItemDropZone>();
			}
			
			return InItemDropZone;
		}
		
		return TOptional<EItemDropZone>();
	}

	/** Called to complete a drag and drop onto this drop. */
	FReply OnAcceptDrop(const FDragDropEvent& DragDropEvent, EItemDropZone InItemDropZone, TSharedPtr<FWidgetLayerListItem> InListItem)
	{
		const TSharedPtr<FWidgetLayerListDragDropOp> DragDropOp = DragDropEvent.GetOperationAs<FWidgetLayerListDragDropOp>();
		if (DragDropOp.IsValid() 
			&& DragDropOp->ListItem.IsValid() && InListItem.IsValid()
			&& DragDropOp->ListItem->LayerGuid != InListItem->LayerGuid)
		{			
			// Copy layer to local variable and remove from list
			const TSharedPtr<FWidgetLayerListItem> LayerToMove = DragDropOp->ListItem;
			DragDropOp->ListItem->WidgetLayerList->RemoveAt(InListItem->WidgetLayerList->IndexOfByKey(DragDropOp->ListItem));

			// Determine new index based on drag+drop op
			const int32 RelativeNewIndex = InListItem->WidgetLayerList->IndexOfByKey(InListItem) + (InItemDropZone == EItemDropZone::AboveItem ? 0 : 1);
			
			// Insert copied layer into list at new position and update pointer in DragDropOp->ListItem
			DragDropOp->ListItem->WidgetLayerList->Insert(LayerToMove, RelativeNewIndex);

			// Reconstruct and refresh ListView
			InListItem->OnLayerListUpdated();
			
			return FReply::Handled();
		}
		return FReply::Unhandled();
	}

private:
	TWeakPtr<FWidgetLayerListItem> ListItem;
	TSharedPtr<STextBlock> TextBlock;
};

/* SLandscapeLayerListDialog implementation */

void SLandscapeLayerListDialog::Construct(const FArguments& InArgs, const TWeakObjectPtr<ALandscape>& InLandscape)
{
	Landscape = InLandscape;
	check(Landscape.IsValid());

	int32 LayerIndex = 0;
	const TArray<const ULandscapeEditLayerBase*> LandscapeEditLayers = Landscape->GetEditLayersConst();
	for (const ULandscapeEditLayerBase* EditLayer : LandscapeEditLayers)
	{
		WidgetLayerList.Add(MakeShared<FWidgetLayerListItem>(LandscapeEditLayers[LandscapeEditLayers.Num() - 1 - LayerIndex], [this](){ OnLayerListUpdated(); }, &WidgetLayerList));
		++LayerIndex;
	}

	WidgetLayerList[0]->bAllowedToDrag = true;
	InsertedLayerIndex = LandscapeEditLayers.Num() - 1;
	
	// Construct list view
	SAssignNew(LayerListView, SWidgetLayerListView)
	.SelectionMode(ESelectionMode::Single)
	.OnGenerateRow(this, &SLandscapeLayerListDialog::OnGenerateRow)
	.ListItemsSource(&WidgetLayerList);
	
	// Construct custom dialog with list view supporting drag + drop
	SCustomDialog::Construct(SCustomDialog::FArguments()
		.Title(FText(LOCTEXT("LandscapeLayerListDialogTitleText", "Insert New Landscape Edit Layer")))
		.UseScrollBox(false)
		.HAlignButtonBox(HAlign_Center)
		.WindowArguments(SWindow::FArguments()
		.HasCloseButton(false))
		.ContentAreaPadding(10.0f)
		.Content()
		[
			SNew(SBox)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Font(FCoreStyle::Get().GetFontStyle("NormalFont"))
					.WrapTextAt(350)
					.Text(FText::Format(LOCTEXT(
						"LandscapeLayerListDialogInstructionText",
						"Drag/drop the \"{0}\" layer in the list to choose where in the edit layer stack it should be inserted.\n"),
						FText::FromName(WidgetLayerList[0]->LayerName)))
				]
				+ SVerticalBox::Slot()
				.MaxHeight(125)
				.HAlign(HAlign_Center)
				[
					SNew(SBorder)
					[
						LayerListView.ToSharedRef()
					]
				]
			]
		]
		.Buttons({
			FButton(LOCTEXT("CompleteButtonText", "Complete"), FSimpleDelegate::CreateSP(this, &SLandscapeLayerListDialog::OnComplete), SCustomDialog::EButtonRole::Confirm)
				.SetPrimary(true)
		})
	);
}

void SLandscapeLayerListDialog::OnLayerListUpdated()
{
	if (LayerListView.IsValid())
	{
		LayerListView->RequestListRefresh();
	}
}

TSharedRef<ITableRow> SLandscapeLayerListDialog::OnGenerateRow(TSharedPtr<FWidgetLayerListItem> InListItem, const TSharedRef< STableViewBase >& InOwnerTableView) const
{
	return SNew( SWidgetLayerListItem, InOwnerTableView, InListItem );
}

void SLandscapeLayerListDialog::OnComplete()
{
	check(Landscape.IsValid());
	const TArray<const ULandscapeEditLayerBase*> LandscapeEditLayers = Landscape->GetEditLayersConst();
	// Find the draggable layer in WidgetLayerList
	for (int WidgetLayerListIndex = 0; WidgetLayerListIndex < WidgetLayerList.Num(); ++WidgetLayerListIndex)
	{
		const TSharedPtr<FWidgetLayerListItem>& WidgetLayer = WidgetLayerList[WidgetLayerList.Num() - 1 - WidgetLayerListIndex];
		
		if (WidgetLayer->bAllowedToDrag)
		{
			// Find actual edit layer corresponding to draggable layer
			for (int32 LayerIndex = 0; LayerIndex < LandscapeEditLayers.Num(); ++LayerIndex)
			{
				const ULandscapeEditLayerBase* EditLayer = LandscapeEditLayers[LayerIndex];

				// Once found, remove and copy edit layer into correct spot in LayerList
				if (WidgetLayer->LayerGuid == EditLayer->GetGuid())
				{
					InsertedLayerIndex = WidgetLayerListIndex;
					Landscape->ReorderLayer(LayerIndex, InsertedLayerIndex);
					
					return;
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE 