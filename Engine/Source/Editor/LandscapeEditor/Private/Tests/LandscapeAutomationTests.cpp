// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Engine/DirectionalLight.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Misc/Paths.h"
#include "Misc/AutomationTest.h"
#include "Misc/EngineVersion.h"
#include "Widgets/SWindow.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LandscapeEditorObject.h"
#include "LandscapeInfoMap.h"

#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorDetailCustomization_ImportExport.h"
#include "LandscapeEditorDetailCustomization_MiscTools.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "LandscapeEditor.NewLandscape"

DEFINE_LOG_CATEGORY_STATIC(LogLandscapeAutomationTests, Log, All);

/**
* Landscape test helper functions
*/
namespace LandscapeTestUtils
{
	constexpr EAutomationTestFlags LandscapeTestFlags = EAutomationTestFlags::EditorContext | EAutomationTestFlags::NonNullRHI | EAutomationTestFlags::EngineFilter;
	const FString LandscapeDailyTestTags = TEXT("[GraphicsTools][Terrain][Landscape][DailyEssential]");

	/**
	* Finds the viewport to use for the landscape tool
	*/
	static FLevelEditorViewportClient* FindSelectedViewport()
	{
		FLevelEditorViewportClient* SelectedViewport = NULL;

		for(FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
		{
			if (!ViewportClient->IsOrtho())
			{
				SelectedViewport = ViewportClient;
			}
		}

		return SelectedViewport;
	}

	struct LandscapeTestCommands
	{
		static void Import(const FString& HeightMapFilenamee);
		static bool CreateNewMapWithLandscape(int32 ComponentCountXY, int32 QuadsPerComponent);
	};

	void LandscapeTestCommands::Import(const FString& HeightMapFilename)
	{
		//Switch to the Landscape tool
		GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_Landscape);
		FEdModeLandscape* LandscapeEdMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);

		TSharedPtr<FLandscapeEditorDetailCustomization_MiscTools> Customization_MiscTools = MakeShareable(new FLandscapeEditorDetailCustomization_MiscTools);
		TSharedPtr<FLandscapeEditorDetailCustomization_ImportExport> Customization_ImportExportLandscape = MakeShareable(new FLandscapeEditorDetailCustomization_ImportExport);

		LandscapeEdMode->UISettings->ImportLandscape_HeightmapFilename = HeightMapFilename;
		LandscapeEdMode->UISettings->bHeightmapSelected = true;
		LandscapeEdMode->UISettings->ImportExportMode = ELandscapeImportExportMode::All;
		LandscapeEdMode->UISettings->ImportType = ELandscapeImportTransformType::Resample;

		Customization_MiscTools->ResetGizmoToOrigin();
		Customization_ImportExportLandscape->OnImportExportButtonClicked();
	}

	bool LandscapeTestCommands::CreateNewMapWithLandscape(int32 ComponentCountXY, int32 QuadsPerComponent)
	{
		UE_LOGF(LogLandscapeAutomationTests, Display, "Creating a new map...");
		UWorld* NewMap = FAutomationEditorCommonUtils::CreateNewMap();
		if (!NewMap)
		{
			UE_LOGF(LogLandscapeAutomationTests, Error, "Unable to create new map");
			return false;
		}

		UE_LOGF(LogLandscapeAutomationTests, Display, "Switching to Landscape Editor Mode...");
		GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_Landscape);
		FEdModeLandscape* LandscapeEditMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
		if (!LandscapeEditMode)
		{
			UE_LOGF(LogLandscapeAutomationTests, Error, "Unable to enter Landscape Edit Mode");
			return false;
		}

		UE_LOGF(LogLandscapeAutomationTests, Display, "Creating a new landscape...");
		LandscapeEditMode->UISettings->NewLandscape_QuadsPerSection = QuadsPerComponent;
		LandscapeEditMode->UISettings->NewLandscape_ComponentCount.X = ComponentCountXY;
		LandscapeEditMode->UISettings->NewLandscape_ComponentCount.Y = ComponentCountXY;
		LandscapeEditMode->UISettings->NewLandscape_ClampSize();

		TSharedPtr<FLandscapeEditorDetailCustomization_NewLandscape> Customization_NewLandscape = MakeShareable(new FLandscapeEditorDetailCustomization_NewLandscape);
		Customization_NewLandscape->OnCreateButtonClicked();

		if (!LandscapeEditMode->GetLandscape())
		{
			UE_LOGF(LogLandscapeAutomationTests, Error, "Unable to create a new landscape");
			return false;
		}

		return true;
	}
}

/**
* Latent command to create a new landscape
*/
DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER(FCreateNewMapWithLandscapeCommand, int32, ComponentCountXY, int32, QuadsPerComponent);
bool FCreateNewMapWithLandscapeCommand::Update()
{
	LandscapeTestUtils::LandscapeTestCommands::CreateNewMapWithLandscape(ComponentCountXY, QuadsPerComponent);
	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FImportLandscapeCommand, FString, HeightMapFilename);
bool FImportLandscapeCommand::Update()
{
	LandscapeTestUtils::LandscapeTestCommands::Import(HeightMapFilename);
	return true;
}


DEFINE_LATENT_AUTOMATION_COMMAND(FAddDirectionalLight);
bool FAddDirectionalLight::Update()
{
	UWorld* World = GEditor->GetEditorWorldContext().World();
	
	ADirectionalLight*  DirectionalLight = World->SpawnActor<ADirectionalLight>();

	return true;
}

/**
* Latent command to start using the sculpting tool
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FBeginModifyLandscapeCommand);
bool FBeginModifyLandscapeCommand::Update()
{
	//Find the landscape
	FEdModeLandscape* LandscapeEdMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);

	//Find a location on the edge of the landscape along the x axis so the default camera can see it in the distance.
	FVector LandscapeSizePerComponent = LandscapeEdMode->UISettings->NewLandscape_QuadsPerSection * LandscapeEdMode->UISettings->NewLandscape_SectionsPerComponent * LandscapeEdMode->UISettings->NewLandscape_Scale;
	FVector TargetLoctaion(0);
	TargetLoctaion.X = -LandscapeSizePerComponent.X * (LandscapeEdMode->UISettings->NewLandscape_ComponentCount.X / 2.f);

	ALandscapeProxy* Proxy = LandscapeEdMode->CurrentToolTarget.LandscapeInfo.Get()->GetCurrentLevelLandscapeProxy(true);
	if (Proxy)
	{
		TargetLoctaion = Proxy->LandscapeActorToWorld().InverseTransformPosition(TargetLoctaion);
	}

	//Begin using the sculpting tool
	FLevelEditorViewportClient* SelectedViewport = LandscapeTestUtils::FindSelectedViewport();
	LandscapeEdMode->CurrentTool->BeginTool(SelectedViewport, LandscapeEdMode->CurrentToolTarget, TargetLoctaion);
	SelectedViewport->Invalidate();

	UE_LOGF(LogLandscapeAutomationTests, Display, "Modified the landscape using the sculpt tool");

	return true;
}

/**
*  Latent command to stop using the sculpting tool
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FEndModifyLandscapeCommand);
bool FEndModifyLandscapeCommand::Update()
{
	//Find the landscape
	FEdModeLandscape* LandscapeEdMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);

	//End using the sculpting tool
	FLevelEditorViewportClient* SelectedViewport = LandscapeTestUtils::FindSelectedViewport();
	LandscapeEdMode->CurrentTool->EndTool(SelectedViewport);

	return true;
}

DEFINE_LATENT_AUTOMATION_COMMAND(FCheckHeight);
bool FCheckHeight::Update()
{
	FAutomationTestBase* CurrentTest = FAutomationTestFramework::Get().GetCurrentTest();
	if (!CurrentTest)
	{
		return true;
	}

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		return true;
	}

	
	if (auto It = ULandscapeInfoMap::GetLandscapeInfoMap(World).Map.CreateConstIterator(); It)
	{
		const auto& InfoMapPair = *It;
		TOptional<float> Height = InfoMapPair.Value->GetLandscapeProxy()->GetHeightAtLocation(FVector(0, 0, 0), EHeightfieldSource::Editor);
		CurrentTest->TestEqual("Has Height Value at 0,0", Height.IsSet(), true);
		CurrentTest->TestNearlyEqual("Height Value at 0,0 is 0", Height.GetValue(), 0.0f, 1e-4f);
	}
	return true;
}

/**
*  Latent command resetting to Editor Mode defaults
*/
DEFINE_LATENT_AUTOMATION_COMMAND(FResetToDefaultModeCommand);
bool FResetToDefaultModeCommand::Update()
{
	// Reset to the default editing mode
	GLevelEditorModeTools().ActivateDefaultMode();
	return true;
}

/**
* Landscape creation / edit test
*/
const FString LandscapeEditorTestName = TEXT("Editor.Landscape.Create and Modify New Landscape");

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLandscapeEditorTest, LandscapeEditorTestName, LandscapeTestUtils::LandscapeTestFlags)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FLandscapeEditorTest, LandscapeEditorTestName, LandscapeTestUtils::LandscapeDailyTestTags)
bool FLandscapeEditorTest::RunTest(const FString& Parameters)
{
	// Create a new map and landscape
	ADD_LATENT_AUTOMATION_COMMAND(FCreateNewMapWithLandscapeCommand(/* NumComponents = */ 8, /* QuadsPerComponent = */ 7));

	//For some reason the heightmap component takes a few ticks to register with the nav system.  We crash if we try to modify the heightmap before then.
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.0f));
	ADD_LATENT_AUTOMATION_COMMAND(FBeginModifyLandscapeCommand());
	ADD_LATENT_AUTOMATION_COMMAND(FEndModifyLandscapeCommand());
	ADD_LATENT_AUTOMATION_COMMAND(FResetToDefaultModeCommand());

	return true;
}

/**
 * Landscape - Import Landscape Test
 */
const FString LandscapeEditorImportTestName = TEXT("Editor.Landscape.Import Landscape");

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLandscapeEditorImportTest, LandscapeEditorImportTestName, LandscapeTestUtils::LandscapeTestFlags)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FLandscapeEditorImportTest, LandscapeEditorImportTestName, LandscapeTestUtils::LandscapeDailyTestTags)
bool FLandscapeEditorImportTest::RunTest(const FString& Parameters)
{
	// Create a new map and landscape
	ADD_LATENT_AUTOMATION_COMMAND(FCreateNewMapWithLandscapeCommand(/* NumComponents = */ 8, /* QuadsPerComponent = */ 7));

	//For some reason the heightmap component takes a few ticks to register with the nav system.  We crash if we try to modify the heightmap before then.
	ADD_LATENT_AUTOMATION_COMMAND(FWaitLatentCommand(1.0f));

	FString HeightMapFilename = FPaths::Combine(FPaths::EngineContentDir(), FString(TEXT("FunctionalTesting\\height-505-flat.png")));
	ADD_LATENT_AUTOMATION_COMMAND(FImportLandscapeCommand(HeightMapFilename));

	ADD_LATENT_AUTOMATION_COMMAND(FAddDirectionalLight());

	ADD_LATENT_AUTOMATION_COMMAND(FCheckHeight());

	// Importing the landscape switches to the landscape mode
	ADD_LATENT_AUTOMATION_COMMAND(FResetToDefaultModeCommand());

	return true;
}

/*
 * Verify that we can create a new edit layer for a landscape
 * QMetry: UE-TC-4704
 */
const FString LandscapeEditorNewLayerTestName = TEXT("Editor.Landscape.Create New Edit Layer");

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FLandscapeEditorCreateNewLayer, LandscapeEditorNewLayerTestName, LandscapeTestUtils::LandscapeTestFlags)
REGISTER_SIMPLE_AUTOMATION_TEST_TAGS(FLandscapeEditorCreateNewLayer, LandscapeEditorNewLayerTestName, LandscapeTestUtils::LandscapeDailyTestTags)
bool FLandscapeEditorCreateNewLayer::RunTest(const FString& Parameters)
{
	// Create a new map with landscape
	LandscapeTestUtils::LandscapeTestCommands::CreateNewMapWithLandscape(8, 63);

	UE_LOGF(LogLandscapeAutomationTests, Display, "Verifying that new landscapes start with 1 layer...");
	FEdModeLandscape* LandscapeEditMode = (FEdModeLandscape*)GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_Landscape);
	ALandscape* CurrentLandscape = LandscapeEditMode->GetLandscape();
	TestNotNull("Current landscape should not be null", CurrentLandscape);
	TestEqual("New landscapes should start with one edit layer", LandscapeEditMode->GetLayerCount(), 1);

	UE_LOGF(LogLandscapeAutomationTests, Display, "Verifying that we successfully added a new layer...");
	FName LayerName = TEXT("TestLayer");
	CurrentLandscape->CreateLayer(LayerName);
	TestEqual("The landscape should now have a new edit layer", LandscapeEditMode->GetLayerCount(), 2);
	TestNotNull("There should be a new layer with the specified name", CurrentLandscape->GetLayerConst(LayerName));

	ADD_LATENT_AUTOMATION_COMMAND(FResetToDefaultModeCommand());

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif //WITH_DEV_AUTOMATION_TESTS
