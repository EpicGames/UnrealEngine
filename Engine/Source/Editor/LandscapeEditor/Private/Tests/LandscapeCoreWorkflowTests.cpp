// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "CQTest.h"
#include "EditorWorldUtils.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "EditorModes.h"
#include "EditorModeManager.h"
#include "Modules/ModuleManager.h"
#include "Landscape.h"
#include "LandscapeEdMode.h"
#include "EditorModeTools.h"
#include "IAssetViewport.h"
#include "LandscapeEditorDetailCustomization_NewLandscape.h"
#include "LandscapeEditorObject.h"
#include "LandscapeEdModeTools.h"
#include "LandscapeSettings.h"
#include "LevelEditor.h"
#include "Layers/LayersSubsystem.h"
#include "Tests/AutomationEditorCommon.h"
#include "Framework/Application/SlateApplication.h"

DEFINE_LOG_CATEGORY_STATIC(LandscapeTestsLog, Log, All);

TEST_CLASS_WITH_FLAGS(LandscapeCoreWorkflow, "Editor.Landscape", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	UWorld* World = nullptr;
	const FEditorModeID LandscapeMode = FBuiltinEditorModes::EM_Landscape;
	FEdModeLandscape* LandscapeEditMode = nullptr;
	FModeTool* ActiveTool = nullptr;

	BEFORE_EACH()
	{
		// Initialize world creation values.
		FWorldInitializationValues InitializationValues = {};

		// Define the level template to use ("Basic" template).
		const FString TemplateMap = TEXT("Basic");

		// Set up world partition and streaming properties for the world creation.
		InitializationValues.CreateWorldPartition(false);
		InitializationValues.EnableWorldPartitionStreaming(false);

		// Create a scoped editor world using the specified template and initialization values.
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(TemplateMap, InitializationValues);

		// Retrieve the created world's pointer and ensure it is valid.
		World = ScopedEditorWorld->GetWorld();
		ASSERT_THAT(IsNotNull(World, TEXT("World in invalid")));

		// Remove any existing static mesh actors labeled "Floor" from the world to maintain a clean slate.
		CleanupActorsByLabel(TEXT("Floor"));
	}

	// Test verifies Landscape Mode is visible in the selection mode.
	TEST_METHOD(Verify_Landscape_Mode_Is_Visible_In_SelectionPanel)
	{
		bool bIsModeVisible = false;
		// Retrieve the Editor's Asset Editor Subsystem.
		if (const UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
		{
			// Check if the Landscape Mode is registered and retrieve its visibility status.
			FEditorModeInfo ModeInfo;
			if (AssetEditorSubsystem->FindEditorModeInfo(LandscapeMode, ModeInfo))
			{
				bIsModeVisible = ModeInfo.IsVisible();
			}
		}
		// Assert that Landscape Mode is visible in the selection panel.
		ASSERT_THAT(IsTrue(bIsModeVisible, TEXT("Landscape Mode is not visible in the mode selection panel.")));
	}

	// Test verifies that pressing the hotkey (Shift + 2) correctly exposes the Landscape Mode panel in the Unreal Editor.
	TEST_METHOD(Verify_Activate_Landscape_Mode_With_Hotkey)
	{
		// Ensure the Slate Application is initialized
		ASSERT_THAT(IsTrue(FSlateApplication::IsInitialized(), TEXT("SlateApplication should be initialized before proceeding.")));

		FSlateApplication& Slate = FSlateApplication::Get();

		// Define the modifier key states for pressing and releasing the "2" key (Shift + 2)
		const FModifierKeysState KeyTwoDown = {
			true, // Shift is held down
			false, false, false,
			false, false, false, false, false
		};
		const FModifierKeysState KeyTwoRelease = {
			false, // Shift is released
			false, false, false,
			false, false, false, false, false
		};

		// Create key events for pressing and releasing the "2" key with LeftShift as a modifier
		const FKeyEvent HotkeyDownEvent = FKeyEvent(EKeys::Two, KeyTwoDown, 0, false, 0, 0);
		const FKeyEvent HotkeyUpEvent = FKeyEvent(EKeys::Two, KeyTwoRelease, 0, false, 0, 0);

		// Load the Level Editor module and get the active viewport
		FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>("LevelEditor");
		ASSERT_THAT(IsNotNull(LevelEditorModule, TEXT("Failed to load LevelEditor module.")));

		const TSharedPtr<IAssetViewport> ActiveViewport = LevelEditorModule->GetFirstActiveViewport();
		ASSERT_THAT(IsNotNull(ActiveViewport, TEXT("Failed to find active Level Editor viewport.")));

		// Set keyboard focus to the active viewport to ensure it receives input
		Slate.ClearAllUserFocus(EFocusCause::SetDirectly);
		const bool bFocusSet = Slate.SetKeyboardFocus(ActiveViewport->AsWidget(), EFocusCause::SetDirectly);
		ASSERT_THAT(IsTrue(bFocusSet, TEXT("Failed to set keyboard focus to the active viewport.")));

		// Simulate the pressing and releasing of the hotkey (Shift + 2)
		Slate.ProcessKeyDownEvent(HotkeyDownEvent);
		Slate.ProcessKeyUpEvent(HotkeyUpEvent);

		// Check if the Landscape Mode panel is activated after the hotkey is pressed
		LandscapeEditMode = static_cast<FEdModeLandscape*>(GLevelEditorModeTools().
			GetActiveMode(FBuiltinEditorModes::EM_Landscape));

		ASSERT_THAT(IsNotNull(LandscapeEditMode, TEXT("Landscape Mode panel is not properly exposed after pressing the hotkey (Shift + 2).")));
	}

	// Test verifies that Landscape Creation is successful and default material is WorldGridMaterial.
	TEST_METHOD(Verify_Landscape_Creation_In_Viewport)
	{
		// Access the Editor Mode Tools to activate Landscape Mode
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
		EditorModeTools.ActivateMode(LandscapeMode);
		ASSERT_THAT(IsTrue(EditorModeTools.IsModeActive(LandscapeMode), TEXT("Failed to activate Landscape Mode.")));

		LandscapeEditMode = static_cast<FEdModeLandscape*>(EditorModeTools.GetActiveMode(LandscapeMode));
		ASSERT_THAT(IsNotNull(LandscapeEditMode, TEXT("Failed to retrieve the Landscape Edit Mode.")));

		// Create the landscape using predefined settings
		ALandscape* CreatedLandscape = CreateLandscape();

		// Assert that a landscape object exists in the world after creation
		ASSERT_THAT(IsNotNull(CreatedLandscape, TEXT("Failed to create a landscape in the viewport.")));

		// Verify that the default WorldGridMaterial is applied as the landscape material
		const UMaterialInterface* LandscapeMaterial = CreatedLandscape->GetLandscapeMaterial();
		const FString MaterialName = LandscapeMaterial->GetName();
		ASSERT_THAT(IsTrue(MaterialName.Contains(TEXT("WorldGridMaterial")),
			TEXT("WorldGridMaterial is not applied by default as a landscape material")));
	}

	// Test verifies that when a new landscape is created in the editor, the Sculpt tool is activated by default.
	TEST_METHOD(Verify_Creating_Landscape_Activates_Sculpt_Tool_By_Default)
	{
		// Access the Editor Mode Tools to activate Landscape Mode
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
		EditorModeTools.ActivateMode(LandscapeMode);
		ASSERT_THAT(IsTrue(EditorModeTools.IsModeActive(LandscapeMode), TEXT("Failed to activate Landscape Mode.")));

		LandscapeEditMode = static_cast<FEdModeLandscape*>(EditorModeTools.GetActiveMode(LandscapeMode));
		ASSERT_THAT(IsNotNull(LandscapeEditMode, TEXT("Failed to retrieve the Landscape Edit Mode.")));

		// Create the landscape using predefined settings
		ALandscape* CreatedLandscape = CreateLandscape();
		// Assert that a landscape object exists in the world after creation
		ASSERT_THAT(IsNotNull(CreatedLandscape, TEXT("Failed to create a landscape in the viewport.")));

		// Verify that the Sculpt tool is activated by default
		FLandscapeToolMode* ToolMode = LandscapeEditMode->CurrentToolMode;
		ASSERT_THAT(IsNotNull(ToolMode, TEXT("Failed to retrieve current tool for landscape mode.")));

		// Compare the current tool name with the Sculpt tool
		const FName SculptToolName = TEXT("Sculpt");
		const FName ToolName = ToolMode->CurrentToolName;
		ASSERT_THAT(IsTrue(ToolName == SculptToolName, TEXT("Sculpt tool is not activated by default.")));
	}

	// Test verifies that the Sculpt Brush Gizmo is enabled by default when Landscape Mode is activated.
	TEST_METHOD(Verify_Landscape_Sculpt_Brush_Gizmo_Enabled_By_Default)
	{
		// Access the Editor Mode Tools to activate Landscape Mode
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
		EditorModeTools.ActivateMode(LandscapeMode);
		ASSERT_THAT(IsTrue(EditorModeTools.IsModeActive(LandscapeMode), TEXT("Failed to activate Landscape Mode.")));

		LandscapeEditMode = static_cast<FEdModeLandscape*>(EditorModeTools.GetActiveMode(LandscapeMode));
		ASSERT_THAT(IsNotNull(LandscapeEditMode, TEXT("Failed to retrieve the Landscape Edit Mode.")));

		// Create the landscape using predefined settings
		ALandscape* CreatedLandscape = CreateLandscape();
		// Assert that a landscape object exists in the world after creation
		ASSERT_THAT(IsNotNull(CreatedLandscape, TEXT("Failed to create a landscape in the viewport.")));

		// Retrieve the current tool mode in Landscape Edit Mode
		FLandscapeToolMode* ToolMode = LandscapeEditMode->CurrentToolMode;
		ASSERT_THAT(IsNotNull(ToolMode, TEXT("Failed to retrieve current tool for landscape mode.")));

		// Retrieve the current landscape brush (gizmo) being used
		FLandscapeBrush* SculptBrush = LandscapeEditMode->CurrentBrush;
		ASSERT_THAT(IsNotNull(SculptBrush, TEXT("Failed to retrieve the active landscape sculpt brush.")));

		// Get the name of the active brush
		const FString ActiveBrushName = FString(SculptBrush->GetBrushName());
		const FString ExpectedBrushName = TEXT("Circle_Smooth");

		ASSERT_THAT(IsTrue(ActiveBrushName == ExpectedBrushName, TEXT("The default landscape sculpt brush is not 'Circle_Smooth'.")));
	}

	AFTER_EACH()
	{
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
		// Reset the EditorMode to Default
		const FEditorModeID DefaultMode = FBuiltinEditorModes::EM_Default;
		if (!EditorModeTools.IsModeActive(DefaultMode))
		{
			EditorModeTools.ActivateMode(DefaultMode);
		}
		ON_SCOPE_EXIT
		{
			ScopedEditorWorld.Reset();
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		};
	}

	void CleanupActorsByLabel(const FString& ActorLabel) const
	{
		const FString ActorName = ActorLabel;
		for (AActor* Actor : TActorRange<AActor>(World))
		{
			if (Actor->GetActorLabel() == ActorName)
			{
				ULayersSubsystem* Layers = GEditor->GetEditorSubsystem<ULayersSubsystem>();
				if (Actor->CanModify())
				{
					GEditor->SelectActor(Actor, false, false);
					Layers->DisassociateActorFromLayers(Actor);
					World->EditorDestroyActor(Actor, true);
					break;
				}
			}
		}
	}

	static ALandscape* CreateLandscape()
	{
		// Access the Editor Mode Tools to activate Landscape Mode
		FEditorModeTools& EditorModeTools = GLevelEditorModeTools();
		// Retrieve the active Landscape Edit Mode and ensure it's not null
		const FEdModeLandscape* LandscapeEditMode = static_cast<FEdModeLandscape*>(EditorModeTools.GetActiveMode(FBuiltinEditorModes::EM_Landscape));
		if (!LandscapeEditMode)
		{
			UE_LOGF(LandscapeTestsLog, Error, "Failed to retrieve the Landscape Edit Mode.");
			return nullptr;
		}

		// Set up the necessary UISettings for landscape creation
		LandscapeEditMode->UISettings->NewLandscape_QuadsPerSection = 63;
		LandscapeEditMode->UISettings->NewLandscape_SectionsPerComponent = 1;
		LandscapeEditMode->UISettings->NewLandscape_ComponentCount.X = 8;
		LandscapeEditMode->UISettings->NewLandscape_ComponentCount.Y = 8;
		LandscapeEditMode->UISettings->NewLandscape_ClampSize();

		// The tests expect the default landscape material, not the cached material from the editor ini.
		if (const ULandscapeSettings* LandscapeSettings = GetDefault<ULandscapeSettings>())
		{
			LandscapeEditMode->UISettings->NewLandscape_Material = LandscapeSettings->GetDefaultLandscapeMaterial().LoadSynchronous();
		}

		// Call the function that triggers the landscape creation
		const TSharedPtr<FLandscapeEditorDetailCustomization_NewLandscape> Customization = MakeShareable(new FLandscapeEditorDetailCustomization_NewLandscape);
		const FReply Reply = Customization->OnCreateButtonClicked();

		// Ensure that the function executed successfully
		if (!Reply.IsEventHandled())
		{
			UE_LOGF(LandscapeTestsLog, Error, "Create button click event was not handled successfully.");
			return nullptr;
		}

		// Verify that the landscape has been created in the viewport
		UWorld* EditorWorld = EditorModeTools.GetWorld();
		for (ALandscape* Landscape : TActorRange<ALandscape>(EditorWorld))
		{
			if (LandscapeEditMode->GetLandscape()->GetActorNameOrLabel() == Landscape->GetActorNameOrLabel())
			{
				return Landscape;
			}
		}

		// If no landscape was found, log the failure and return nullptr
		UE_LOGF(LandscapeTestsLog, Error, "No landscape was created in the viewport.");
		return nullptr;
	}
};

#endif

#endif
