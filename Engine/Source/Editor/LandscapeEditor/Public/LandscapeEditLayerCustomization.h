// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/Map.h"
class ULandscapeEditLayerBase;
class FMenuBuilder;
struct FMenuEntryParams;

struct FEditLayerMenuBlock
{
	TArray<FMenuEntryParams> Entries;
	FText SectionLabel;
};

/* Map of context menu Category Names to their entries */
using FEditLayerCategoryToEntryMap = TMap<FName, FEditLayerMenuBlock>;

/**
 * Interface for Edit Layer classes that expand on customizations not covered by IDetailsView (Context Menu, Dialog, etc.)
 */
class IEditLayerCustomization : public TSharedFromThis<IEditLayerCustomization>
{
public:
	virtual ~IEditLayerCustomization() = default;

	/** Called when building edit layer context menus */
	virtual void CustomizeContextMenu(ULandscapeEditLayerBase* InEditLayer, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) = 0;
};
