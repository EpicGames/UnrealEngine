// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/EnableIf.h"
#include "Containers/Map.h" 

template<class T>
class ILandscapeFileFormat;

using ILandscapeHeightmapFileFormat = ILandscapeFileFormat<uint16>;
using ILandscapeWeightmapFileFormat = ILandscapeFileFormat<uint8>;

class FUICommandList;
class FLandscapeImageFileCache;
class IEditLayerCustomization;
class ULandscapeEditLayerBase;
struct FEditLayerMenuBlock;
using FEditLayerCategoryToEntryMap = TMap<FName, FEditLayerMenuBlock>;

/** Delegate called to get customizations for a specific edit layer class */
DECLARE_DELEGATE_RetVal(TSharedRef<IEditLayerCustomization>, FOnGetLandscapeEditLayerCustomizationInstance);

using FCustomEditLayerLayoutNameMap = TMap<FName, FOnGetLandscapeEditLayerCustomizationInstance>;

/**
 * LandscapeEditor module interface
 */
class ILandscapeEditorModule : public IModuleInterface
{
public:
	// Register / unregister a landscape file format plugin
	virtual void RegisterHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> FileFormat) = 0;
	virtual void RegisterWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> FileFormat) = 0;
	virtual void UnregisterHeightmapFileFormat(TSharedRef<ILandscapeHeightmapFileFormat> FileFormat) = 0;
	virtual void UnregisterWeightmapFileFormat(TSharedRef<ILandscapeWeightmapFileFormat> FileFormat) = 0;

	// Gets the type string used by the import/export file dialog
	virtual const TCHAR* GetHeightmapImportDialogTypeString() const = 0;
	virtual const TCHAR* GetWeightmapImportDialogTypeString() const = 0;
	virtual const TCHAR* GetHeightmapExportDialogTypeString() const = 0;
	virtual const TCHAR* GetWeightmapExportDialogTypeString() const = 0;

	// Edit layer customizations
	virtual void RegisterCustomEditLayerClassLayout(FName ClassName, FOnGetLandscapeEditLayerCustomizationInstance EditLayerLayoutDelegate) = 0;
	virtual void UnregisterCustomEditLayerClassLayout(FName ClassName) = 0;

	virtual TArray<TSharedRef<IEditLayerCustomization>> QueryCustomEditLayerLayoutRecursive(UStruct* InClass) = 0;
	virtual TSharedPtr<IEditLayerCustomization> QueryCustomEditLayerLayoutForClass(UStruct* InClass) = 0;

	virtual void ApplyEditLayerContextMenuCustomizations(ULandscapeEditLayerBase* InEditLayer, const TArray<TSharedRef<IEditLayerCustomization>>& InCustomizations, FEditLayerCategoryToEntryMap& OutContextMenuEntryMap) = 0;

	// Gets the heightmap/weightmap format associated with a given extension (null if no plugin is registered for this extension)
	virtual const ILandscapeHeightmapFileFormat* GetHeightmapFormatByExtension(const TCHAR* Extension) const = 0;
	virtual const ILandscapeWeightmapFileFormat* GetWeightmapFormatByExtension(const TCHAR* Extension) const = 0;

	template<typename T>
	typename TEnableIf<std::is_same_v<T, uint16>, const ILandscapeHeightmapFileFormat*>::Type GetFormatByExtension(const TCHAR* Extension)
	{
		return GetHeightmapFormatByExtension(Extension);
	}

	template<typename T>
	typename TEnableIf<std::is_same_v<T, uint8>, const ILandscapeWeightmapFileFormat*>::Type GetFormatByExtension(const TCHAR* Extension)
	{
		return GetWeightmapFormatByExtension(Extension);
	}

	virtual TSharedPtr<FUICommandList> GetLandscapeLevelViewportCommandList() const = 0;

	virtual FLandscapeImageFileCache& GetImageFileCache() const = 0;


};
