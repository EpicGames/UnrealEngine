// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "LandscapeProxy.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeBlueprintBrushBase.h"
#include "LandscapeImportHelper.h"

#include "LandscapeEditorObject.generated.h"

class ALandscapeGizmoActiveActor;
class FEdModeLandscape;
class ULandscapeMaterialInstanceConstant;
class UTexture2D;
enum class ELandscapeGizmoSnapType;

UENUM()
enum class ELandscapeToolFlattenMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Flatten may both raise and lower values */
	Both = 0,

	/** Flatten may only raise values, values above the clicked point will be left unchanged */
	Raise = 1,

	/** Flatten may only lower values, values below the clicked point will be left unchanged */
	Lower = 2,

	/** Flatten to closest specific terrace interval at the clicked point */
	Interval = 3,

	/** Flatten to specific terrace height intervals */
	Terrace = 4
};

UENUM()
enum class ELandscapeToolErosionMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Apply all erosion effects, both raising and lowering the heightmap */
	Both = 0,

	/** Only applies erosion effects that result in raising the heightmap */
	Raise = 1,

	/** Only applies erosion effects that result in lowering the heightmap */
	Lower = 2,
};

UENUM()
enum class ELandscapeToolHydroErosionMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Rains in some places and not others, randomly */
	Both = 0,

	/** Rain is applied to the entire area */
	Positive = 1,
};

UENUM()
enum class ELandscapeToolNoiseMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Noise will both raise and lower the heightmap */
	Both = 0,

	/** Noise will only raise the heightmap */
	Add = 1,

	/** Noise will only lower the heightmap */
	Sub = 2,
};

#if CPP
inline float NoiseModeConversion(ELandscapeToolNoiseMode Mode, float NoiseAmount, float OriginalValue)
{
	switch (Mode)
	{
	case ELandscapeToolNoiseMode::Add: // always +
		OriginalValue += NoiseAmount;
		break;
	case ELandscapeToolNoiseMode::Sub: // always -
		OriginalValue -= NoiseAmount;
		break;
	case ELandscapeToolNoiseMode::Both:
		break;
	}
	return OriginalValue;
}
#endif

UENUM()
enum class ELandscapeToolPasteMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Paste may both raise and lower values */
	Both = 0,

	/** Paste may only raise values, places where the pasted data would be below the heightmap are left unchanged. Good for copy/pasting mountains */
	Raise = 1,

	/** Paste may only lower values, places where the pasted data would be above the heightmap are left unchanged. Good for copy/pasting valleys or pits */
	Lower = 2,
};

UENUM()
enum class ELandscapeConvertMode : int8
{
	Invalid = -1 UMETA(Hidden),

	/** Will round up the number of components for the new world size, which might expand the world size compared to previous settings*/
	Expand = 0,

	/** Will floor the number of components for the new world size, which might reduce the world size compared to previous settings*/
	Clip = 1,

	/** The landscape will have the same overall size in the world, and have the same number of components. Existing landscape geometry and layer data will be resampled to match the new resolution. */
	Resample = 2,
};

UENUM()
enum class ELandscapeTextureColorChannel : int32
{
	Red,
	Green,
	Blue,
	Alpha,
};

UENUM()
enum class ELandscapeMirrorOperation : uint8
{
	MinusXToPlusX UMETA(DisplayName="-X to +X"),
	PlusXToMinusX UMETA(DisplayName="+X to -X"),
	MinusYToPlusY UMETA(DisplayName="-Y to +Y"),
	PlusYToMinusY UMETA(DisplayName="+Y to -Y"),
	RotateMinusXToPlusX UMETA(DisplayName="Rotate -X to +X"),
	RotatePlusXToMinusX UMETA(DisplayName="Rotate +X to -X"),
	RotateMinusYToPlusY UMETA(DisplayName="Rotate -Y to +Y"),
	RotatePlusYToMinusY UMETA(DisplayName="Rotate +Y to -Y"),
};

USTRUCT()
struct FGizmoImportLayer
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category="Import", EditAnywhere, meta = (DisplayName = "Target Layer Filename"))
	FString LayerFilename;

	UPROPERTY(Category="Import", EditAnywhere, meta = (DisplayName = "Target Layer Name", ToolTip = "Layer Name must match an existing target layer on the Landscape or the import will fail."))
	FString LayerName;

	UPROPERTY(Category="Import", EditAnywhere)
	bool bNoImport;

	FGizmoImportLayer()
		: LayerFilename("")
		, LayerName("")
		, bNoImport(false)
	{
	}
};

UENUM()
enum class ELandscapeImportHeightmapError
{
	None,
	FileNotFound,
	InvalidSize,
	CorruptFile,
	ColorPng,
	LowBitDepth,
};

UENUM()
enum class ELandscapeImportLayerError : uint8
{
	None,
	MissingLayerInfo,
	FileNotFound,
	FileSizeMismatch,
	CorruptFile,
	ColorPng,
};

USTRUCT()
struct FLandscapeImportLayer : public FLandscapeImportLayerInfo
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category="Import", VisibleAnywhere)
	TObjectPtr<ULandscapeMaterialInstanceConstant> ThumbnailMIC;

	UPROPERTY(Category = "Import", VisibleAnywhere)
	ELandscapeImportResult ImportResult;

	UPROPERTY(Category = "Import", VisibleAnywhere)
	FText ErrorMessage;

	UPROPERTY(Category="Export", EditAnywhere, meta = (DisplayName = "Layer File"))
	FString ExportFilePath;

	UPROPERTY(Category="Import", EditAnywhere)
	bool bSelected;

	FLandscapeImportDescriptor ImportDescriptor;

	FLandscapeImportLayer()
		: FLandscapeImportLayerInfo()
		, ThumbnailMIC(nullptr)
		, ImportResult(ELandscapeImportResult::Success)
		, ErrorMessage(FText())
		, bSelected(false)
	{
	}
};

USTRUCT()
struct FLandscapePatternBrushWorldSpaceSettings
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category="World-Space", EditAnywhere)
	FVector2D Origin;

	UPROPERTY(Category = "World-Space", EditAnywhere, Meta = (ClampMin = "-360", ClampMax = "360", UIMin = "-180", UIMax = "180"))
	float Rotation;

	// if true, the texture used for the pattern is centered on the PatternOrigin.
	// if false, the corner of the texture is placed at the PatternOrigin
	UPROPERTY(Category = "World-Space", EditAnywhere)
	bool bCenterTextureOnOrigin;

	UPROPERTY(Category = "World-Space", EditAnywhere)
	float RepeatSize;

	FLandscapePatternBrushWorldSpaceSettings()
		: Origin(ForceInit)
		, Rotation(0.0f)
		, bCenterTextureOnOrigin(false)
		, RepeatSize(0.0f)
	{}

	FLandscapePatternBrushWorldSpaceSettings(const FVector2D& InOrigin, float InRotation, bool bInCenterTextureOnOrigin, float bRepeatSize)
		: Origin(InOrigin)
		, Rotation(InRotation)
		, bCenterTextureOnOrigin(bInCenterTextureOnOrigin)
		, RepeatSize(bRepeatSize)
	{}
};

USTRUCT()
struct FLandscapeTargetLayerAssetFilePath
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Category = "Target Layers", EditAnywhere, NonTransactional, meta=(InlineEditConditionToggle))
	bool bUseAssetDirectoryPath = false;

	UPROPERTY(Category = "Target Layers", EditAnywhere, NonTransactional, meta = (EditCondition = bUseAssetDirectoryPath, LongPackageName))
	FDirectoryPath DirectoryPath;

	FLandscapeTargetLayerAssetFilePath()
		: bUseAssetDirectoryPath(false)
		, DirectoryPath()
	{
	}
};

UENUM()
enum class ELandscapeImportExportMode
{
	// Import and export only loaded landscape proxies
	LoadedOnly, 
	// Import and export the whole world loading & unloading regions as required.
	All,		
	// Import and export regions selected in the WP editor loading & unloading as required.
	// Selected, // TODO: don.boogert   
};

UENUM()
enum class ELandscapeWeightBlendPaintMode
{
	/** 
	 * Exclusive Mode for weight-blended layers will be active only when the Ctrl key is held. Non-Exclusive Mode is used otherwise.
	 * Note : this mode is incompatible with LandscapeEditorControlType "RequireCtrl" since the Ctrl key is then reserved for activating the landscape tools in that case. Use one of the manual modes in that case. 
	 */
	ExclusiveRequiresCtrl UMETA(DisplayName = "Hold Ctrl"),

	/** 
	 * Exclusive Mode for weight-blended layers will be active only when the Ctrl key is not held. Non-Exclusive Mode is used otherwise.
	 * Note : this mode is incompatible with LandscapeEditorControlType "RequireNoCtrl" since the Ctrl key is then reserved for activating the landscape tools in that case. Use one of the manual modes in that case. 
	 */
	ExclusiveRequiresNoCtrl UMETA(DisplayName = "Don't Hold Ctrl"),

	/** Force use Exclusive Mode for weight-blended layers. The Ctrl key is ineffective in that mode. */
	ForceExclusive UMETA(DisplayName = "Exclusive"),

	/** Force use Non-Exclusive Mode for weight-blended layers. The Ctrl key is ineffective in that mode. */
	ForceNonExclusive UMETA(DisplayName = "Non-Exclusive"),
};


UCLASS(MinimalAPI)
class ULandscapeEditorObject : public UObject
{
	GENERATED_UCLASS_BODY()

	FEdModeLandscape* ParentMode;


	// Common Tool Settings:

	// Strength of the Sculpt tool. If you're using a pen/tablet with pressure-sensing, the pressure used affects the strength of the tool.
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Tool Strength", ShowForTools="Paint,Sculpt,Erase,Smooth,Flatten,Erosion,HydraErosion,Noise,Mask,CopyPaste", ShowForTargetTypes = "Heightmap,Visibility",  ClampMin="0", ClampMax="10", UIMin="0", UIMax="1"))
	float ToolStrength;

	// Strength of the Paint tool. If you're using a pen/tablet with pressure-sensing, the pressure used affects the strength of the tool.
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Tool Strength", ShowForTools="Paint,Sculpt,Erase,Smooth,Flatten,Erosion,HydraErosion,Noise,Mask,CopyPaste", ShowForTargetTypes = "Weightmap", ClampMin="0", ClampMax="10", UIMin="0", UIMax="1"))
	float PaintToolStrength;

	// Enable to make tools blend towards a target value
	UPROPERTY(Category = "Tool Settings", NonTransactional, EditAnywhere, meta = (InlineEditConditionToggle))
	bool bUseWeightTargetValue;

	// Enable to make tools blend towards a target value
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Use Target Value", EditCondition="bUseWeightTargetValue", ShowForTools="Paint,Sculpt,Noise", ShowForTargetTypes = "Weightmap", ClampMin="0", ClampMax="10", UIMin="0", UIMax="1"))
	float WeightTargetValue;

	// I have no idea what this is for but it's used by the noise and erosion tools, and isn't exposed to the UI
	UPROPERTY(NonTransactional)
	float MaximumValueRadius;

	// Use the combined result of the underlying layers as input to the operation.  When not checked, it will use only the data in the currently selected layer as input.
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Flatten,Smooth,Erosion,HydraErosion,Ramp", ShowForTargetTypes="Heightmap"))
	bool bCombinedLayersOperation;

	// Flatten Tool:

	// Whether to flatten by lowering, raising, both or terracing
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Flatten"))
	ELandscapeToolFlattenMode FlattenMode;

	// Flattens to the angle of the clicked point, instead of horizontal
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Flatten", ShowForTargetTypes="Heightmap"))
	bool bUseSlopeFlatten;

	// Constantly picks new values to flatten towards when dragging around, instead of only using the first clicked point
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Flatten", ShowForTargetTypes="Heightmap"))
	bool bPickValuePerApply;

	// Enable to flatten towards a target height
	UPROPERTY(Category = "Tool Settings", NonTransactional, EditAnywhere, meta = (InlineEditConditionToggle))
	bool bUseFlattenTarget;

	// Target height to flatten towards (in Unreal Units)
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Flatten", ShowForTargetTypes="Heightmap", EditCondition="bUseFlattenTarget", UIMin="-32768", UIMax="32768"))
	float FlattenTarget;

	// Whether to show the preview grid for the flatten target height
	UPROPERTY(Category = "Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta = (DisplayName = "Show Preview Grid", ShowForTools = "Flatten", ShowForTargetTypes = "Heightmap", EditCondition = "bUseFlattenTarget", HideEditConditionToggle))
	bool bShowFlattenTargetPreview;

	// Height of the terrace intervals in unreal units, for the terrace flatten mode 
	UPROPERTY(Category = "Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta = (DisplayName = "Terrace Interval", ShowForTools = "Flatten", ShowForTargetTypes = "Heightmap", UIMin = "1", UIMax = "32768"))
	float TerraceInterval;

	// Smoothing value for terrace flatten mode
	UPROPERTY(Category = "Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta = (DisplayName = "Terrace Smoothing", ShowForTools = "Flatten", ShowForTargetTypes = "Heightmap", UIMin = "0.0001", UIMax = "1.0"))
	float TerraceSmooth;

	// Whether the Eye Dropper mode is activated
	UPROPERTY(NonTransactional, Transient)
	bool bFlattenEyeDropperModeActivated;

	// When in Eye Dropper mode, indicates whether we're currently mousing over the viewport 
	UPROPERTY(NonTransactional, Transient)
	bool bFlattenEyeDropperModeMousingOverViewport;

	UPROPERTY(NonTransactional, Transient)
	float FlattenEyeDropperModeDesiredTarget;

	// Ramp Tool:

	// Width of ramp
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Ramp", ClampMin="1", UIMin="1", UIMax="8192", SliderExponent=3))
	float RampWidth;

	// Falloff on side of ramp
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Side Falloff", ShowForTools="Ramp", ClampMin="0", ClampMax="1", UIMin="0", UIMax="1"))
	float RampSideFalloff;

	// Smooth Tool:

	// The radius smoothing is performed over
	// Higher values smooth out bigger details, lower values only smooth out smaller details
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Filter Kernel Radius", ShowForTools="Smooth", ClampMin="1", ClampMax="31", UIMin="0", UIMax="7"))
	int32 SmoothFilterKernelSize;

	// If checked, performs a detail preserving smooth using the specified detail smoothing value
	UPROPERTY(Category = "Tool Settings", NonTransactional, EditAnywhere, meta = (InlineEditConditionToggle))
	bool bDetailSmooth;

	// Larger detail smoothing values remove more details, while smaller values preserve more details
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Detail Smooth", EditCondition="bDetailSmooth", ShowForTools="Smooth", ClampMin="0", ClampMax="0.99"))
	float DetailScale;

	// Erosion Tool:

	// The minimum height difference necessary for the erosion effects to be applied. Smaller values will result in more erosion being applied
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Threshold", ShowForTools="Erosion", ClampMin="0", ClampMax="256", UIMin="0", UIMax="128"))
	int32 ErodeThresh;

	// The thickness of the surface for the layer weight erosion effect
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Surface Thickness", ShowForTools="Erosion", ClampMin="128", ClampMax="1024", UIMin="128", UIMax="512"))
	int32 ErodeSurfaceThickness;

	// Number of erosion iterations, more means more erosion but is slower
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Iterations", ShowForTools="Erosion", ClampMin="1", ClampMax="300", UIMin="1", UIMax="150"))
	int32 ErodeIterationNum;

	// Whether to erode by lowering, raising, or both
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Noise Mode", ShowForTools="Erosion"))
	ELandscapeToolErosionMode ErosionNoiseMode;

	// The size of the perlin noise filter used
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Noise Scale", ShowForTools="Erosion", ClampMin="1", ClampMax="512", UIMin="1.1", UIMax="256"))
	float ErosionNoiseScale;

	// Whether the erosion tool should take into account the paint layer's hardness parameter (a hardness of 0 means the layer is fully affected by erosion, while 1 means fully unaffected)
	UPROPERTY(Category = "Tool Settings", EditAnywhere, NonTransactional, meta = (DisplayName = "Use Layer Hardness", ShowForTools = "Erosion"))
	bool bErosionUseLayerHardness;

	// Hydraulic Erosion Tool:

	// The amount of rain to apply to the surface. Larger values will result in more erosion
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="HydraErosion", ClampMin="1", ClampMax="512", UIMin="1", UIMax="256"))
	int32 RainAmount;

	// The amount of sediment that the water can carry. Larger values will result in more erosion
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Sediment Capacity", ShowForTools="HydraErosion", ClampMin="0.1", ClampMax="1.0"))
	float SedimentCapacity;

	// Number of erosion iterations, more means more erosion but is slower
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Iterations", ShowForTools="HydraErosion", ClampMin="1", ClampMax="300", UIMin="1", UIMax="150"))
	int32 HErodeIterationNum;

	// Selects how rain is distributed over the brush area for hydro erosion
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Initial Rain Distribution", ShowForTools="HydraErosion"))
	ELandscapeToolHydroErosionMode RainDistMode;

	// The size of the noise filter for applying initial rain to the surface
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="HydraErosion", ClampMin="1", ClampMax="512", UIMin="1.1", UIMax="256"))
	float RainDistScale;

	// If checked, performs a detail-preserving smooth to the erosion effect using the specified detail smoothing value
	UPROPERTY(Category = "Tool Settings", NonTransactional, EditAnywhere, meta = (InlineEditConditionToggle))
	bool bHErosionDetailSmooth;

	// Larger detail smoothing values remove more details, while smaller values preserve more details
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Detail Smooth", EditCondition="bHErosionDetailSmooth", ShowForTools="HydraErosion", ClampMin="0", ClampMax="0.99"))
	float HErosionDetailScale;

	// Noise Tool:

	// Whether to apply noise that raises, lowers, or both
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Noise Mode", ShowForTools="Noise"))
	ELandscapeToolNoiseMode NoiseMode;

	// The size of the perlin noise filter used
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Noise Scale", ShowForTools="Noise", ClampMin="1", ClampMax="512", UIMin="1.1", UIMax="256"))
	float NoiseScale;

	// Mask Tool:

	// Uses selected region as a mask for other tools
	UPROPERTY(Category="Select Mask", EditAnywhere, NonTransactional, meta=(DisplayName="Use Region as Mask", ShowForTools="Mask", ShowForMask))
	bool bUseSelectedRegion;

	// If enabled, protects the selected region from changes
	// If disabled, only allows changes in the selected region
	UPROPERTY(Category="Select Mask", EditAnywhere, NonTransactional, meta=(DisplayName="Negative Mask", ShowForTools="Mask", ShowForMask))
	bool bUseNegativeMask;

	// Copy/Paste Tool:

	// Whether to paste will only raise, only lower, or both
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="CopyPaste"))
	ELandscapeToolPasteMode PasteMode;

	// If set, copies/pastes all layers, otherwise only copy/pastes the layer selected in the targets panel
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Gizmo copy/paste all layers", ShowForTools="CopyPaste"))
	bool bApplyToAllTargets;

	// The current gizmo actor. Only the gizmo location is revealed to the user interface
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, Transient, meta=(ShowForTools="ImportExport,CopyPaste"))
	TWeakObjectPtr<ALandscapeGizmoActiveActor> CurrentGizmoActor;

	// Allows to snap the copied/imported data to a grid
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Snap Gizmo to Landscape grid", ShowForTools="CopyPaste,ImportExport"))
	ELandscapeGizmoSnapType SnapMode;

	// Smooths the edges of the gizmo data into the landscape. Without this, the edges of the pasted data will be sharp
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Use Smooth Gizmo Brush", ShowForTools="CopyPaste"))
	bool bSmoothGizmoBrush;

	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta=(DisplayName="Heightmap", ShowForTools="CopyPaste"))
	FString GizmoHeightmapFilenameString;

	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta=(DisplayName="Heightmap Size", ShowForTools="CopyPaste"))
	FIntPoint GizmoImportSize;

	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, AdvancedDisplay, meta=(DisplayName="Target Layers", ShowForTools="CopyPaste"))
	TArray<FGizmoImportLayer> GizmoImportLayers;

	// Mirror Tool

	// Location of the mirror plane, defaults to the center of the landscape. Doesn't normally need to be changed!
	UPROPERTY(Category="Tool Settings", EditAnywhere, Transient, meta=(DisplayName="Mirror Point", ShowForTools="Mirror"))
	FVector2D MirrorPoint;

	// Type of mirroring operation to perform e.g. "Minus X To Plus X" copies and flips the -X half of the landscape onto the +X half
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Operation", ShowForTools="Mirror"))
	ELandscapeMirrorOperation MirrorOp;

	// Number of vertices either side of the mirror plane to smooth over
	UPROPERTY(Category="Tool Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Smoothing Width", ShowForTools="Mirror", ClampMin="0", UIMin="0", UIMax="20"))
	int32 MirrorSmoothingWidth;

	// Selects the blueprint brush to apply to the current edit layer. Click on the landscape to apply it.
	UPROPERTY(Category = "Tool Settings", EditAnywhere, Transient, meta = (DisplayName = "Blueprint Brush", ShowForTools = "BlueprintBrush"))
	TSubclassOf<ALandscapeBlueprintBrushBase> BlueprintBrush;

	// Resize Landscape Tool

	// Number of quads per landscape component section
	UPROPERTY(Category="Change Component Size", EditAnywhere, NonTransactional, meta=(DisplayName="Section Size", ShowForTools="ResizeLandscape"))
	int32 ResizeLandscape_QuadsPerSection;

	// Number of sections per landscape component
	UPROPERTY(Category="Change Component Size", EditAnywhere, NonTransactional, meta=(DisplayName="Sections Per Component", ShowForTools="ResizeLandscape"))
	int32 ResizeLandscape_SectionsPerComponent;

	// Number of components in resulting landscape
	UPROPERTY(Category="Change Component Size", EditAnywhere, NonTransactional, meta=(DisplayName="Number of Components", ShowForTools="ResizeLandscape"))
	FIntPoint ResizeLandscape_ComponentCount;

	// Determines how the new component size will be applied to the existing landscape geometry.
	UPROPERTY(Category="Change Component Size", EditAnywhere, NonTransactional, meta=(DisplayName="Resize Mode", ShowForTools="ResizeLandscape"))
	ELandscapeConvertMode ResizeLandscape_ConvertMode;

	int32 ResizeLandscape_Original_QuadsPerSection;
	int32 ResizeLandscape_Original_SectionsPerComponent;
	FIntPoint ResizeLandscape_Original_ComponentCount;

	// New Landscape "Tool"

	// Material initially applied to the landscape. Setting a material here exposes properties for setting up layer info based on the landscape blend nodes in the material.
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Material", ShowForTools="NewLandscape"))
	TWeakObjectPtr<UMaterialInterface> NewLandscape_Material;

	// The number of quads in a single landscape section. One section is the unit of LOD transition for landscape rendering.
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Section Size", ShowForTools="NewLandscape"))
	int32 NewLandscape_QuadsPerSection;

	// The number of sections in a single landscape component. This along with the section size determines the size of each landscape component. A component is the base unit of rendering and culling.
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Sections Per Component", ShowForTools="NewLandscape"))
	int32 NewLandscape_SectionsPerComponent;

	// The number of components in the X and Y direction, determining the overall size of the landscape.
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Number of Components", ShowForTools="NewLandscape"))
	FIntPoint NewLandscape_ComponentCount;

	static const FVector NewLandscape_DefaultLocation;

	// The location of the new landscape
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Location", ShowForTools="NewLandscape"))
	FVector NewLandscape_Location;

	static const FRotator NewLandscape_DefaultRotation;

	// The rotation of the new landscape
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Rotation", ShowForTools="NewLandscape"))
	FRotator NewLandscape_Rotation;

	static const FVector NewLandscape_DefaultScale;

	// The scale of the new landscape. This is the distance between each vertex on the landscape, defaulting to 100 units.
	UPROPERTY(Category="New Landscape", EditAnywhere, meta=(DisplayName="Scale", ShowForTools="NewLandscape"))
	FVector NewLandscape_Scale;

	UPROPERTY(Category="New Landscape", VisibleAnywhere, NonTransactional, meta=(ShowForTools="NewLandscape,ImportExport"))
	ELandscapeImportResult ImportLandscape_HeightmapImportResult;

	UPROPERTY(Category="New Landscape", VisibleAnywhere, NonTransactional, meta=(ShowForTools="NewLandscape,ImportExport"))
	FText ImportLandscape_HeightmapErrorMessage;

	// Specify a height map file in 16-bit RAW or PNG format
	UPROPERTY(Category="New Landscape", EditAnywhere, NonTransactional, meta=(DisplayName="Heightmap File", ShowForTools="NewLandscape,ImportExport"))
	FString ImportLandscape_HeightmapFilename;
	UPROPERTY(NonTransactional)
	uint32 ImportLandscape_Width;
	UPROPERTY(NonTransactional)
	uint32 ImportLandscape_Height;

	UPROPERTY(Category="Import / Export", EditAnywhere, NonTransactional, meta=(DisplayName="Heightmap File", ShowForTools="ImportExport"))
	FString HeightmapExportFilename;
		
	UPROPERTY(NonTransactional)
	FIntPoint ImportLandscape_GizmoLocalPosition;

	// Describes the transform that should be applied to the import data 
	UPROPERTY(Category = "Import / Export", EditAnywhere, NonTransactional, meta =(ShowForTools = "ImportExport"))
	ELandscapeImportTransformType ImportType;
		
	UPROPERTY(NonTransactional)
	bool bHeightmapSelected = false;
	
	UPROPERTY(Category = "Import / Export", EditAnywhere, NonTransactional, meta = (DisplayName="Export Selected Edit Layer", ShowForTools = "ImportExport", ToolTip="When true exports the selected edit layer, if false exports the blended result"))
	bool bExportEditLayer = true;

	UPROPERTY(Category = "Import / Export", EditAnywhere, NonTransactional, meta = (ShowForTools = "ImportExport", ToolTip = "(World Partition only) When true, exports the landscape as a single file, if false exports each grid tile individually."))
	bool bExportSingleFile = false;

	UPROPERTY(Category = "Import / Export", EditAnywhere, NonTransactional, meta = (ShowForTools = "ImportExport"))
	ELandscapeImportExportMode ImportExportMode = ELandscapeImportExportMode::LoadedOnly;

	UPROPERTY(NonTransactional)
	FLandscapeImportDescriptor HeightmapImportDescriptor;
	
	UPROPERTY(NonTransactional)
	int32 HeightmapImportDescriptorIndex;
private:
	UPROPERTY(NonTransactional)
	TArray<uint16> ImportLandscape_Data;
public:
	UE_DEPRECATED(5.7, "Non-edit layer landscapes are deprecated, all landscapes use the edit layer system now.")
	UPROPERTY(NonTransactional)
	bool bCanHaveLayersContent_DEPRECATED = true;

	UPROPERTY(Category = "New Landscape", EditAnywhere, NonTransactional, meta = (DisplayName = "Flip Y Axis", ToolTip = "Whether to flip Y coordinate of imported files.", ShowForTools = "NewLandscape,ImportExport"))
	bool bFlipYAxis = false;

	UPROPERTY(Category = "New Landscape", EditAnywhere, NonTransactional, meta = (DisplayName= "World Partition Grid Size", ToolTip="Number of components per landscape streaming proxies per axis", ShowForTools="NewLandscape", ClampMin=1, ClampMax=16, UIMin=1, UIMax=16))
	uint32 WorldPartitionGridSize = 2;

	UPROPERTY(Category = "New Landscape", EditAnywhere, NonTransactional, meta = (DisplayName= "World Partition Region Size", ToolTip="Number of components per Landscape World Partition Region per axis.", ShowForTools="NewLandscape", ClampMin=4, ClampMax=64, UIMin=4, UIMax=64))
	uint32 WorldPartitionRegionSize = 16;

	// Whether the imported alpha maps are to be interpreted as "layered" or "additive" (UE uses additive internally)
	UPROPERTY(Category="New Landscape", EditAnywhere, NonTransactional, meta=(DisplayName="Layer Alphamap Type", ShowForTools="NewLandscape,ImportExport"))
	ELandscapeImportAlphamapType ImportLandscape_AlphamapType;

	// The landscape layers that will be created via Import tab. Only layer names referenced in the material assigned above are shown here. Modify the material to add more layers.
	UPROPERTY(Category="New Landscape", EditAnywhere, NonTransactional, EditFixedSize, meta=(DisplayName="Layers", ShowForTools="ImportExport"))
	TArray<FLandscapeImportLayer> ImportLandscape_Layers;

	// The landscape layers that will be created via New tab. Only layer names referenced in the material assigned above are shown here. Modify the material to add more layers.
	UPROPERTY(Category="New Landscape", EditAnywhere, NonTransactional, EditFixedSize, meta=(DisplayName="Layers", ShowForTools="NewLandscape"))
	TArray<FLandscapeImportLayer> NewLandscape_Layers;

	// Common Brush Settings:

	// The radius of the sculpt brush, in unreal units
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Brush Size", ShowForBrushes="BrushSet_Circle,BrushSet_Alpha,BrushSet_Pattern", ShowForTargetTypes = "Heightmap,Visibility", ClampMin="1", ClampMax="65536", UIMin="1", UIMax="8192", SliderExponent="3", MaxFractionalDigits="2", ToolTip = "Radius of the sculpt editing brush. The maximum slider/clamp value can be set with the BrushSizeUIMax/BrushSizeClampMax values in the Landscape project settings."))
	float BrushRadius;

	// The radius of the paint brush, in unreal units
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Brush Size", ShowForBrushes="BrushSet_Circle,BrushSet_Alpha,BrushSet_Pattern", ShowForTargetTypes = "Weightmap", ClampMin="1", ClampMax="65536", UIMin="1", UIMax="8192", SliderExponent="3", ToolTip = "Radius of the paint editing brush. The maximum slider/clamp value can be set with the BrushSizeUIMax/BrushSizeClampMax values in the Landscape project settings."))
	float PaintBrushRadius;

	// The falloff at the edge of the sculpt brush, as a fraction of the brush's size. 0 = no falloff, 1 = all falloff
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Brush Falloff", ShowForBrushes="BrushSet_Circle,BrushSet_Gizmo,BrushSet_Pattern", ShowForTargetTypes = "Heightmap,Visibility", ClampMin="0", ClampMax="1", UIMin = "0", UIMax = "1"))
	float BrushFalloff;
	
	// The falloff at the edge of the paint brush, as a fraction of the brush's size. 0 = no falloff, 1 = all falloff
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Brush Falloff", ShowForBrushes="BrushSet_Circle,BrushSet_Gizmo,BrushSet_Pattern", ShowForTargetTypes = "Weightmap", ClampMin="0", ClampMax="1", UIMin = "0", UIMax = "1"))
	float PaintBrushFalloff;

	// Selects the Clay Brush painting mode
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(ShowForTools="Sculpt", ShowForBrushes="BrushSet_Circle,BrushSet_Alpha,BrushSet_Pattern"))
	bool bUseClayBrush;

	// Apply brush continuously while held stationary, after a small delay
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName = "Apply Without Moving", ShowForTargetTypes = "Heightmap", ShowForTools = "Sculpt,Erase,Smooth,Flatten,Erosion,HydraErosion,Noise,Mask,CopyPaste", ShowForBrushes = "BrushSet_Circle,BrushSet_Alpha,BrushSet_Gizmo,BrushSet_Pattern,BrushSet_Component"))
	bool bApplyWithoutMovingSculpt;

	// Apply brush continuously while held stationary, after a small delay
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName = "Apply Without Moving", ShowForTargetTypes = "Weightmap", ShowForTools = "Paint,Smooth,Flatten,Noise,Mask", ShowForBrushes = "BrushSet_Circle,BrushSet_Alpha,BrushSet_Pattern,BrushSet_Component"))
	bool bApplyWithoutMovingPaint;   // Internally, store a separate "Apply without Moving" value for paint mode

	// Alpha/Pattern Brush:

	// Scale of the brush texture. A scale of 1.000 maps the brush texture to the landscape at a 1 pixel = 1 vertex size
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture Scale", ShowForBrushes="BrushSet_Pattern", ClampMin="0.005", ClampMax="5", SliderExponent="3"))
	float AlphaBrushScale;

	// Rotate brush to follow mouse
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta = (DisplayName = "Auto-Rotate", ShowForBrushes = "BrushSet_Alpha"))
	bool bAlphaBrushAutoRotate;

	// Rotates the brush mask texture
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture Rotation", ShowForBrushes="BrushSet_Alpha,BrushSet_Pattern", ClampMin="-360", ClampMax="360", UIMin="-180", UIMax="180"))
	float AlphaBrushRotation;

	// Horizontally offsets the brush mask texture
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture Pan U", ShowForBrushes="BrushSet_Pattern", ClampMin="0", ClampMax="1"))
	float AlphaBrushPanU;

	// Vertically offsets the brush mask texture
	UPROPERTY(Category="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture Pan V", ShowForBrushes="BrushSet_Pattern", ClampMin="0", ClampMax="1"))
	float AlphaBrushPanV;

	// Tile the pattern in world space (this only takes scale and position into account, not rotation)
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta = (DisplayName = "Use World-Space", ShowForBrushes = "BrushSet_Pattern"))
	bool bUseWorldSpacePatternBrush;

	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta = (DisplayName = "World-Space Settings", ShowForBrushes = "BrushSet_Pattern", EditCondition = "bUseWorldSpacePatternBrush"))
	FLandscapePatternBrushWorldSpaceSettings WorldSpacePatternBrushSettings;

	// Mask texture to use
	UPROPERTY(Category  ="Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture", ShowForBrushes="BrushSet_Alpha,BrushSet_Pattern"))
	TObjectPtr<UTexture2D> AlphaTexture;

	// Channel of Mask Texture to use
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Texture Channel", ShowForBrushes="BrushSet_Alpha,BrushSet_Pattern"))
	ELandscapeTextureColorChannel AlphaTextureChannel;

	UPROPERTY(NonTransactional)
	int32 AlphaTextureSizeX;
	UPROPERTY(NonTransactional)
	int32 AlphaTextureSizeY;
	UPROPERTY(NonTransactional)
	TArray<uint8> AlphaTextureData;

	// Component Brush:

	// Number of components X/Y to affect at once. 1 means 1x1, 2 means 2x2, etc
	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta=(DisplayName="Brush Size", ShowForBrushes="BrushSet_Component", ClampMin="1", ClampMax="128", UIMin="1", UIMax="64", SliderExponent="3"))
	int32 BrushComponentSize;

	UPROPERTY(Category = "Brush Settings", EditAnywhere, NonTransactional, meta = (DisplayName = "Include Border", ShowForBrushes = "BrushSet_Component", ShowForTools = "Paint,Sculpt", ToolTip = "When true, the brush will affect all pixels within the component, including those on the border, which means neightboring components (which share a line/row with their neighbor) will be affected as well"))
	bool bBrushComponentIncludeBorder = true;


	// Target Layer Settings:

	/**
	 * Limits painting to only the components that already have the selected layer.
	 * Use the '=' key and activate the landscape brush to add the selected layer to the painted components' allow list. 
	 * Use the '-' key and activate the landscape brush to remove the selected layer from the painted components' allow list. 
	 */ 
	UPROPERTY(Category = "Target Layers", EditAnywhere, NonTransactional, meta=(ShowForTargetTypes="Weightmap,Visibility"))
	ELandscapeLayerPaintingRestriction PaintingRestriction;

	/*
	 * Defines how to activate the Exclusive Paint mode for weight-blended target layers.
	 * Exclusive Paint mode allows all weight-blended target layers from the same blend group to be painted in the same brush stroke. 
	 * The selected target layer is painted while the unselected ones are "inversely" painted (erased). 
	 * This allows the painted target layer to "override" the other ones instead of being merely being "blended" with them. It therefore (destructively) removes the weight from the other target layers. 
	 */
	UPROPERTY(Category = "Target Layers", EditAnywhere, NonTransactional, meta = (DisplayName = "Exclusive Paint Mode", ShowForTools="Paint", ShowForTargetTypes = "Weightmap"))
	ELandscapeWeightBlendPaintMode WeightBlendedTargetLayerPaintMode = ELandscapeWeightBlendPaintMode::ExclusiveRequiresNoCtrl;

	UPROPERTY(Category = "Target Layers", EditAnywhere, NonTransactional, meta = (ShowForTargetTypes = "Weightmap"))
	FLandscapeTargetLayerAssetFilePath TargetLayerAssetFilePath;

	// This allows to hide layer names that do not match the filter
	UPROPERTY(Transient)
	FString TargetLayersFilterString;

	// Display order of the targets
	UPROPERTY(Category = "Target Layers", EditAnywhere)
	ELandscapeLayerDisplayMode TargetDisplayOrder;

	UPROPERTY(Category = "Target Layers", EditAnywhere)
	bool bTargetDisplayOrdersAscending;

	// Show layers that are not currently being painted anywhere
	UPROPERTY(Category = "Target Layers", EditAnywhere)
	bool ShowUnusedLayers;	

#if WITH_EDITOR
	//~ Begin UObject Interface
	void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
#endif // WITH_EDITOR

	void Load();
	void Save();

	// Region
	void SetbUseSelectedRegion(bool InbUseSelectedRegion);
	void SetbUseNegativeMask(bool InbUseNegativeMask);

	// Copy/Paste
	void SetPasteMode(ELandscapeToolPasteMode InPasteMode);

	// Alpha/Pattern Brush
	void SetAlphaTexture(UTexture2D* InTexture, ELandscapeTextureColorChannel InTextureChannel);
	bool HasValidAlphaTextureData() const;

	// New Landscape
	FString LastImportPath;

	const TArray<uint16>& GetImportLandscapeData() const { return ImportLandscape_Data; }
	void ClearImportLandscapeData() { ImportLandscape_Data.Empty(); }
	void ChooseBestComponentSizeForImport();
	void ImportLandscapeData();
	void RefreshImportLayersList(bool bRefreshFromTarget = false, bool bRegenerateThumbnails = false);
	void AutoFillTargetLayers(const FString& DefaultTargetLayerAssetPath, const bool bCreateAssetForEmptyLayers);
	ELandscapeImportResult CreateImportLayersInfo(TArray<FLandscapeImportLayerInfo>& OutImportLayerInfos);
	ELandscapeImportResult CreateNewLayersInfo(TArray<FLandscapeImportLayerInfo>& OutNewLayerInfos);
	void InitializeDefaultHeightData(TArray<uint16>& OutData);
	void ExpandImportData(TArray<uint16>& OutHeightData, TArray<FLandscapeImportLayerInfo>& OutImportLayerInfos);
	void UpdateComponentLayerAllowList();
	bool UseSingleFileImport() const;
	void OnChangeImportLandscapeResolution(int32 DescriptorIndex);
	void OnImportHeightmapFilenameChanged() { RefreshImports(); }
	void RefreshImports();
	void OnImportWeightmapFilenameChanged() { RefreshLayerImports(); }
	void RefreshLayerImports();
	void RefreshLayerImport(FLandscapeImportLayer& ImportLayer);

	int32 ClampLandscapeSize(int32 InComponentsCount) const
	{
		// Max size is either whole components below 8192 verts, or 256 components
		return FMath::Clamp(InComponentsCount, 1, FMath::Min(256, FMath::FloorToInt(8191.0f / (float)(NewLandscape_SectionsPerComponent * NewLandscape_QuadsPerSection))));
	}
	
	int32 CalcComponentsCount(int32 InResolution) const
	{
		return ClampLandscapeSize(InResolution / (NewLandscape_SectionsPerComponent * NewLandscape_QuadsPerSection));
	}

	void NewLandscape_ClampSize()
	{	
		NewLandscape_ComponentCount.X = ClampLandscapeSize(NewLandscape_ComponentCount.X);
		NewLandscape_ComponentCount.Y = ClampLandscapeSize(NewLandscape_ComponentCount.Y);
	}

	void UpdateComponentCount()
	{
		// ignore invalid cases
		if (ResizeLandscape_QuadsPerSection == 0 || ResizeLandscape_SectionsPerComponent == 0 ||ResizeLandscape_ComponentCount.X == 0 || ResizeLandscape_ComponentCount.Y == 0) 
		{
			return;
		}
		const int32 ComponentSizeQuads = ResizeLandscape_QuadsPerSection * ResizeLandscape_SectionsPerComponent;
		const int32 Original_ComponentSizeQuads = ResizeLandscape_Original_QuadsPerSection * ResizeLandscape_Original_SectionsPerComponent;
		const FIntPoint OriginalResolution = ResizeLandscape_Original_ComponentCount * Original_ComponentSizeQuads;
		switch (ResizeLandscape_ConvertMode)
		{
		case ELandscapeConvertMode::Expand:
			ResizeLandscape_ComponentCount.X = FMath::DivideAndRoundUp(OriginalResolution.X, ComponentSizeQuads);
			ResizeLandscape_ComponentCount.Y = FMath::DivideAndRoundUp(OriginalResolution.Y, ComponentSizeQuads);
			break;
		case ELandscapeConvertMode::Clip:
			ResizeLandscape_ComponentCount.X = FMath::Max(1, OriginalResolution.X / ComponentSizeQuads);
			ResizeLandscape_ComponentCount.Y = FMath::Max(1, OriginalResolution.Y / ComponentSizeQuads);
			break;
		case ELandscapeConvertMode::Resample:
			ResizeLandscape_ComponentCount = ResizeLandscape_Original_ComponentCount;
			break;
		default:
			check(0);
		}
	}

	void SetGizmoSnapMode(ELandscapeGizmoSnapType SnapMode);

	void SetParent(FEdModeLandscape* LandscapeParent)
	{
		ParentMode = LandscapeParent;
	}

	void UpdateTargetLayerDisplayOrder();

	float GetCurrentToolStrength() const;
	void SetCurrentToolStrength(float NewToolStrength);

	float GetCurrentToolBrushRadius() const;
	void SetCurrentToolBrushRadius(float NewBrushStrength);

	float GetCurrentToolBrushFalloff() const;
	void SetCurrentToolBrushFalloff(float NewBrushFalloff);

	float GetFlattenTarget(bool bInReturnPreviewValueIfActive) const;

private:

	bool IsWeightmapTarget() const;

	static bool LoadAlphaTextureSourceData(UTexture2D* InTexture, TArray<uint8>& OutSourceData, int32& OutSourceDataSizeX, int32& OutSourceDataSizeY, ELandscapeTextureColorChannel& InOutTextureChannel);
};
