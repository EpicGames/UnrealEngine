// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "FileHelpers.h"
#include "Containers/Map.h"

class ALandscapeProxy;
class ALandscapeStreamingProxy;
class ULandscapeLayerInfoObject;
class FMenuBuilder;
struct FEditLayerMenuBlock;
using FEditLayerCategoryToEntryMap = TMap<FName, FEditLayerMenuBlock>;

namespace LandscapeEditorUtils
{
	bool LANDSCAPEEDITOR_API SetHeightmapData(ALandscapeProxy* Landscape, const TArray<uint16>& Data);
	bool LANDSCAPEEDITOR_API SetWeightmapData(ALandscapeProxy* Landscape, ULandscapeLayerInfoObject* LayerObject, const TArray<uint8>& Data);

	int32 GetMaxSizeInComponents();
	TOptional<FString> GetImportExportFilename(const FString& InDialogTitle, const FString& InStartPath, const FString& InDialogTypeString, bool bInImporting);

	template<typename T>
	void SaveObjects(TArrayView<T*> InObjects)
	{
		TArray<UPackage*> Packages;
		Algo::Transform(InObjects, Packages, [](UObject* InObject) { return InObject->GetPackage(); });
		UEditorLoadingAndSavingUtils::SavePackages(Packages, /* bOnlyDirty = */ false);
	}

	void SaveLandscapeProxies(UWorld* InWorld, TArrayView<ALandscapeProxy*> Proxies);

	TArray<FAssetData> GetLandscapeTargetLayerInfoAssets();
	TOptional<FAssetData> FindLandscapeTargetLayerInfoAsset(const FName& LayerName, const FString& TargetLayerAssetPackagePath);

	// Edit layer context menu customization
	void BuildContextMenuFromCategoryEntryMap(const FEditLayerCategoryToEntryMap& InMap, FMenuBuilder& OutMenuBuilder);
}
