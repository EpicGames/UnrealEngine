// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Containers/ArrayView.h"
#include "Misc/Paths.h"
#include "LandscapeFileFormatInterface.generated.h"

class Error;

struct FLandscapeFileTypeInfo
{
	// Description of file type for the file selector
	FText Description;

	// Extensions for this type, with leading dot, e.g. ".png"
	TArray<FString, TInlineAllocator<2>> Extensions;

	// Whether this file type supports exporting from the editor back to file
	// (All file types must support *importing*, but exporting is optional)
	bool bSupportsExport = false;
};

UENUM()
enum class ELandscapeImportResult : uint8
{
	Success = 0,
	Warning,
	Error,
};

USTRUCT()
struct FLandscapeFileResolution
{
	GENERATED_USTRUCT_BODY()

	FLandscapeFileResolution() {}
	FLandscapeFileResolution(uint32 InWidth, uint32 InHeight) : Width(InWidth), Height(InHeight) {}

	UPROPERTY()
	uint32 Width = 0;

	UPROPERTY()
	uint32 Height = 0;
};

FORCEINLINE bool operator==(const FLandscapeFileResolution& Lhs, const FLandscapeFileResolution& Rhs)
{
	return Lhs.Width == Rhs.Width && Lhs.Height == Rhs.Height;
}

FORCEINLINE bool operator!=(const FLandscapeFileResolution& Lhs, const FLandscapeFileResolution& Rhs)
{
	return !(Lhs == Rhs);
}

struct FLandscapeFileInfo
{
	// Whether the the file is usable or has errors/warnings
	ELandscapeImportResult ResultCode = ELandscapeImportResult::Success;
	
	// Message to show as the warning/error result
	FText ErrorMessage;

	// Normally contains a single resolution, but .raw is awful
	TArray<FLandscapeFileResolution> PossibleResolutions;

	// The inherent scale of the data format, if it has one, in centimeters
	// The default for data with no inherent scale is 100,100,0.78125 (100.0/128, shown as 100 in the editor UI)
	TOptional<FVector> DataScale;
};

template< class T >
struct FLandscapeImportData
{
	// Whether the import data is usable or has errors/warnings
	ELandscapeImportResult ResultCode = ELandscapeImportResult::Success;

	// Message to show as the warning/error result
	FText ErrorMessage;

	// The Data
	TArray<T> Data;
};

using FLandscapeHeightmapImportData = FLandscapeImportData<uint16>;
using FLandscapeWeightmapImportData = FLandscapeImportData<uint8>;

using FLandscapeHeightmapInfo = FLandscapeFileInfo;
using FLandscapeWeightmapInfo = FLandscapeFileInfo;

// Interface
template< class T >
class ILandscapeFileFormat
{
public:
	/** Gets info about this format
	 * @return information about the file types supported by this file format plugin
	 */
	virtual const FLandscapeFileTypeInfo& GetInfo() const = 0;

	/** Validate a file for Import
	 * Gives the file format the opportunity to reject a file or return warnings
	 * as well as return information about the file for the import UI (e.g. resolution and scale)
	 * @param Filename path to the file to validate for import
	 * @param LayerName name of layer that is being imported (in case of Heightmap this will be NAME_None)
	 * @return information about the file and (optional) error message
	 */
	virtual FLandscapeFileInfo Validate(const TCHAR* Filename, FName LayerName) const = 0;

	virtual FLandscapeFileInfo Validate(const TCHAR* Filename) const { return Validate(Filename, NAME_None); }

	/** Import a file
	 * @param Filename path to the file to import
	 * @param LayerName name of layer being imported (in case of Heightmap this will be NAME_None)
	 * @param ExpectedResolution resolution selected in the import UI (mostly for the benefit of .raw)
	 * @return imported data and (optional) error message
	 */
	virtual FLandscapeImportData<T> Import(const TCHAR* Filename, FName LayerName, FLandscapeFileResolution ExpectedResolution) const = 0;

	virtual FLandscapeImportData<T> Import(const TCHAR* Filename, FLandscapeFileResolution ExpectedResolution) const { return Import(Filename, NAME_None, ExpectedResolution); }

	/** Export a file (if supported)
	 * @param Filename path to the file to export to
	 * @param LayerName name of layer being exported (in case of Heightmap this will be NAME_None)
	 * @param Data raw data to export
	 * @param DataResolution resolution of Data
	 * @param Scale scale of the landscape data, in centimeters
	 */
	virtual void Export(const TCHAR* Filename, FName LayerName, TArrayView<const T> Data, FLandscapeFileResolution DataResolution, FVector Scale) const
	{
		checkf(0, TEXT("File type hasn't implemented support for export - %s"), *FPaths::GetExtension(Filename, true));
	}

	virtual void Export(const TCHAR* Filename, TArrayView<const T> Data, FLandscapeFileResolution DataResolution, FVector Scale)
	{
		Export(Filename, NAME_None, Data, DataResolution, Scale);
	}

	/**
	 * Note: Even though this is an interface class we need a virtual destructor as derived objects are deleted via a pointer to this interface
	 */
	virtual ~ILandscapeFileFormat() = default;
};

using ILandscapeHeightmapFileFormat = ILandscapeFileFormat<uint16>;
using ILandscapeWeightmapFileFormat = ILandscapeFileFormat<uint8>;

