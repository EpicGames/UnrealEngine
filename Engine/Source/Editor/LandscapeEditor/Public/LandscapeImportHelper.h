// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "LandscapeFileFormatInterface.h"
#include "Templates/UnrealTypeTraits.h"
#include "Templates/EnableIf.h"
#include "LandscapeImportHelper.generated.h"

#define UE_API LANDSCAPEEDITOR_API

USTRUCT()
struct FLandscapeImportFileDescriptor
{
	GENERATED_USTRUCT_BODY()

	FLandscapeImportFileDescriptor(const FString& InFilePath, const FIntPoint& InCoord)
		: Coord(InCoord), FilePath(InFilePath){}

	FLandscapeImportFileDescriptor() 
		: Coord(EForceInit::ForceInit) {}

	/* Which tile does that descriptor represent */
	UPROPERTY()
	FIntPoint Coord;
	/* File path */
	UPROPERTY()
	FString FilePath;
};

USTRUCT()
struct FLandscapeImportResolution
{
	GENERATED_USTRUCT_BODY()

	FLandscapeImportResolution(uint32 InWidth, uint32 InHeight) : Width(InWidth), Height(InHeight) {}

	FLandscapeImportResolution() {}

	UPROPERTY()
	uint32 Width = 0;
	UPROPERTY()
	uint32 Height = 0;
};

UENUM()
enum class ELandscapeImportTransformType : int8
{

	None UMETA(DisplayName="Original", ToolTip="Import the data at the gizmo location in the original size"),
	ExpandOffset UMETA(DisplayName="Expand", ToolTip="Import the data at the gizmo location and expand the data to fill the landscape") ,
	ExpandCentered UMETA(Hidden), 
	Resample UMETA(ToolTip="Resample import data to fit landscape"),
	Subregion UMETA(ToolTip="Import sub-region of the image to landscape")
};

FORCEINLINE bool operator==(const FLandscapeImportResolution& Lhs, const FLandscapeImportResolution& Rhs)
{
	return Lhs.Width == Rhs.Width && Lhs.Height == Rhs.Height;
}

FORCEINLINE bool operator!=(const FLandscapeImportResolution& Lhs, const FLandscapeImportResolution& Rhs)
{
	return !(Lhs == Rhs);
}

USTRUCT()
struct FLandscapeImportDescriptor
{
	GENERATED_USTRUCT_BODY()

	FLandscapeImportDescriptor()
		: Scale(100, 100, 100) {}

	void Reset()
	{
		ImportResolutions.Reset();
		FileResolutions.Reset();
		FileDescriptors.Reset();
		Scale = FVector(100, 100, 100);
	}

	int32 FindDescriptorIndex(int32 ImportWidth, int32 ImportHeight)
	{
		for (int32 Index = 0; Index < ImportResolutions.Num(); ++Index)
		{
			if (ImportResolutions[Index].Width == ImportWidth && ImportResolutions[Index].Height == ImportHeight)
			{
				return Index;
			}
		}

		return INDEX_NONE;
	}

	/* Landscape Import Resolution based on File Coords + Resolutions */
	UPROPERTY()
	TArray<FLandscapeImportResolution> ImportResolutions;
	/* Single File Resolutions */
	UPROPERTY()
	TArray<FLandscapeFileResolution> FileResolutions;
	/* Files contributing to this descriptor */
	UPROPERTY()
	TArray<FLandscapeImportFileDescriptor> FileDescriptors;
	/* Scale of Import data */
	UPROPERTY()
	FVector Scale;
};

class FLandscapeImportHelper
{
public:
	/**
	 * @param FilePath File to import when bSingleFile is true.
	 * @param bSingleFile If false FilePath will be used as the seed to find matching "_xA_yB" files in the same directory.
	 * @param OutImportDescriptor Descriptor output which contains all the information needed to call GetHeightmapImportData.
	 * @param OutMessage In case the result of the operation is  an error or warning, this will contain the reason.
	 * @return If the operation was a Success/Error/Warning. 
	 */
	static UE_API ELandscapeImportResult GetHeightmapImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage);

	/**
	 * @param FilePath File to import when bSingleFile is true.
	 * @param bSingleFile If false FilePath will be used as the seed to find matching "_xA_yB" files in the same directory.
	 * @param LayerName Name of the Weightmap layer being imported. This can be used by the file formats.
	 * @param OutImportDescriptor Descriptor output which contains all the information needed to call GetHeightmapImportData.
	 * @param OutMessage In case the result of the operation is  an error or warning, this will contain the reason.
	 * @return If the operation was a Success/Error/Warning.
	 */
	static UE_API ELandscapeImportResult GetWeightmapImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FName LayerName, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage);

	static UE_API ELandscapeImportResult GetHeightmapImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, TArray<uint16>& OutData, FText& OutMessage);
	static UE_API ELandscapeImportResult GetWeightmapImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, FName LayerName, TArray<uint8>& OutData, FText& OutMessage);

	static UE_API void TransformWeightmapImportData(const TArray<uint8>& InData, TArray<uint8>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0,0));
	static UE_API void TransformHeightmapImportData(const TArray<uint16>& InData, TArray<uint16>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0, 0));

	static UE_API void ChooseBestComponentSizeForImport(int32 Width, int32 Height, int32& InOutQuadsPerSection, int32& InOutSectionsPerComponent, FIntPoint& OutComponentCount);

	static UE_API bool ExtractCoordinates(const FString& BaseFilename, FIntPoint& OutCoord, FString& OutBaseFilePattern);
	static UE_API void GetMatchingFiles(const FString& FilePathPattern, TArray<FString>& OutFilePaths);


	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint16>, ELandscapeImportResult>::Type GetImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FName LayerName, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage)
	{
		return GetHeightmapImportDescriptor(FilePath, bSingleFile, bFlipYAxis, OutImportDescriptor, OutMessage);
	}

	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint16>, ELandscapeImportResult>::Type GetImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, FName LayerName, TArray<T>& OutData, FText& OutMessage)
	{
		return GetHeightmapImportData(ImportDescriptor, DescriptorIndex, OutData, OutMessage);
	}
	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint16>, void>::Type TransformImportData(const TArray<T>& InData, TArray<T>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0, 0))
	{
		return TransformHeightmapImportData(InData, OutData, CurrentResolution, RequiredResolution, TransformType, Offset);
	}

	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint8>, ELandscapeImportResult>::Type GetImportDescriptor(const FString& FilePath, bool bSingleFile, bool bFlipYAxis, FName LayerName, FLandscapeImportDescriptor& OutImportDescriptor, FText& OutMessage)
	{
		return GetWeightmapImportDescriptor(FilePath, bSingleFile, bFlipYAxis, LayerName, OutImportDescriptor, OutMessage);
	}
	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint8>, ELandscapeImportResult>::Type GetImportData(const FLandscapeImportDescriptor& ImportDescriptor, int32 DescriptorIndex, FName LayerName, TArray<T>& OutData, FText& OutMessage)
	{
		return GetWeightmapImportData(ImportDescriptor, DescriptorIndex, LayerName, OutData, OutMessage);
	}
	template<typename T>
	static typename TEnableIf<std::is_same_v<T, uint8>, void>::Type TransformImportData(const TArray<T>& InData, TArray<T>& OutData, const FLandscapeImportResolution& CurrentResolution, const FLandscapeImportResolution& RequiredResolution, ELandscapeImportTransformType TransformType = ELandscapeImportTransformType::ExpandCentered, FIntPoint Offset = FIntPoint(0, 0))
	{
		return TransformWeightmapImportData(InData, OutData, CurrentResolution, RequiredResolution, TransformType, Offset);
	}
};

#undef UE_API
