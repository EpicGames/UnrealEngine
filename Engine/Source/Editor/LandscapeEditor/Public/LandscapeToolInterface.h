// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "UObject/GCObject.h"
#include "UnrealWidgetFwd.h"
#include "EdMode.h"
#include "LandscapeEdit.h"
#include "LandscapeEditTypes.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class ULandscapeInfo;
class ULandscapeLayerInfoObject;
class UMaterialInstance;
class UMaterialInterface;
class UViewportInteractor;
struct FViewportClick;

// FLandscapeToolMousePosition - Struct to store mouse positions since the last time we applied the brush
struct FLandscapeToolInteractorPosition
{
	// Stored in heightmap space.
	FVector2D Position = FVector2D::Zero();
	bool bModifierPressed = false;
	bool bAlternateModifierPressed = false;

	FLandscapeToolInteractorPosition(FVector2D InPosition, const bool bInModifierPressed, const bool bInAlternateModifierPressed)
		: Position(InPosition)
		, bModifierPressed(bInModifierPressed)
		, bAlternateModifierPressed(bInAlternateModifierPressed)
	{
	}
};

enum class ELandscapeBrushType
{
	Normal = 0,
	Alpha,
	Component,
	Gizmo,
	Splines
};

class FLandscapeBrushData
{
protected:
	FIntRect Bounds;
	TArray<float> BrushAlpha;

public:
	FLandscapeBrushData()
		: Bounds()
	{
	}

	FLandscapeBrushData(FIntRect InBounds)
		: Bounds(InBounds)
	{
		BrushAlpha.SetNumZeroed(Bounds.Area());
	}

	FIntRect GetBounds() const
	{
		return Bounds;
	}

	// For compatibility with older landscape code that uses inclusive bounds in 4 int32s
	void GetInclusiveBounds(int32& X1, int32& Y1, int32& X2, int32& Y2) const
	{
		X1 = Bounds.Min.X;
		Y1 = Bounds.Min.Y;
		X2 = Bounds.Max.X - 1;
		Y2 = Bounds.Max.Y - 1;
	}

	float* GetDataPtr(FIntPoint Position)
	{
		return BrushAlpha.GetData() + (Position.Y - Bounds.Min.Y) * Bounds.Width() + (Position.X - Bounds.Min.X);
	}
	const float* GetDataPtr(FIntPoint Position) const
	{
		return BrushAlpha.GetData() + (Position.Y - Bounds.Min.Y) * Bounds.Width() + (Position.X - Bounds.Min.X);
	}

	FORCEINLINE explicit operator bool() const
	{
		return BrushAlpha.Num() != 0;
	}

	FORCEINLINE bool operator!() const
	{
		return !(bool)*this;
	}
};

class FLandscapeBrush : public FGCObject
{
public:
	virtual void MouseMove(float LandscapeX, float LandscapeY) = 0;
	virtual TOptional<FVector2D> GetLastMousePosition() const { return TOptional<FVector2D>(); }
	virtual FLandscapeBrushData ApplyBrush(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) = 0;

	struct FOverlapInfo
	{
		FORCEINLINE explicit operator bool() const
		{
			return ExclusiveBounds.Area() > 0;
		}

		FORCEINLINE bool operator!() const
		{
			return !(bool)*this;
		}

		FIntRect ExclusiveBounds;
		bool bOverlapsUnloadedComponents = false;
		TSet<ULandscapeComponent*> OverlappedLoadedComponents;
	};
	/** @return landscape overlap information about the components being overlapped by the tool interactor positions. */
	virtual FOverlapInfo GetOverlapInfo(const TArray<FLandscapeToolInteractorPosition>& InteractorPositions) const { return FOverlapInfo(); };

	virtual TOptional<bool> InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) { return TOptional<bool>(); }
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) {};
	virtual void BeginStroke(float LandscapeX, float LandscapeY, class FLandscapeTool* CurrentTool);
	virtual void EndStroke();
	virtual void EnterBrush() {}
	virtual void LeaveBrush() {}
	virtual ~FLandscapeBrush() = default;
	virtual UMaterialInterface* GetBrushMaterial() { return nullptr; }
	virtual const TCHAR* GetBrushName() = 0;
	virtual FText GetDisplayName() = 0;
	virtual ELandscapeBrushType GetBrushType() { return ELandscapeBrushType::Normal; }

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}
	virtual FString GetReferencerName() const override
	{
		return TEXT("FLandscapeBrush");
	}
};

struct FLandscapeBrushSet
{
	FLandscapeBrushSet(const TCHAR* InBrushSetName)
		: BrushSetName(InBrushSetName)
		, PreviousBrushIndex(0)
	{
	}

	const FName BrushSetName;
	TArray<FLandscapeBrush*> Brushes;
	int32 PreviousBrushIndex;

	virtual ~FLandscapeBrushSet()
	{
		for (int32 BrushIdx = 0; BrushIdx < Brushes.Num(); BrushIdx++)
		{
			delete Brushes[BrushIdx];
		}
	}
};

namespace ELandscapeToolTargetTypeMask
{
	enum UE_DEPRECATED(5.7, "Use ELandscapeToolTargetTypeFlags instead.")  Type : uint8
	{
		Heightmap  = 1 << static_cast<uint8>(ELandscapeToolTargetType::Heightmap),
		Weightmap  = 1 << static_cast<uint8>(ELandscapeToolTargetType::Weightmap),
		Visibility = 1 << static_cast<uint8>(ELandscapeToolTargetType::Visibility),

		NA = 0,
		All = 0xFF,
	};

	UE_DEPRECATED(5.7, "Use UE::Landscape::GetLandscapeToolTargetTypeAsFlags with ELandscapeToolTargetType instead.")
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline ELandscapeToolTargetTypeMask::Type FromType(ELandscapeToolTargetType TargetType)
	{
		if (TargetType == ELandscapeToolTargetType::Invalid)
		{
			return ELandscapeToolTargetTypeMask::NA;
		}
		return (ELandscapeToolTargetTypeMask::Type)(1 << static_cast<uint8>(TargetType));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

struct FLandscapeToolTarget
{
	TWeakObjectPtr<ULandscapeInfo> LandscapeInfo;
	ELandscapeToolTargetType TargetType;

	// TODO [jonathan.bard] : eventually, will become multiple layers (multi-paint) : don't assume there will be a single one of these
	TWeakObjectPtr<ULandscapeLayerInfoObject> LayerInfo;
	FName LayerName;
	
	TArray<ULandscapeLayerInfoObject*> GetLayerInfos() const
	{
		return { LayerInfo.Get() };
	}

	FLandscapeToolTarget()
		: LandscapeInfo()
		, TargetType(ELandscapeToolTargetType::Heightmap)
		, LayerInfo()
		, LayerName(NAME_None)
	{
	}
};

enum class ELandscapeToolType
{
	Normal = 0,
	Mask,
};

/** Describes the type of action that a landscape tool will perform on a given target layer */
enum class ELandscapeTargetLayerActionType
{
	None, // The target layer will not be affected by any action
	Normal, // The target layer will be affected by the action (e.g. the paint action by the Paint brush)
	Inverted, // The target layer will be inversely affected by the action (e.g. the inverse paint action by the Paint brush in exclusive mode)
};

/**
 * FLandscapeTool
 */
class FLandscapeTool : public FGCObject
{
public:
	virtual void EnterTool() {}
	virtual bool IsToolActive() const { return false;  }
	virtual void ExitTool() {}
	virtual bool BeginTool(FEditorViewportClient* ViewportClient, const FLandscapeToolTarget& Target, const FVector& InHitLocation) = 0;
	virtual void EndTool(FEditorViewportClient* ViewportClient) = 0;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime) {};
	virtual bool MouseEnter(FEditorViewportClient* InViewportClient, FViewport* InViewport, int32 MouseX, int32 MouseY) { return true; }
	virtual bool MouseLeave(FEditorViewportClient* InViewportClient, FViewport* InViewport) { return true; }
	virtual bool MouseMove(FEditorViewportClient* ViewportClient, FViewport* Viewport, int32 x, int32 y) = 0;
	virtual bool HandleClick(HHitProxy* HitProxy, const FViewportClick& Click) { return false; }
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) { return false; }
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) { return false; }
	virtual bool GetCursor(EMouseCursor::Type& OutCursor) const { return false;  }

	/** FEdMode: Called when a transform operation begins */
	virtual bool BeginTransform(const FGizmoState& InState) { return false; }

	/** FEdMode: Called when a transform operation ends */
	virtual bool EndTransform(const FGizmoState& InState) { return false; }

	FLandscapeTool() {}
	virtual ~FLandscapeTool() = default;
	virtual const TCHAR* GetToolName() const = 0;
	virtual FText GetDisplayName() const = 0;
	virtual FText GetDisplayMessage() const = 0;
	virtual void SetEditRenderType();
	virtual void Render(const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI) {}
	virtual bool HitTrace(const FVector& TraceStart, const FVector& TraceEnd, FVector& OutHitLocation) { return false; }
	virtual bool UseSphereTrace() const { return true; }
	virtual bool SupportsMask() const { return true; }
	virtual bool OverrideSelection() const { return false; }
	virtual bool IsSelectionAllowed(AActor* InActor, bool bInSelection) const { return false; }
	virtual bool UsesTransformWidget() const { return false; }
	virtual EAxisList::Type GetWidgetAxisToDraw(UE::Widget::EWidgetMode InWidgetMode) const { return EAxisList::All; }
	virtual bool AffectsEditLayers() const { return true; };

	virtual bool OverrideWidgetLocation() const { return true; }
	virtual bool OverrideWidgetRotation() const { return true; }
	virtual FVector GetWidgetLocation() const { return FVector::ZeroVector; }
	virtual FMatrix GetWidgetRotation() const { return FMatrix::Identity; }
	virtual bool DisallowMouseDeltaTracking() const { return false; }

	/** Get override cursor visibility settings */
	virtual bool GetOverrideCursorVisibility(bool& bWantsOverride, bool& bHardwareCursorVisible, bool bSoftwareCursorVisible) const { return false; }

	/** Called before mouse movement is converted to drag/rot */
	virtual bool PreConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false; }

	/** Called after mouse movement is converted to drag/rot */
	virtual bool PostConvertMouseMovement(FEditorViewportClient* InViewportClient) { return false; }

	virtual void SetCanToolBeActivated(bool Value) { }
	virtual bool CanToolBeActivated() const { return true;  }
	
	virtual EEditAction::Type GetActionEditDuplicate() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditDelete() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCut() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditCopy() { return EEditAction::Skip; }
	virtual EEditAction::Type GetActionEditPaste() { return EEditAction::Skip; }
	virtual bool ProcessEditDuplicate() { return false; }
	virtual bool ProcessEditDelete() { return false; }
	virtual bool ProcessEditCut() { return false; }
	virtual bool ProcessEditCopy() { return false; }
	virtual bool ProcessEditPaste() { return false; }

	/** Returns the resolution difference when the Tool action is applied. */
	virtual int32 GetToolActionResolutionDelta() const { return 0; }

	// Functions which doesn't need Viewport data...
	virtual void Process(int32 Index, int32 Arg) {}
	virtual ELandscapeToolType GetToolType() const { return ELandscapeToolType::Normal; }
	/** 
	 * @return the type of target type that this tool supports. Some tools might support both heightmaps and weightmaps, for example, but they should be specialized 
	 *  to return only one type (e.g. FLandscapeToolSmooth, that is templated to either FHeightmapToolTarget (ELandscapeToolTargetType::Heightmap) or FWeightmapToolTarget (ELandscapeToolTargetType::Weightmap)
	 *  That is the reason why we return ELandscapeToolTargetType here, rather than ELandscapeToolTargetTypeFlags
	 */
	virtual ELandscapeToolTargetType GetSupportedTargetType() const { return ELandscapeToolTargetType::Invalid; }

	/** 
	 * Returns the type of action that this tool will perform on the target layer
	 * @param InLayerInfo - the target layer for which to return the action type
	 * @param bInIsModifierPressed indicates the status of the standard modifier key (Shift) : some tools might return a different action type depending on the modifier pressed
	 * @param bInIsAlternateModifierPressed indicates the status of the alternate modifier key (Ctrl) : some tools might return a different action type depending on the modifier pressed
	 * @return the action type 
	 */
	virtual ELandscapeTargetLayerActionType GetTargetLayerActionType(const ULandscapeLayerInfoObject* InLayerInfo, bool bInIsModifierPressed, bool bInIsAlternateModifierPressed) const
	{ 
		return ELandscapeTargetLayerActionType::None; 
	}

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override {}
	virtual FString GetReferencerName() const override
	{
		return FString(TEXT("FLandscapeTool::")).Append(GetToolName());
	}

public:
	int32 PreviousBrushIndex = INDEX_NONE;
	TArray<FName> ValidBrushes;
};
