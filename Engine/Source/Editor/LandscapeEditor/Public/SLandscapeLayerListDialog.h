// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SListView.h"	// SWidgetLayerListView
#include "Dialog/SCustomDialog.h"		// SLandscapeLayerListDialog

class FWidgetLayerListDragDropOp;
class ALandscape;
struct FWidgetLayerListItem;
typedef SListView<TSharedPtr<FWidgetLayerListItem> > SWidgetLayerListView;

/**
 *	Displays a list of layers with drag + drop support for layer reordering outside of landscape mode.
 *	Can be directly constructed in editor code or can be constructed indirectly through ILandscapeEditorServices.
 */
class SLandscapeLayerListDialog : public SCustomDialog
{
public:
	virtual ~SLandscapeLayerListDialog() = default;
	
	SLATE_BEGIN_ARGS( SLandscapeLayerListDialog	){}
	SLATE_END_ARGS()

	/**
	 * Construct this widget. Called by the SNew() Slate macro.
	 * @param InArgs Declaration used by the SNew() macro to construct this widget
	 * @param InLayers The array of landscape layers which should be displayed
	 */
	void Construct(const FArguments& InArgs, const TWeakObjectPtr<ALandscape>& InLandscape);

	int32 GetInsertedLayerIndex() const { return InsertedLayerIndex; }

private:
	/* Called when LayerList is changed to update WidgetLayerList and the dialog window. */
	void OnLayerListUpdated();
	
	/* Called when a new row is being generated. Converts FWidgetLayerListItem to ITableRow. */
	TSharedRef<ITableRow> OnGenerateRow( TSharedPtr<FWidgetLayerListItem> InListItem, const TSharedRef< STableViewBase >& InOwnerTableView ) const;

	/* Called when Complete button is pressed */
	void OnComplete();

	/* The list view of WidgetLayerList. */
	TSharedPtr<SWidgetLayerListView> LayerListView;

	/* Array of FWidgetLayerListItem, which contains ULandscapeEditLayerBase* in addition to intermediate data.
	 * Note: This array is a UI reflection of LayerList but the order is the reverse order of LayerList */
	TArray<TSharedPtr<FWidgetLayerListItem>> WidgetLayerList;

	TWeakObjectPtr<ALandscape> Landscape;

	/* Index of the layer which is being inserted into the layer stack */
	int InsertedLayerIndex = -1;
};
